{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Geometry Script",
	"Description": "Geometry Script provides a library of functions for creating and editing Meshes in Blueprints and Python",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": false,
	"IsBetaVersion":  false,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeometryScriptingCore",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "GeometryScriptingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "PlanarCut",
			"Enabled":  true
		}
	]
}
