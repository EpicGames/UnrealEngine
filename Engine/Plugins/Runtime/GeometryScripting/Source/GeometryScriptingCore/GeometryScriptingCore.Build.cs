// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryScriptingCore : ModuleRules
{
	public GeometryScriptingCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);


		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);


		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"PhysicsCore",
				"RenderCore",
				"GeometryCore",
				"GeometryFramework",
				"DynamicMesh"
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"MeshConversion",
				"MeshConversionEngineTypes",
				"GeometryAlgorithms",
				"ModelingOperators",
				"ModelingComponents",
				"PlanarCut",
				"Chaos",
				"GeometryCollectionEngine"
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

		// add UnrealEd dependency if building for editor
		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("ModelingOperatorsEditorOnly");
			PrivateDependencyModuleNames.Add("MeshReductionInterface"); // for UE standard simplification 
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
