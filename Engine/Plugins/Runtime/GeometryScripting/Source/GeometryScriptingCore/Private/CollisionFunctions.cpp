// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/CollisionFunctions.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Operations/MeshConvexHull.h"
#include "Operations/MeshProjectionHull.h"
#include "UDynamicMesh.h"
#include "Components/DynamicMeshComponent.h"
#include "Spatial/FastWinding.h"
#include "ProjectionTargets.h"
#include "MeshSimplification.h"

#include "Selections/MeshConnectedComponents.h"
#include "DynamicSubmesh3.h"
#include "Polygroups/PolygroupUtil.h"

#include "ShapeApproximation/ShapeDetection3.h"
#include "ShapeApproximation/MeshSimpleShapeApproximation.h"
#include "MinVolumeSphere3.h"
#include "MinVolumeBox3.h"
#include "FitCapsule3.h"

#include "Clustering/FaceNormalClustering.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "OrientedBoxTypes.h"
#include "MeshQueries.h"
#include "MeshAdapter.h"
#include "SphereTypes.h"

#include "Generators/MeshShapeGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/CapsuleGenerator.h"

// physics data
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"

// requires ModelingComponents
#include "Physics/CollisionGeometryConversion.h"
#include "Physics/PhysicsDataCollection.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionFunctions)

#if WITH_EDITOR
#include "Editor.h"
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_CollisionFunctions"


namespace UELocal
{

void ComputeCollisionFromMesh(
	const FDynamicMesh3& Mesh, 
	FKAggregateGeom& GeneratedCollision, 
	FGeometryScriptCollisionFromMeshOptions& Options)
{
	FPhysicsDataCollection NewCollision;

	FMeshConnectedComponents Components(&Mesh);
	Components.FindConnectedTriangles();
	int32 NumComponents = Components.Num();

	TArray<FDynamicMesh3> Submeshes;
	TArray<const FDynamicMesh3*> SubmeshPointers;

	if (NumComponents == 1)
	{
		SubmeshPointers.Add(&Mesh);
	}
	else
	{
		Submeshes.SetNum(NumComponents);
		SubmeshPointers.SetNum(NumComponents);
		ParallelFor(NumComponents, [&](int32 k)
		{
			FDynamicSubmesh3 Submesh(&Mesh, Components[k].Indices, (int32)EMeshComponents::None, false);
			Submeshes[k] = MoveTemp(Submesh.GetSubmesh());
			SubmeshPointers[k] = &Submeshes[k];
		});
	}

	FMeshSimpleShapeApproximation ShapeGenerator;
	ShapeGenerator.InitializeSourceMeshes(SubmeshPointers);

	ShapeGenerator.bDetectSpheres = Options.bAutoDetectSpheres;
	ShapeGenerator.bDetectBoxes = Options.bAutoDetectBoxes;
	ShapeGenerator.bDetectCapsules = Options.bAutoDetectCapsules;

	ShapeGenerator.MinDimension = Options.MinThickness;

	switch (Options.Method)
	{
	case EGeometryScriptCollisionGenerationMethod::AlignedBoxes:
		ShapeGenerator.Generate_AlignedBoxes(NewCollision.Geometry);
		break;
	case EGeometryScriptCollisionGenerationMethod::OrientedBoxes:
		ShapeGenerator.Generate_OrientedBoxes(NewCollision.Geometry);
		break;
	case EGeometryScriptCollisionGenerationMethod::MinimalSpheres:
		ShapeGenerator.Generate_MinimalSpheres(NewCollision.Geometry);
		break;
	case EGeometryScriptCollisionGenerationMethod::Capsules:
		ShapeGenerator.Generate_Capsules(NewCollision.Geometry);
		break;
	case EGeometryScriptCollisionGenerationMethod::ConvexHulls:
		ShapeGenerator.bSimplifyHulls = Options.bSimplifyHulls;
		ShapeGenerator.HullTargetFaceCount = Options.ConvexHullTargetFaceCount;
		ShapeGenerator.HullGeometricToleranceAfterTarget = (double)Options.ConvexHullGeometricTolerance;
		if (Options.MaxConvexHullsPerMesh > 1)
		{
			ShapeGenerator.ConvexDecompositionMaxPieces = Options.MaxConvexHullsPerMesh;
			ShapeGenerator.ConvexDecompositionSearchFactor = Options.ConvexDecompositionSearchFactor;
			ShapeGenerator.ConvexDecompositionErrorTolerance = Options.ConvexDecompositionErrorTolerance;
			ShapeGenerator.ConvexDecompositionMinPartThickness = Options.ConvexDecompositionMinPartThickness;
			ShapeGenerator.Generate_ConvexHullDecompositions(NewCollision.Geometry);
		}
		else
		{
			ShapeGenerator.Generate_ConvexHulls(NewCollision.Geometry);
		}
		break;
	case EGeometryScriptCollisionGenerationMethod::SweptHulls:
		ShapeGenerator.bSimplifyHulls = Options.bSimplifyHulls;
		ShapeGenerator.HullSimplifyTolerance = Options.SweptHullSimplifyTolerance;
		ShapeGenerator.Generate_ProjectedHulls(NewCollision.Geometry, 
			static_cast<FMeshSimpleShapeApproximation::EProjectedHullAxisMode>(Options.SweptHullAxis));
		break;
	case EGeometryScriptCollisionGenerationMethod::MinVolumeShapes:
		ShapeGenerator.Generate_MinVolume(NewCollision.Geometry);
		break;
	case EGeometryScriptCollisionGenerationMethod::LevelSets:
		ShapeGenerator.Generate_LevelSets(NewCollision.Geometry);
		break;
	}

	if (Options.bRemoveFullyContainedShapes && Components.Num() > 1)
	{
		NewCollision.Geometry.RemoveContainedGeometry();
	}

	if (Options.MaxShapeCount > 0 && Options.MaxShapeCount < Components.Num())
	{
		NewCollision.Geometry.FilterByVolume(Options.MaxShapeCount);
	}

	NewCollision.CopyGeometryToAggregate();
	GeneratedCollision = NewCollision.AggGeom;
}

static void UpdateStaticMeshCollision(UStaticMesh* StaticMeshAsset, TFunctionRef<bool(UStaticMesh*, UBodySetup*)> ApplyUpdate, bool bEmitTransaction, bool bMarkCollisionAsCustomized = true)
{
	if (!ensure(StaticMeshAsset))
	{
		return;
	}
#if WITH_EDITOR
	if (bEmitTransaction && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("UpdateStaticMesh", "Update Static Mesh Collision"));

		StaticMeshAsset->Modify();
	}
#endif

	UBodySetup* BodySetup = StaticMeshAsset->GetBodySetup();
	if (BodySetup != nullptr)
	{
		// mark the BodySetup for modification.
#if WITH_EDITOR
		if (bEmitTransaction)
		{
			BodySetup->Modify();
		}
#endif
	}

	if (ApplyUpdate(StaticMeshAsset, BodySetup))
	{
		if (BodySetup)
		{
			// rebuild physics meshes
			BodySetup->CreatePhysicsMeshes();
		}

		StaticMeshAsset->RecreateNavCollision();

		// update physics state on all components using this StaticMesh
		for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
		{
			UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(*Iter);
			if (SMComponent->GetStaticMesh() == StaticMeshAsset)
			{
				if (SMComponent->IsPhysicsStateCreated())
				{
					SMComponent->RecreatePhysicsState();
				}
			}
		}

		// do we need to do a post edit change here??

		// mark static mesh as dirty so it gets resaved?
		StaticMeshAsset->MarkPackageDirty();

#if WITH_EDITORONLY_DATA
		// mark the static mesh as having customized collision so it is not regenerated on reimport
		if (bMarkCollisionAsCustomized)
		{
			StaticMeshAsset->SetCustomizedCollision(true);
		}
#endif // WITH_EDITORONLY_DATA
	}

#if WITH_EDITOR
	if (bEmitTransaction && GEditor)
	{
		GEditor->EndTransaction();
	}
#endif

}

static void SetStaticMeshSimpleCollision(UStaticMesh* StaticMeshAsset, const FKAggregateGeom& NewSimpleCollision, bool bEmitTransaction, bool bMarkCollisionAsCustomized = true)
{
	UpdateStaticMeshCollision(StaticMeshAsset, [&NewSimpleCollision](UStaticMesh*, UBodySetup* BodySetup) -> bool
	{
		if (!BodySetup)
		{
			return false;
		}
		// clear existing simple collision. This will call BodySetup->InvalidatePhysicsData()
		BodySetup->RemoveSimpleCollision();

		// set new collision geometry
		BodySetup->AggGeom = NewSimpleCollision;

		// update collision type
		//BodySetup->CollisionTraceFlag = (ECollisionTraceFlag)(int32)Settings->SetCollisionType;

		return true;
	}, bEmitTransaction, bMarkCollisionAsCustomized);
}


// local helper to convert the blueprint-accessible enum to the geometrycore equivalent
static UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod ConvertNegativeSpaceSampleMethodEnum(ENegativeSpaceSampleMethod SampleMethod)
{
	switch (SampleMethod)
	{
	case ENegativeSpaceSampleMethod::Uniform:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::Uniform;
	case ENegativeSpaceSampleMethod::VoxelSearch:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch;
	case ENegativeSpaceSampleMethod::NavigableVoxelSearch:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch;
	}
	return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::Uniform;
}

static void ApplyNegativeSpaceToleranceDirectionalScales(UE::Geometry::FNegativeSpaceSampleSettings& NegativeSpaceSettings, const TArray<FNegativeSpaceDirectionalToleranceScale>& ToleranceDirectionalScales)
{
	if (ToleranceDirectionalScales.IsEmpty())
	{
		return;
	}

	// Copy the input scalars to transform them for use by the custom scale function
	TArray<FNegativeSpaceDirectionalToleranceScale> TransformedScalars = ToleranceDirectionalScales;
	double MinScale = 1.0;

	for (FNegativeSpaceDirectionalToleranceScale& Scalar : TransformedScalars)
	{
		Scalar.Direction.Normalize();
		Scalar.AngleRange = FMath::Max(FMathd::ZeroTolerance, FMath::Cos(FMathd::DegToRad * Scalar.AngleRange)); // transform to dot product threshold
		Scalar.ScaleFactor = FMath::Clamp(Scalar.ScaleFactor, .1, 1.0); // clamp to supported range
		MinScale = FMath::Min(Scalar.ScaleFactor, MinScale);
	}
	TransformedScalars.Sort([](const FNegativeSpaceDirectionalToleranceScale& A, const FNegativeSpaceDirectionalToleranceScale& B)
		{
			return A.ScaleFactor < B.ScaleFactor;
		});
	if (MinScale < 1.0)
	{
		NegativeSpaceSettings.MinCustomMarginScale = MinScale;
		NegativeSpaceSettings.CustomRadiusMarginScale =
			[Scalars = MoveTemp(TransformedScalars)](const FDynamicMesh3& Mesh, int32 NearTID, FVector3d Pos)
			{
				FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(Mesh, NearTID, Pos);
				FVector3d ToSurfaceDir = Query.ClosestTrianglePoint - Pos;
				if (ToSurfaceDir.Normalize())
				{
					for (const FNegativeSpaceDirectionalToleranceScale& Scalar : Scalars)
					{
						if (Scalar.Direction.Dot(ToSurfaceDir) > Scalar.AngleRange)
						{
							return Scalar.ScaleFactor;
						}
					}
				}
				return 1.0;
			};
	}
}

// local helper to convert negative space settings from the blueprint-accessible struct to the geometrycore equivalent
static FNegativeSpaceSampleSettings ConvertNegativeSpaceOptions(const FComputeNegativeSpaceOptions& NegativeSpaceOptions)
{
	UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;
	NegativeSpaceSettings.TargetNumSamples = NegativeSpaceOptions.TargetNumSamples;
	NegativeSpaceSettings.MinRadius = NegativeSpaceOptions.MinRadius;
	NegativeSpaceSettings.ReduceRadiusMargin = NegativeSpaceOptions.NegativeSpaceTolerance;
	NegativeSpaceSettings.MinSpacing = NegativeSpaceOptions.MinSampleSpacing;
	NegativeSpaceSettings.SampleMethod = ConvertNegativeSpaceSampleMethodEnum(NegativeSpaceOptions.SampleMethod);
	NegativeSpaceSettings.bRequireSearchSampleCoverage = NegativeSpaceOptions.bRequireSearchSampleCoverage;
	NegativeSpaceSettings.bOnlyConnectedToHull = NegativeSpaceOptions.bOnlyConnectedToHull;
	NegativeSpaceSettings.MaxVoxelsPerDim = NegativeSpaceOptions.MaxVoxelsPerDim;
	ApplyNegativeSpaceToleranceDirectionalScales(NegativeSpaceSettings, NegativeSpaceOptions.ToleranceDirectionalScales);
	NegativeSpaceSettings.Sanitize();
	return NegativeSpaceSettings;
}

// local helper to append a convex elem to a compact dynamic mesh, if it has more than a given number of tris
static bool AppendConvexElemToCompactDynamicMesh(const FKConvexElem& Elem, FDynamicMesh3& Mesh, int32 MinTris = 0)
{
	checkSlow(Mesh.IsCompact());
	if (Elem.IndexData.Num() <= MinTris * 3)
	{
		return false;
	}

	int32 StartV = Mesh.MaxVertexID();
	for (FVector V : Elem.VertexData)
	{
		Mesh.AppendVertex(Elem.GetTransform().TransformPosition(V));
	}
	for (int32 TriStart = 0; TriStart + 2 < Elem.IndexData.Num(); TriStart += 3)
	{
		Mesh.AppendTriangle(StartV + Elem.IndexData[TriStart], StartV + Elem.IndexData[TriStart + 2], StartV + Elem.IndexData[TriStart + 1]);
	}

	return true;
}


static double GetConvexElemVolume(const FKConvexElem& Convex)
{
	// Note: Not reliable to use the FKConvexElem::GetVolume function because it depends on the chaos convex being allocated, and also is not currently exported
	TIndexMeshArrayAdapter<int32, double, FVector3d> HullMeshAdapter(&Convex.VertexData, &Convex.IndexData);
	// Note: We take the negative volume because the hull triangles have opposite winding from ordinary meshes
	double Volume = -TMeshQueries<TIndexMeshArrayAdapter<int32, double, FVector3d>>::GetVolumeArea(HullMeshAdapter).X * Convex.GetTransform().GetDeterminant();
	return Volume;
}

}		// end namespace UELocal


bool UGeometryScriptLibrary_CollisionFunctions::SetStaticMeshCustomComplexCollision(
	UStaticMesh* StaticMeshAsset,
	UStaticMesh* StaticMeshCollisionAsset,
	bool bEmitTransaction,
	bool bMarkCollisionAsCustomized,
	UGeometryScriptDebug* Debug
)
{
	if (StaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCustomComplexCollision_InvalidInput", "SetStaticMeshCustomComplexCollision: StaticMeshAsset is Null"));
		return false;
	}
	if (StaticMeshCollisionAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCustomComplexCollision_InvalidInput2", "SetStaticMeshCustomComplexCollision: StaticMeshCollisionAsset is Null"));
		return false;
	}
#if WITH_EDITORONLY_DATA
	UELocal::UpdateStaticMeshCollision(StaticMeshAsset, [StaticMeshCollisionAsset](UStaticMesh* UpdateStaticMesh, UBodySetup* BodySetup) -> bool
	{
		if (BodySetup)
		{
			BodySetup->InvalidatePhysicsData();
		}

		// set new collision geometry
		UpdateStaticMesh->ComplexCollisionMesh = StaticMeshCollisionAsset;

		return true;
	}, bEmitTransaction, bMarkCollisionAsCustomized);
	return true;
#else
	return false;
#endif
}

UDynamicMesh* UGeometryScriptLibrary_CollisionFunctions::SetStaticMeshCollisionFromMesh(
	UDynamicMesh* FromDynamicMesh,
	UStaticMesh* ToStaticMeshAsset,
	FGeometryScriptCollisionFromMeshOptions Options,
	FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions,
	UGeometryScriptDebug* Debug)
{
	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCollisionFromMesh_InvalidInput1", "SetStaticMeshCollisionFromMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}
	if (ToStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCollisionFromMesh_InvalidInput2", "SetStaticMeshCollisionFromMesh: ToStaticMeshAsset is Null"));
		return FromDynamicMesh;
	}

	FKAggregateGeom NewCollision;
	FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		UELocal::ComputeCollisionFromMesh(ReadMesh, NewCollision, Options);
	});

	UELocal::SetStaticMeshSimpleCollision(ToStaticMeshAsset, NewCollision, Options.bEmitTransaction, StaticMeshCollisionOptions.bMarkAsCustomized);

	return FromDynamicMesh;
}




void UGeometryScriptLibrary_CollisionFunctions::SetStaticMeshCollisionFromComponent(
	UStaticMesh* UpdateStaticMeshAsset, 
	UPrimitiveComponent* SourceComponent,
	FGeometryScriptSetSimpleCollisionOptions Options,
	FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions,
	UGeometryScriptDebug* Debug)
{
	if (UpdateStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCollisionFromComponent_InvalidStaticMesh", "SetStaticMeshCollisionFromComponent: UpdateStaticMeshAsset is Null"));
		return;
	}
	if (SourceComponent == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCollisionFromComponent_InvalidSourceComponent", "SetStaticMeshCollisionFromComponent: SourceComponent is Null"));
		return;
	}

	const UBodySetup* BodySetup = SourceComponent->GetBodySetup();
	if (BodySetup == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetStaticMeshCollisionFromComponent_InvalidBodySetup", "SetStaticMeshCollisionFromComponent: SourceComponent BodySetup is Null"));
		return;
	}

	UELocal::SetStaticMeshSimpleCollision(UpdateStaticMeshAsset, BodySetup->AggGeom, Options.bEmitTransaction, StaticMeshCollisionOptions.bMarkAsCustomized);
}


bool UGeometryScriptLibrary_CollisionFunctions::StaticMeshHasCustomizedCollision(UStaticMesh* StaticMeshAsset)
{
#if WITH_EDITORONLY_DATA
	if (!StaticMeshAsset)
	{
		UE_LOGF(LogGeometry, Warning, "StaticMeshHasCustomizedCollision: StaticMeshAsset is Null");
		return false;
	}
	return StaticMeshAsset->GetCustomizedCollision();
#else
	return false;
#endif
}



UDynamicMesh* UGeometryScriptLibrary_CollisionFunctions::SetDynamicMeshCollisionFromMesh(
	UDynamicMesh* FromDynamicMesh,
	UDynamicMeshComponent* DynamicMeshComponent,
	FGeometryScriptCollisionFromMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetDynamicMeshCollisionFromMesh_InvalidInput1", "SetDynamicMeshCollisionFromMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}
	if (DynamicMeshComponent == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetDynamicMeshCollisionFromMesh_InvalidInput2", "SetDynamicMeshCollisionFromMesh: ToDynamicMeshComponent is Null"));
		return FromDynamicMesh;
	}

	FKAggregateGeom NewCollision;
	FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		UELocal::ComputeCollisionFromMesh(ReadMesh, NewCollision, Options);
	});

#if WITH_EDITOR
	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("UpdateDynamicMesh", "Set Simple Collision"));

		DynamicMeshComponent->Modify();
	}
#endif

#if WITH_EDITOR
	if (Options.bEmitTransaction)
	{
		UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup();
		if (BodySetup != nullptr)
		{
			BodySetup->Modify();
		}
	}
#endif

	// set new collision geometry
	DynamicMeshComponent->SetSimpleCollisionShapes(NewCollision, true /*bUpdateCollision*/);

	// do we need to do a post edit change here??

#if WITH_EDITOR
	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->EndTransaction();
	}
#endif

	return FromDynamicMesh;
}


void UGeometryScriptLibrary_CollisionFunctions::ResetDynamicMeshCollision(
	UDynamicMeshComponent* DynamicMeshComponent,
	bool bEmitTransaction,
	UGeometryScriptDebug* Debug)
{
	if (DynamicMeshComponent == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ResetDynamicMeshCollision_InvalidInput2", "ResetDynamicMeshCollision: Component is Null"));
		return;
	}

#if WITH_EDITOR
	if (bEmitTransaction && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("ResetDynamicMeshCollisionTransaction", "Clear Simple Collision"));
		DynamicMeshComponent->Modify();
	}
#endif

#if WITH_EDITOR
	if (bEmitTransaction)
	{
		// mark the BodySetup for modification.
		UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup();
		if (BodySetup != nullptr)
		{
			BodySetup->Modify();
		}
	}
#endif

	// clear existing simple collision.
	DynamicMeshComponent->ClearSimpleCollisionShapes(true /*bUpdateCollision*/);

	// do we need to do a post edit change here??

#if WITH_EDITOR
	if (bEmitTransaction && GEditor)
	{
		GEditor->EndTransaction();
	}
#endif

}

void UGeometryScriptLibrary_CollisionFunctions::ResetSimpleCollision(FGeometryScriptSimpleCollision & SimpleCollision)
{
	SimpleCollision.AggGeom.EmptyElements();
}


FGeometryScriptSimpleCollision
UGeometryScriptLibrary_CollisionFunctions::GenerateCollisionFromMesh(
	UDynamicMesh* FromDynamicMesh,
	FGeometryScriptCollisionFromMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptSimpleCollision ToRet;
	
	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GenerateCollisionFromMesh_InvalidInput", "GenerateCollisionFromMesh: FromDynamicMesh is Null"));
		return ToRet;
	}

	FromDynamicMesh->ProcessMesh([&ToRet, &Options](const FDynamicMesh3& ReadMesh)
	{
		UELocal::ComputeCollisionFromMesh(ReadMesh, ToRet.AggGeom, Options);
	});

	return ToRet;
}


FGeometryScriptSimpleCollision
UGeometryScriptLibrary_CollisionFunctions::GetSimpleCollisionFromComponent(
	UPrimitiveComponent* Component,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptSimpleCollision ToRet;
	if (Component == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSimpleCollisionFromComponent_InvalidComponent", "GetSimpleCollisionFromComponent: Component is Null"));
		return ToRet;
	}
	const UBodySetup* BodySetup = Component->GetBodySetup();
	if (BodySetup == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSimpleCollisionFromComponent_InvalidBodySetup", "GetSimpleCollisionFromComponent: Component's BodySetup is Null"));
		return ToRet;
	}
	ToRet.AggGeom = BodySetup->AggGeom;
	
	return ToRet;
}

void UGeometryScriptLibrary_CollisionFunctions::SetSimpleCollisionOfDynamicMeshComponent(
	const FGeometryScriptSimpleCollision& SimpleCollision,
	UDynamicMeshComponent* DynamicMeshComponent,
	FGeometryScriptSetSimpleCollisionOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (DynamicMeshComponent == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSimpleCollisionOfDynamicMeshComponent_InvalidComponent", "SetSimpleCollisionOfDynamicMeshComponent: Component is Null"));
		return;
	}

#if WITH_EDITOR
	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("UpdateDynamicMesh", "Set Simple Collision"));

		DynamicMeshComponent->Modify();
	}
#endif

#if WITH_EDITOR
	if (Options.bEmitTransaction)
	{
		UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup();
		if (BodySetup != nullptr)
		{
			BodySetup->Modify();
		}
	}
#endif

	// set new collision geometry
	DynamicMeshComponent->SetSimpleCollisionShapes(SimpleCollision.AggGeom, true /*bUpdateCollision*/);

	// do we need to do a post edit change here??

#if WITH_EDITOR
	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->EndTransaction();
	}
#endif
}


FGeometryScriptSimpleCollision UGeometryScriptLibrary_CollisionFunctions::GetSimpleCollisionFromStaticMesh(
	UStaticMesh* StaticMeshAsset, UGeometryScriptDebug* Debug)
{
	FGeometryScriptSimpleCollision ToRet;
	
	if (StaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSimpleCollisionFromStaticMesh_InvalidStaticMesh", "GetSimpleCollisionFromStaticMesh: Input Mesh is Null"));
		return ToRet;
	}
	const UBodySetup* BodySetup = StaticMeshAsset->GetBodySetup();
	if (BodySetup == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSimpleCollisionFromStaticMesh_InvalidBodySetup", "GetSimpleCollisionFromStaticMesh: Input Mesh's BodySetup is Null"));
		return ToRet;
	}

	ToRet.AggGeom = BodySetup->AggGeom;

	return ToRet;
}

void UGeometryScriptLibrary_CollisionFunctions::SetSimpleCollisionOfStaticMesh(
	const FGeometryScriptSimpleCollision& SimpleCollision,
	UStaticMesh* StaticMesh, 
	FGeometryScriptSetSimpleCollisionOptions Options,
	FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions,
	UGeometryScriptDebug* Debug)
{
	if (StaticMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSimpleCollisionOfStaticMesh_InvalidStaticMesh", "SetSimpleCollisionOfStaticMesh: Input Mesh is Null"));
		return;
	}
	UELocal::SetStaticMeshSimpleCollision(StaticMesh, SimpleCollision.AggGeom, Options.bEmitTransaction, StaticMeshCollisionOptions.bMarkAsCustomized);
}

FGeometryScriptSimpleCollision UGeometryScriptLibrary_CollisionFunctions::TransformSimpleCollisionShapes(
	const FGeometryScriptSimpleCollision& SimpleCollision,
	FTransform Transform,
	const FGeometryScriptTransformCollisionOptions& TransformOptions,
	bool& bSuccess,
	UGeometryScriptDebug* Debug)
{
	bSuccess = true;

	FGeometryScriptSimpleCollision TransformedCollision;
	FVector Scale = Transform.GetScale3D();
	bool bUniformScale = Scale.AllComponentsEqual();
	bool bNoRotation = Transform.GetRotation().IsIdentity();
	double ApproxUniformScale = Scale.X;
	if (!bUniformScale)
	{
		ApproxUniformScale = FMathd::Cbrt(Scale.X * Scale.Y * Scale.Z);
	}

	int32 NumSpheres = SimpleCollision.AggGeom.SphereElems.Num();
	if (NumSpheres > 0)
	{
		if (!bUniformScale)
		{
			if (TransformOptions.bWarnOnInvalidTransforms)
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Non-Uniform Scale on Sphere", "TransformSimpleCollisionShapes: Cannot apply Non-Uniform Scale to Spheres"));
			}
			bSuccess = false;
		}
		TransformedCollision.AggGeom.SphereElems.Reserve(NumSpheres);
		for (FKSphereElem Sphere : SimpleCollision.AggGeom.SphereElems)
		{
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				// Apply Transform w/ pivot at the sphere center; only the translation part will have an effect
				Sphere.Center += Transform.GetTranslation();
			}
			else
			{
				Sphere.Center = Transform.TransformPosition(Sphere.Center);
			}
			Sphere.Radius = FMath::Abs(ApproxUniformScale * Sphere.Radius);
			TransformedCollision.AggGeom.SphereElems.Add(Sphere);
		}
	}

	int32 NumBoxes = SimpleCollision.AggGeom.BoxElems.Num();
	if (NumBoxes > 0)
	{
		if (!bUniformScale)
		{
			bool bHasRotatedBox = false;
			for (const FKBoxElem& Box : SimpleCollision.AggGeom.BoxElems)
			{
				if (!Box.Rotation.IsNearlyZero())
				{
					bHasRotatedBox = true;
					break;
				}
			}
			if (bHasRotatedBox)
			{
				if (TransformOptions.bWarnOnInvalidTransforms)
				{
					UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Non-Uniform Scale on Rotated Box", "TransformSimpleCollisionShapes: Cannot apply Non-Uniform Scale to Rotated Box"));
				}
				bSuccess = false;
			}
		}
		TransformedCollision.AggGeom.BoxElems.Reserve(NumBoxes);
		for (FKBoxElem Box : SimpleCollision.AggGeom.BoxElems)
		{
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				Box.Center += Transform.GetTranslation();
			}
			else
			{
				Box.Center = Transform.TransformPosition(Box.Center);
			}
			Box.Rotation = (Transform.GetRotation() * Box.Rotation.Quaternion()).Rotator();
			Box.X = FMath::Abs(Box.X * Scale.X);
			Box.Y = FMath::Abs(Box.Y * Scale.Y);
			Box.Z = FMath::Abs(Box.Z * Scale.Z);
			TransformedCollision.AggGeom.BoxElems.Add(Box);
		}
	}

	int32 NumSphyl = SimpleCollision.AggGeom.SphylElems.Num();
	if (NumSphyl > 0)
	{
		if (!bUniformScale)
		{
			if (TransformOptions.bWarnOnInvalidTransforms)
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Non-Uniform Scale on Sphyl", "TransformSimpleCollisionShapes: Cannot apply Non-Uniform Scale to Sphyls"));
			}
			bSuccess = false;
		}

		TransformedCollision.AggGeom.SphylElems.Reserve(NumSphyl);
		for (FKSphylElem Sphyl : SimpleCollision.AggGeom.SphylElems)
		{
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				Sphyl.Center += Transform.GetTranslation();
			}
			else
			{
				Sphyl.Center = Transform.TransformPosition(Sphyl.Center);
			}
			Sphyl.Radius = FMath::Abs(Sphyl.Radius * ApproxUniformScale);
			Sphyl.Length = FMath::Abs(Sphyl.Length * ApproxUniformScale);
			Sphyl.Rotation = (Transform.GetRotation() * Sphyl.Rotation.Quaternion()).Rotator();
			TransformedCollision.AggGeom.SphylElems.Add(Sphyl);
		}
	}

	int32 NumConvex = SimpleCollision.AggGeom.ConvexElems.Num();
	TransformedCollision.AggGeom.ConvexElems.Reserve(NumConvex);
	for (const FKConvexElem& Convex : SimpleCollision.AggGeom.ConvexElems)
	{
		FKConvexElem& AddConvex = TransformedCollision.AggGeom.ConvexElems.Add_GetRef(Convex);
		AddConvex.ElemBox = FBox(EForceInit::ForceInit);
		for (FVector& V : AddConvex.VertexData)
		{
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				FVector Center = Convex.ElemBox.GetCenter();
				V = Transform.TransformPosition(V - Center) + Center;
			}
			else
			{
				V = Transform.TransformPosition(V);
			}
			AddConvex.ElemBox += V;
		}
	}

	int32 NumTaperedCapsule = SimpleCollision.AggGeom.TaperedCapsuleElems.Num();
	if (NumTaperedCapsule > 0)
	{
		if (!bUniformScale)
		{
			if (TransformOptions.bWarnOnInvalidTransforms)
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Non-Uniform Scale on Tapered Capsule", "TransformSimpleCollisionShapes: Cannot apply Non-Uniform Scale to Tapered Capsules"));
			}
			bSuccess = false;
		}

		TransformedCollision.AggGeom.TaperedCapsuleElems.Reserve(NumTaperedCapsule);
		for (FKTaperedCapsuleElem TaperedCapsule : SimpleCollision.AggGeom.TaperedCapsuleElems)
		{
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				TaperedCapsule.Center += Transform.GetTranslation();
			}
			else
			{
				TaperedCapsule.Center = Transform.TransformPosition(TaperedCapsule.Center);
			}
			if (ApproxUniformScale < 0)
			{
				TaperedCapsule.Radius0 = TaperedCapsule.Radius1 * ApproxUniformScale;
				TaperedCapsule.Radius1 = TaperedCapsule.Radius0 * ApproxUniformScale;
				TaperedCapsule.Length *= -ApproxUniformScale;
			}
			else
			{
				TaperedCapsule.Radius0 *= ApproxUniformScale;
				TaperedCapsule.Radius1 *= ApproxUniformScale;
				TaperedCapsule.Length *= ApproxUniformScale;
			}

			TaperedCapsule.Rotation = (Transform.GetRotation() * TaperedCapsule.Rotation.Quaternion()).Rotator();
			TransformedCollision.AggGeom.TaperedCapsuleElems.Add(TaperedCapsule);
		}
	}

	int32 NumLevelSet = SimpleCollision.AggGeom.LevelSetElems.Num();
	if (NumLevelSet > 0)
	{
		// Non-uniform scale cannot be properly represented if a level set was already rotated
		if (!bUniformScale)
		{
			bool bHasUnsupportedTransform = false;
			for (const FKLevelSetElem& LevelSet : SimpleCollision.AggGeom.LevelSetElems)
			{
				const FTransform ElemTransform = LevelSet.GetTransform();
				bool bElementNoRotation = ElemTransform.GetRotation().IsIdentity();
				if (!bElementNoRotation)
				{
					bHasUnsupportedTransform = true;
					break;
				}
			}
			if (bHasUnsupportedTransform)
			{
				if (TransformOptions.bWarnOnInvalidTransforms)
				{
					UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Unsupported Level Set Transform", "TransformSimpleCollisionShapes: Cannot apply requested transform to Level Set"));
				}
				bSuccess = false;
			}
		}

		TransformedCollision.AggGeom.LevelSetElems.Reserve(NumLevelSet);
		for (const FKLevelSetElem& LevelSet : SimpleCollision.AggGeom.LevelSetElems)
		{
			FKLevelSetElem& AddLevelSet = TransformedCollision.AggGeom.LevelSetElems.Add_GetRef(LevelSet);
			if (TransformOptions.bCenterTransformPivotPerShape)
			{
				FTransform LevelSetTransform = LevelSet.GetTransform();
				// Translate to the bounds center, apply Transform, and translate back
				FVector OrigLocation = LevelSetTransform.TransformPosition(LevelSet.UntransformedAABB().GetCenter());
				LevelSetTransform.SetLocation(LevelSetTransform.GetLocation() - OrigLocation);
				LevelSetTransform = LevelSetTransform * Transform;
				LevelSetTransform.SetTranslation(LevelSetTransform.GetLocation() + OrigLocation);
				AddLevelSet.SetTransform(LevelSetTransform);
			}
			else
			{
				AddLevelSet.SetTransform(LevelSet.GetTransform() * Transform);
			}
		}
	}

	int32 NumSkinnedLevelSet = SimpleCollision.AggGeom.SkinnedLevelSetElems.Num();
	if (NumSkinnedLevelSet > 0)
	{
		if (TransformOptions.bWarnOnInvalidTransforms)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformSimpleCollisionShapes Cannot Transform Skinned Level Set", "TransformSimpleCollisionShapes: Cannot transform Skinned Level Sets"));
		}
		TransformedCollision.AggGeom.SkinnedLevelSetElems = SimpleCollision.AggGeom.SkinnedLevelSetElems;
		bSuccess = false;
	}

	return TransformedCollision;
}

void UGeometryScriptLibrary_CollisionFunctions::CombineSimpleCollision(
	FGeometryScriptSimpleCollision& Collision,
	const FGeometryScriptSimpleCollision& AppendCollision,
	UGeometryScriptDebug* Debug
)
{
	// specially handle appending to self
	if (&Collision.AggGeom == &AppendCollision.AggGeom)
	{
		// No apparent reason to combine collision shapes with themselves aside from a bug, so rather than adding duplicates of each shape, log a warning and return
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CombineSimpleCollision Combining Collision Shapes with Self", "CombineSimpleCollision: Collision and AppendCollision must be different Simple Collision objects."));
		return;
	}
	Collision.AggGeom.BoxElems.Append(AppendCollision.AggGeom.BoxElems);
	Collision.AggGeom.ConvexElems.Append(AppendCollision.AggGeom.ConvexElems);
	Collision.AggGeom.SphylElems.Append(AppendCollision.AggGeom.SphylElems);
	Collision.AggGeom.SphereElems.Append(AppendCollision.AggGeom.SphereElems);
	Collision.AggGeom.TaperedCapsuleElems.Append(AppendCollision.AggGeom.TaperedCapsuleElems);
	Collision.AggGeom.LevelSetElems.Append(AppendCollision.AggGeom.LevelSetElems);
	Collision.AggGeom.SkinnedLevelSetElems.Append(AppendCollision.AggGeom.SkinnedLevelSetElems);
}


void UGeometryScriptLibrary_CollisionFunctions::CombineSimpleCollisionArray(
	const TArray<FGeometryScriptSimpleCollision>& SimpleCollisionArray,
	FGeometryScriptSimpleCollision& SimpleCollision,
	UGeometryScriptDebug* Debug
)
{
	for (const FGeometryScriptSimpleCollision& Coll : SimpleCollisionArray)
	{
		CombineSimpleCollision(SimpleCollision, Coll, Debug);
	}
}

void UGeometryScriptLibrary_CollisionFunctions::SimplifyConvexHulls(
	FGeometryScriptSimpleCollision& SimpleCollision,
	const FGeometryScriptConvexHullSimplificationOptions& SimplifyOptions,
	bool& bHasSimplified,
	UGeometryScriptDebug* Debug
)
{
	bHasSimplified = false;
	TArray<FKConvexElem>& ConvexElems = SimpleCollision.AggGeom.ConvexElems;
	for (int32 ConvexIdx = 0; ConvexIdx < ConvexElems.Num(); ++ConvexIdx)
	{
		FKConvexElem& Elem = ConvexElems[ConvexIdx];
		Elem.ComputeChaosConvexIndices(false); // make sure indices are computed

		int32 TargetFaceCount = FMath::Max(4, SimplifyOptions.MinTargetFaceCount);

		// Convert hull to a dynamic mesh
		FDynamicMesh3 Mesh;
		if (!UELocal::AppendConvexElemToCompactDynamicMesh(Elem, Mesh, TargetFaceCount))
		{
			continue;
		}

		int32 InitialTriangleCount = Mesh.TriangleCount();

		if (SimplifyOptions.SimplificationMethod == EGeometryScriptConvexHullSimplifyMethod::MeshQSlim)
		{
			// Run simplification
			FVolPresMeshSimplification Simplifier(&Mesh);
			Simplifier.CollapseMode = FVolPresMeshSimplification::ESimplificationCollapseModes::MinimalExistingVertexError;
			Simplifier.GeometricErrorConstraint = UE::Geometry::FVolPresMeshSimplification::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = SimplifyOptions.SimplificationDistanceThreshold;

			FDynamicMesh3 ProjectionTargetMesh(Mesh);
			FDynamicMeshAABBTree3 ProjectionTargetSpatial(&ProjectionTargetMesh, true);
			FMeshProjectionTarget ProjTarget(&ProjectionTargetMesh, &ProjectionTargetSpatial);
			Simplifier.SetProjectionTarget(&ProjTarget);
			Simplifier.SimplifyToTriangleCount(TargetFaceCount);

			// Simplification didn't reduce triangle count, so skip updating the convex hull
			if (Mesh.TriangleCount() == InitialTriangleCount)
			{
				continue;
			}

			Elem.VertexData.Reset(Mesh.VertexCount());
			for (FVector3d V : Mesh.VerticesItr())
			{
				Elem.VertexData.Add(V);
			}
		}
		else // EGeometryScriptConvexHullSimplifyMethod::AngleTolerance
		{
			TArray<int32> VertexCorners;
			FaceNormalClustering::FClusterOptions ClusterOptions;
			ClusterOptions.TargetMinGroups = TargetFaceCount;
			ClusterOptions.bApplyNormalToleranceToClusters = true;
			ClusterOptions.SmallFaceAreaThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER;
			ClusterOptions.SetNormalAngleToleranceInDegrees(SimplifyOptions.SimplificationAngleThreshold);
			FaceNormalClustering::ComputeClusterCornerVertices(Mesh, VertexCorners, ClusterOptions, nullptr);
			Elem.VertexData.Reset(VertexCorners.Num());
			for (int32 CornerVID : VertexCorners)
			{
				Elem.VertexData.Add(Mesh.GetVertex(CornerVID));
			}
		}


		Elem.UpdateElemBox();
		bHasSimplified = true;
	}
}

void UGeometryScriptLibrary_CollisionFunctions::ApproximateConvexHullsWithSimplerCollisionShapes(
	FGeometryScriptSimpleCollision& SimpleCollision,
	const FGeometryScriptConvexHullApproximationOptions& ApproximateOptions,
	bool& bHasApproximated,
	UGeometryScriptDebug* Debug
)
{
	bHasApproximated = false;

	if (!ApproximateOptions.bFitBoxes && !ApproximateOptions.bFitSpheres && !ApproximateOptions.bFitCapsules)
	{
		// nothing to approximate
		return;
	}

	// Helper to call ProcessPt on vertices and triangle-centroids of a convex hull mesh, stopping early if ProcessPt returns false
	auto SampleHull = [](const FKConvexElem& Elem, TFunctionRef<bool(FVector)> ProcessPt) -> bool
	{
		for (FVector V : Elem.VertexData)
		{
			if (!ProcessPt(V))
			{
				return false;
			}
		}
		for (int32 TriStart = 0; TriStart + 2 < Elem.IndexData.Num(); TriStart += 3)
		{
			FVector V0 = Elem.VertexData[Elem.IndexData[TriStart]];
			FVector V1 = Elem.VertexData[Elem.IndexData[TriStart+1]];
			FVector V2 = Elem.VertexData[Elem.IndexData[TriStart+2]];
			if (!ProcessPt((V0 + V1 + V2) / 3))
			{
				return false;
			}
		}
		return true;
	};

	auto GetVolumeDifferenceFrac = [](double HullVolume, double ApproxShapeVolume) -> double
	{
		return FMath::Abs(ApproxShapeVolume / HullVolume - 1.0);
	};

	TArray<FKConvexElem>& ConvexElems = SimpleCollision.AggGeom.ConvexElems;
	for (int32 ConvexIdx = 0; ConvexIdx < ConvexElems.Num(); ++ConvexIdx)
	{
		const FKConvexElem& Elem = ConvexElems[ConvexIdx];
		double HullVolume = UELocal::GetConvexElemVolume(Elem);

		bool bFoundBox = false, bFoundSphere = false, bFoundCapsule = false;
		UE::Geometry::FSphere3d ApproxSphere;
		FOrientedBox3d ApproxBox;
		FCapsule3d ApproxCapsule;
		double SphereVolumeDiff = FMathd::MaxReal, BoxVolumeDiff = FMathd::MaxReal, CapsuleVolumeDiff = FMathd::MaxReal;

		if (ApproximateOptions.bFitSpheres)
		{
			FMinVolumeSphere3d FitSphere;
			if (FitSphere.Solve(Elem.VertexData.Num(), [&Elem](int32 Idx) {return Elem.VertexData[Idx];}))
			{
				FitSphere.GetResult(ApproxSphere);
				double SphereVolume = ApproxSphere.Volume();
				SphereVolumeDiff = GetVolumeDifferenceFrac(HullVolume, SphereVolume);
				bFoundSphere = (SphereVolumeDiff < ApproximateOptions.VolumeDiffThreshold_Fraction)
					&& SampleHull(Elem, [&](FVector V) -> bool
					{
						return FMath::Abs(ApproxSphere.SignedDistance(V)) < ApproximateOptions.DistanceThreshold;
					});
			}
		}

		if (ApproximateOptions.bFitBoxes)
		{
			FMinVolumeBox3d FitBox;
			if (FitBox.Solve(Elem.VertexData.Num(), [&Elem](int32 Idx) {return Elem.VertexData[Idx];}, false, nullptr))
			{
				FitBox.GetResult(ApproxBox);
				double BoxVolume = ApproxBox.Volume();
				BoxVolumeDiff = GetVolumeDifferenceFrac(HullVolume, BoxVolume);
				bFoundBox = (BoxVolumeDiff < ApproximateOptions.VolumeDiffThreshold_Fraction)
					&& SampleHull(Elem, [&](FVector V) -> bool
					{
						return FMath::Abs(ApproxBox.SignedDistance(V)) < ApproximateOptions.DistanceThreshold;
					});
			}
		}

		if (ApproximateOptions.bFitCapsules)
		{
			
			if (FFitCapsule3d::Solve(Elem.VertexData.Num(), [&Elem](int32 Idx) {return Elem.VertexData[Idx];}, ApproxCapsule))
			{
				double CapsuleVolume = ApproxCapsule.Volume();
				CapsuleVolumeDiff = GetVolumeDifferenceFrac(HullVolume, CapsuleVolume);
				bFoundCapsule = (CapsuleVolumeDiff < ApproximateOptions.VolumeDiffThreshold_Fraction)
					&& SampleHull(Elem, [&ApproxCapsule, &ApproximateOptions](FVector V) -> bool
					{
						return FMath::Abs(ApproxCapsule.SignedDistance(V)) < ApproximateOptions.DistanceThreshold;
					});
			}
		}

		if (bFoundSphere || bFoundBox || bFoundCapsule)
		{
			// Add the approximating element
			switch (FMathd::Min3Index(BoxVolumeDiff, SphereVolumeDiff, CapsuleVolumeDiff))
			{
			case 0: // Box is better
			{
				FKBoxElem& ElemBox = SimpleCollision.AggGeom.BoxElems.Emplace_GetRef();
				GetFKElement(ApproxBox, ElemBox);
				break;
			}
			case 1: // Sphere is better
			{
				FKSphereElem& ElemSphere = SimpleCollision.AggGeom.SphereElems.Emplace_GetRef();
				GetFKElement(ApproxSphere, ElemSphere);
				break;
			}
			case 2: // Capsule is better
			{
				FKSphylElem& ElemCapsule = SimpleCollision.AggGeom.SphylElems.Emplace_GetRef();
				GetFKElement(ApproxCapsule, ElemCapsule);
				break;
			}
			default: // Min3Index can only return 0, 1, or 2
				checkNoEntry();
			}

			// Remove the approximated convex hull
			ConvexElems.RemoveAtSwap(ConvexIdx);
			ConvexIdx--;

			// Log that we accepted an approximation
			bHasApproximated = true;
		}
	}
}

FGeometryScriptSimpleCollision UGeometryScriptLibrary_CollisionFunctions::MergeSimpleCollisionShapes(
	const FGeometryScriptSimpleCollision& SimpleCollision,
	const FGeometryScriptMergeSimpleCollisionOptions& MergeOptions,
	bool& bHasMerged,
	UGeometryScriptDebug* Debug
)
{
	FGeometryScriptSimpleCollision ToRet;
	bHasMerged = false;
	
	// Nothing to merge
	if (SimpleCollision.AggGeom.GetElementCount() <= 1)
	{
		return SimpleCollision;
	}

	TArray<FVector> HullVertices;
	TArray<int32> HullVertexCounts;
	TArray<double> HullVolumes;
	TArray<const FKShapeElem*> HullToShapeElem;
	TUniquePtr<FDynamicMesh3> CollisionMesh;
	if (MergeOptions.bComputeNegativeSpace)
	{
		CollisionMesh = MakeUnique<FDynamicMesh3>();
	}

	auto AppendGeneratorToCollisionMesh = [&CollisionMesh](const FMeshShapeGenerator& Generator)
	{
		if (!CollisionMesh)
		{
			return;
		}
		checkSlow(CollisionMesh->IsCompact());
		int32 NewVertStart = CollisionMesh->MaxVertexID();
		for (FVector3d V : Generator.Vertices)
		{
			CollisionMesh->AppendVertex(V);
		}
		for (FIndex3i T : Generator.Triangles)
		{
			CollisionMesh->AppendTriangle(T.A + NewVertStart, T.B + NewVertStart, T.C + NewVertStart);
		}
	};
	auto TransformVertices = [](TArrayView<FVector3d> Vertices, const FTransform& Transform)
	{
		for (FVector3d& Vertex : Vertices)
		{
			Vertex = Transform.TransformPosition(Vertex);
		}
	};
	auto AppendHullVertices = [&HullToShapeElem, &HullVolumes, &HullVertices, &HullVertexCounts]
				(TArrayView<const FVector3d> Vertices, double Volume, const FKShapeElem* ShapeElem, const FTransform* ShapeTransform = nullptr)
	{
		check(HullToShapeElem.Num() == HullVolumes.Num());
		HullToShapeElem.Add(ShapeElem);
		int32 HullIdxStart = HullVertices.Num();
		HullVertices.Append(Vertices);
		if (ShapeTransform)
		{
			for (int32 Idx = HullIdxStart; Idx < HullVertices.Num(); ++Idx)
			{
				HullVertices[Idx] = ShapeTransform->TransformPosition(HullVertices[Idx]);
			}
		}
		HullVertexCounts.Add(Vertices.Num());
		HullVolumes.Add(Volume);
	};
	auto GeneratorVolume = [](FMeshShapeGenerator* Generator) -> double
	{
		TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d> GenMeshAdapter(&Generator->Vertices, &Generator->Triangles);
		FVector2d VolArea = TMeshQueries<TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d>>::GetVolumeArea(GenMeshAdapter);
		return VolArea.X;
	};

	for (const FKBoxElem& Box : SimpleCollision.AggGeom.BoxElems)
	{
		FOrientedBox3d OrientedBox;
		OrientedBox.Extents = FVector(Box.X * .5, Box.Y * .5, Box.Z * .5);
		OrientedBox.Frame.Origin = Box.Center;
		OrientedBox.Frame.Rotation = (FQuaterniond)Box.Rotation;
		if (CollisionMesh)
		{
			FGridBoxMeshGenerator BoxGenerator;
			BoxGenerator.EdgeVertices = FIndex3i(1, 1, 1);
			BoxGenerator.Box = OrientedBox;
			BoxGenerator.Generate();
			AppendHullVertices(BoxGenerator.Vertices, Box.GetScaledVolume(FVector3d::One()), &Box);
			AppendGeneratorToCollisionMesh(BoxGenerator);
		}
		else
		{
			TArray<FVector3d, TFixedAllocator<8>> BoxVertices;
			OrientedBox.EnumerateCorners([&](FVector3d Corner) { BoxVertices.Add(Corner); });
			AppendHullVertices(BoxVertices, Box.GetScaledVolume(FVector3d::One()), &Box);
		}
	}
	for (const FKSphereElem& Sphere : SimpleCollision.AggGeom.SphereElems)
	{
		FBoxSphereGenerator SphereGenerator;
		SphereGenerator.Box.Frame.Origin = Sphere.Center;
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Sphere.Radius);
		int32 StepsPerSide = FMath::Max(1, MergeOptions.ShapeToHullTriangulation.SphereStepsPerSide);
		SphereGenerator.EdgeVertices = FIndex3i(StepsPerSide, StepsPerSide, StepsPerSide);
		SphereGenerator.Generate();
		double Volume = GeneratorVolume(&SphereGenerator);
		AppendHullVertices(SphereGenerator.Vertices, Volume, &Sphere);
		AppendGeneratorToCollisionMesh(SphereGenerator);
	}
	for (const FKSphylElem& Capsule : SimpleCollision.AggGeom.SphylElems)
	{
		FCapsuleGenerator CapsuleGenerator;
		CapsuleGenerator.Radius = Capsule.Radius;
		CapsuleGenerator.SegmentLength = Capsule.Length;
		CapsuleGenerator.NumHemisphereArcSteps = FMath::Max(2, MergeOptions.ShapeToHullTriangulation.CapsuleHemisphereSteps);
		CapsuleGenerator.NumCircleSteps = FMath::Max(3, MergeOptions.ShapeToHullTriangulation.CapsuleCircleSteps);
		CapsuleGenerator.Generate();
		FTransform CapsuleTransform(Capsule.Rotation, Capsule.Center + Capsule.Rotation.RotateVector(FVector(0, 0, -Capsule.Length * .5)));
		TransformVertices(CapsuleGenerator.Vertices, CapsuleTransform);
		double Volume = GeneratorVolume(&CapsuleGenerator);
		AppendHullVertices(CapsuleGenerator.Vertices, Volume, &Capsule);
		AppendGeneratorToCollisionMesh(CapsuleGenerator);
	}
	for (const FKConvexElem& Convex : SimpleCollision.AggGeom.ConvexElems)
	{
		double Volume = UELocal::GetConvexElemVolume(Convex);
		FTransform ConvexTransform = Convex.GetTransform();
		AppendHullVertices(Convex.VertexData, Volume, &Convex, &ConvexTransform);
		if (CollisionMesh)
		{
			UELocal::AppendConvexElemToCompactDynamicMesh(Convex, *CollisionMesh);
		}
	}

	const int32 InitialNumConvex = HullVertexCounts.Num();
	// Nothing we are able to merge
	if (InitialNumConvex <= 1)
	{
		return SimpleCollision;
	}

	TArray<int32> HullVertexStarts;
	HullVertexStarts.SetNumUninitialized(InitialNumConvex);
	HullVertexStarts[0] = 0; // Note InitialNumConvex is > 1 due to above test
	for (int32 HullIdx = 1, LastEnd = HullVertexCounts[0]; HullIdx < InitialNumConvex; LastEnd += HullVertexCounts[HullIdx++])
	{
		HullVertexStarts[HullIdx] = LastEnd;
	}

	TArray<TPair<int32, int32>> HullProximity;
	if (MergeOptions.bConsiderAllPossibleMerges)
	{
		// Add all n^2 possible merge combinations for consideration
		for (int32 ConvexA = 0; ConvexA < InitialNumConvex; ++ConvexA)
		{
			for (int32 ConvexB = ConvexA + 1; ConvexB < InitialNumConvex; ++ConvexB)
			{
				HullProximity.Emplace(ConvexA, ConvexB);
			}
		}
	}

	FConvexDecomposition3 Decomposition;
	Decomposition.InitializeFromHulls(HullVertexStarts.Num(),
		[&HullVolumes](int32 HullIdx) { return HullVolumes[HullIdx]; }, [&HullVertexCounts](int32 HullIdx) { return HullVertexCounts[HullIdx]; },
		[&HullVertexStarts, &HullVertices](int32 HullIdx, int32 VertIdx) { return HullVertices[HullVertexStarts[HullIdx] + VertIdx]; }, HullProximity);
	double MinProximityOverlapTolerance = 0;
	FSphereCovering NegativeSpace;
	// Build the negative space of the collision shapes, if requested
	if (MergeOptions.bComputeNegativeSpace)
	{
		FNegativeSpaceSampleSettings SampleSettings = UELocal::ConvertNegativeSpaceOptions(MergeOptions.ComputeNegativeSpaceOptions);
		MinProximityOverlapTolerance = FMath::Max(SampleSettings.ReduceRadiusMargin * .5, MinProximityOverlapTolerance);
		FDynamicMeshAABBTree3 CollisionAABBTree(CollisionMesh.Get(), true);
		TFastWindingTree<FDynamicMesh3> CollisionFastWinding(&CollisionAABBTree, true);
		NegativeSpace.AddNegativeSpace(CollisionFastWinding, SampleSettings, false);
	}
	// Add any precomputed negative space, if valid/non-empty
	if (MergeOptions.PrecomputedNegativeSpace.Spheres.IsValid() && MergeOptions.PrecomputedNegativeSpace.Spheres->Num() > 0)
	{
		NegativeSpace.Append(*MergeOptions.PrecomputedNegativeSpace.Spheres);
	}
	FSphereCovering* UseNegativeSpace = NegativeSpace.Num() > 0 ? &NegativeSpace : nullptr;
	if (!MergeOptions.bConsiderAllPossibleMerges)
	{
		// Find possible shape merges based on the shape bounding box overlaps,
		// where bounds are expanded by: Max(a quarter their min dimension, a tenth their max dimension, the half reduce radius margin if negative space is computed)
		Decomposition.InitializeProximityFromDecompositionBoundingBoxOverlaps(.25, .1, MinProximityOverlapTolerance);
	}
	int32 MaxShapeCount = FMath::Max(0, MergeOptions.MaxShapeCount);
	Decomposition.RestrictMergeSearchToLocalAfterTestNumConnections = 1000 + MaxShapeCount * MaxShapeCount; // Restrict searches in very large search cases, when not close to max shape count, to avoid excessive search time
	Decomposition.MergeBest(MaxShapeCount, MergeOptions.ErrorTolerance, MergeOptions.MinThicknessTolerance, true, false, MaxShapeCount, UseNegativeSpace, nullptr /*optional FTransform for negative space*/);

	// Algorithm decided not to merge
	if (Decomposition.NumHulls() == InitialNumConvex)
	{
		return SimpleCollision;
	}

	bHasMerged = true;

	// Merging logic for the below primitives is not implemented, so they are simply copied over for now
	if (!SimpleCollision.AggGeom.TaperedCapsuleElems.IsEmpty() || !SimpleCollision.AggGeom.SkinnedLevelSetElems.IsEmpty() || !SimpleCollision.AggGeom.LevelSetElems.IsEmpty())
	{
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("PrimitiveFunctions_AppendSimpleCollisionShapes Unsupported Shapes", "MergeSimpleCollisionShapes: Merging for Tapered Capsules and Level Set collision is not yet supported; these shapes will be copied without considering them for merging."));
		for (const FKTaperedCapsuleElem& Capsule : SimpleCollision.AggGeom.TaperedCapsuleElems)
		{
			ToRet.AggGeom.TaperedCapsuleElems.Add(Capsule);
		}
		for (const FKSkinnedLevelSetElem& LevelSet : SimpleCollision.AggGeom.SkinnedLevelSetElems)
		{
			ToRet.AggGeom.SkinnedLevelSetElems.Add(LevelSet);
		}
		for (const FKLevelSetElem& LevelSet : SimpleCollision.AggGeom.LevelSetElems)
		{
			ToRet.AggGeom.LevelSetElems.Add(LevelSet);
		}
	}
	
	for (int32 HullIdx = 0; HullIdx < Decomposition.Decomposition.Num(); ++HullIdx)
	{
		const FConvexDecomposition3::FConvexPart& Part = Decomposition.Decomposition[HullIdx];
		// If part was not merged, use the source ID to map it back to the original collision primitive
		if (Part.HullSourceID >= 0)
		{
			const FKShapeElem* Elem = HullToShapeElem[Part.HullSourceID];
			bool bHandledShape = true;
			switch (Elem->GetShapeType())
			{
			case EAggCollisionShape::Box:
				ToRet.AggGeom.BoxElems.Add(*static_cast<const FKBoxElem*>(Elem));
				break;
			case EAggCollisionShape::Sphere:
				ToRet.AggGeom.SphereElems.Add(*static_cast<const FKSphereElem*>(Elem));
				break;
			case EAggCollisionShape::Sphyl:
				ToRet.AggGeom.SphylElems.Add(*static_cast<const FKSphylElem*>(Elem));
				break;
			case EAggCollisionShape::Convex:
				ToRet.AggGeom.ConvexElems.Add(*static_cast<const FKConvexElem*>(Elem));
				break;
			default:
				// Note: All shapes that we add to the HullToShapeElem array should be handled above, so we should not reach here
				ensureMsgf(false, TEXT("Unhandled shape element type could not be restored from source shapes"));
				bHandledShape = false;
			}
			if (bHandledShape)
			{
				continue;
			}
		}
		// Add the merged part
		FKConvexElem& Convex = ToRet.AggGeom.ConvexElems.Emplace_GetRef();
		Convex.VertexData = Decomposition.GetVertices<double>(HullIdx);
		Convex.UpdateElemBox(); // Note: In addition to updating the bounding box, this also re-computes hull indices.
	}
	
	return ToRet;
}

FGeometryScriptSphereCovering UGeometryScriptLibrary_CollisionFunctions::ComputeNegativeSpace(
	const FGeometryScriptDynamicMeshBVH& MeshBVH,
	const FComputeNegativeSpaceOptions& NegativeSpaceOptions,
	UGeometryScriptDebug* Debug
)
{
	FGeometryScriptSphereCovering ToRet;

	if (!MeshBVH.FWNTree || !MeshBVH.Spatial)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeNegativeSpace_Null_BVH", "ComputeNegativeSpace: BVH must be initialized"));
		return ToRet;
	}
	
	FNegativeSpaceSampleSettings UseSettings = UELocal::ConvertNegativeSpaceOptions(NegativeSpaceOptions);
	ToRet.Reset();
	ToRet.Spheres->AddNegativeSpace(*MeshBVH.FWNTree, UseSettings, false);
	return ToRet;
}

FGeometryScriptSimpleCollision UGeometryScriptLibrary_CollisionFunctions::ComputeNavigableConvexDecomposition(
	const UDynamicMesh* TargetMesh,
	const FNavigableConvexDecompositionOptions& NegativeSpaceOptions,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptSimpleCollision ToReturn;
	
	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeNavigableConvexDecomposition_NullInput", "ComputeNavigableConvexDecomposition: Dynamic Mesh is Null"));
		return ToReturn;
	}

	TargetMesh->ProcessMesh([&NegativeSpaceOptions, &ToReturn](const FDynamicMesh3& Mesh) -> void
	{
		FConvexDecomposition3::FPreprocessMeshOptions PreprocessOptions;
		PreprocessOptions.bMergeEdges = true;
		PreprocessOptions.CustomPreprocess = [](FDynamicMesh3& ProcessMesh, const FAxisAlignedBox3d& Bounds) -> void
		{
			// for solid inputs, flip orientation if the initial volume is negative
			if (ProcessMesh.IsClosed())
			{
				double InitialVolume = TMeshQueries<FDynamicMesh3>::GetVolumeArea(ProcessMesh).X;
				if (InitialVolume < 0)
				{
					ProcessMesh.ReverseOrientation();
				}
			}
			// Note: If we add options to simplify the input mesh, should be applied here.
		};

		FConvexDecomposition3 ConvexDecomposition(Mesh, PreprocessOptions);
		const bool bIsSolid = ConvexDecomposition.IsInputSolid();
		ConvexDecomposition.bTreatAsSolid = bIsSolid;

		FNegativeSpaceSampleSettings NegativeSpaceSettings;
		NegativeSpaceSettings.ApplyDefaults();
		NegativeSpaceSettings.SampleMethod = FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch;
		NegativeSpaceSettings.bDeterministic = true;
		NegativeSpaceSettings.bRequireSearchSampleCoverage = true;
		NegativeSpaceSettings.bOnlyConnectedToHull = NegativeSpaceOptions.bIgnoreUnreachableInternalSpace;
		NegativeSpaceSettings.TargetNumSamples = 0;
		NegativeSpaceSettings.bAllowSamplesInsideMesh = !bIsSolid;

		NegativeSpaceSettings.ReduceRadiusMargin = NegativeSpaceOptions.Tolerance;
		NegativeSpaceSettings.MinRadius = NegativeSpaceOptions.MinRadius;
		NegativeSpaceSettings.MinSpacing = 0;
		UELocal::ApplyNegativeSpaceToleranceDirectionalScales(NegativeSpaceSettings, NegativeSpaceOptions.ToleranceDirectionalScales);

		if (!NegativeSpaceOptions.UnreachablePlanes.IsEmpty())
		{
			NegativeSpaceSettings.OptionalObstacleSDF = [&NegativeSpaceOptions](FVector Pos)
			{
				double SDF = FMathd::MaxReal;
				for (const FPlane& Plane : NegativeSpaceOptions.UnreachablePlanes)
				{
					double PlaneSDF = Plane.PlaneDot(Pos);
					SDF = FMath::Min(PlaneSDF, SDF);
				}
				return SDF;
			};
		}
		

		ConvexDecomposition.InitializeNegativeSpace(NegativeSpaceSettings, NegativeSpaceOptions.CustomNavigablePositions);

		ConvexDecomposition.MaxConvexEdgePlanes = 4;
		ConvexDecomposition.bSplitDisconnectedComponents = false;
		ConvexDecomposition.ConvexEdgeAngleMoreSamplesThreshold = 180;
		ConvexDecomposition.ThickenAfterHullFailure = FMath::Max(FMathd::ZeroTolerance, NegativeSpaceSettings.ReduceRadiusMargin * .01);
		constexpr int32 MaxAllowedSplits = 1000000; // more parts than any expected / reasonable decomposition
		for (int32 Split = 0; ; Split++)
		{
			int32 NumSplit = ConvexDecomposition.SplitWorst(false, -1, true, NegativeSpaceSettings.ReduceRadiusMargin * NegativeSpaceSettings.MinCustomMarginScale * .5);

			if (NumSplit == 0)
			{
				break;
			}

			if (!ensureMsgf(Split < MaxAllowedSplits, TEXT("Convex decomposition split the input %d times; likely stuck in a loop"), Split))
			{
				break;
			}
		}

		ConvexDecomposition.FixHullOverlapsInNegativeSpace();
			
		int32 NumHullsBefore = ConvexDecomposition.NumHulls();
		constexpr double MinThicknessToleranceWorldSpace = 0;
		int32 NumMerged = ConvexDecomposition.MergeBest(-1, 0, MinThicknessToleranceWorldSpace, true);

		// transfer to output
		for (int32 HullIdx = 0; HullIdx < ConvexDecomposition.Decomposition.Num(); ++HullIdx)
		{
			const FConvexDecomposition3::FConvexPart& Part = ConvexDecomposition.Decomposition[HullIdx];
			// Add the merged part
			FKConvexElem& Convex = ToReturn.AggGeom.ConvexElems.Emplace_GetRef();
			Convex.VertexData = ConvexDecomposition.GetVertices<double>(HullIdx);
			Convex.UpdateElemBox(); // Note: In addition to updating the bounding box, this also re-computes hull indices.
		}

	});
	return ToReturn;

}


TArray<FSphere> UGeometryScriptLibrary_CollisionFunctions::Conv_GeometryScriptSphereCoveringToSphereArray(const FGeometryScriptSphereCovering& SphereCovering)
{
	TArray<FSphere> ToRet;
	if (SphereCovering.Spheres.IsValid())
	{
		int32 NumSpheres = SphereCovering.Spheres->Num();
		ToRet.SetNumUninitialized(NumSpheres);
		for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
		{
			ToRet[Idx].Center = SphereCovering.Spheres->GetCenter(Idx);
			ToRet[Idx].W = SphereCovering.Spheres->GetRadius(Idx);
		}
	}
	return ToRet;
}

FGeometryScriptSphereCovering UGeometryScriptLibrary_CollisionFunctions::Conv_SphereArrayToGeometryScriptSphereCovering(const TArray<FSphere>& Spheres)
{
	FGeometryScriptSphereCovering Covering;
	Covering.Reset();
	Covering.Spheres->AppendSpheres(Spheres);
	return Covering;
}



#undef LOCTEXT_NAMESPACE
