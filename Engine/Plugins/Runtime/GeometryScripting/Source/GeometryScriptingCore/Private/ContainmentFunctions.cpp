// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/ContainmentFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshConvexHull.h"
#include "Operations/MeshProjectionHull.h"
#include "ShapeApproximation/MeshSimpleShapeApproximation.h"
#include "MinVolumeBox3.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContainmentFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshContainmentFunctions"


UDynamicMesh* UGeometryScriptLibrary_ContainmentFunctions::ComputeMeshConvexHull(
	UDynamicMesh* TargetMesh,
	UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
	UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptConvexHullOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshConvexHull_InvalidInput", "ComputeMeshConvexHull: TargetMesh is Null"));
		return TargetMesh;
	}
	if (CopyToMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshConvexHull_InvalidInput2", "ComputeMeshConvexHull: CopyToMesh is Null"));
		return TargetMesh;
	}
	CopyToMeshOut = CopyToMesh;

	FDynamicMesh3 HullMesh;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh) 
	{
		FMeshConvexHull Hull(&EditMesh);

		if (Selection.IsEmpty() == false)
		{
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID) {
				Hull.VertexSet.Add(VertexID);
			});
		}
		else if (Options.bPrefilterVertices)
		{
			FMeshConvexHull::GridSample(EditMesh, FMath::Max(32, Options.PrefilterGridResolution), Hull.VertexSet);
		}

		Hull.bPostSimplify = (Options.SimplifyToFaceCount > 0);
		Hull.SimplifyOptions.MaxTargetFaceCount = Options.SimplifyToFaceCount;
		Hull.SimplifyOptions.GeometricToleranceAfterTarget = Options.GeometricToleranceAfterTarget;
		if (Hull.Compute(nullptr))
		{
			HullMesh = MoveTemp(Hull.ConvexHull);
			HullMesh.EnableAttributes();
			FMeshNormals::InitializeOverlayToPerTriangleNormals(HullMesh.Attributes()->PrimaryNormals());
		}
	});

	if ( HullMesh.TriangleCount() == 0 )
	{
		// todo: replace output with bounding box mesh?
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeMeshConvexHull_Failed", "ComputeMeshConvexHull: Hull Computation Failed"));
		CopyToMesh->ResetToCube();
	}
	else
	{
		CopyToMesh->SetMesh(MoveTemp(HullMesh));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_ContainmentFunctions::ComputeMeshSweptHull(
	UDynamicMesh* TargetMesh,
	UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
	UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
	FTransform ProjectionFrame,
	FGeometryScriptSweptHullOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshSweptHull_InvalidInput", "ComputeMeshSweptHull: TargetMesh is Null"));
		return TargetMesh;
	}
	if (CopyToMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshSweptHulll_InvalidInput2", "ComputeMeshSweptHull: CopyToMesh is Null"));
		return TargetMesh;
	}
	CopyToMeshOut = CopyToMesh;

	FDynamicMesh3 HullMesh;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh) 
	{
		FMeshProjectionHull Hull(&EditMesh);
		Hull.ProjectionFrame = FFrame3d(ProjectionFrame);
		Hull.MinThickness = FMathd::Max(Options.MinThickness, 0);
		Hull.bSimplifyPolygon = Options.bSimplify;
		Hull.MinEdgeLength = Options.MinEdgeLength;
		Hull.DeviationTolerance = Options.SimplifyTolerance;

		if (Hull.Compute())
		{
			HullMesh = MoveTemp(Hull.ConvexHull3D);
		}
	});

	if ( HullMesh.TriangleCount() == 0 )
	{
		// todo: replace output with bounding box mesh?
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeMeshSweptHull_Failed", "ComputeMeshSweptHull: Hull Computation Failed"));
		CopyToMesh->ResetToCube();
	}
	else
	{
		CopyToMesh->SetMesh(MoveTemp(HullMesh));
	}

	return TargetMesh;
}








UDynamicMesh* UGeometryScriptLibrary_ContainmentFunctions::ComputeMeshConvexDecomposition(
	UDynamicMesh* TargetMesh,
	UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
	UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
	FGeometryScriptConvexDecompositionOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshConvexDecomposition_InvalidInput", "ComputeMeshConvexDecomposition: TargetMesh is Null"));
		return TargetMesh;
	}
	if (CopyToMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshConvexDecomposition_InvalidInput2", "ComputeMeshConvexDecomposition: CopyToMesh is Null"));
		return TargetMesh;
	}
	CopyToMeshOut = CopyToMesh;

	FDynamicMesh3 HullsMesh;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh) 
	{
		TArray<const FDynamicMesh3*> InputMeshes;
		InputMeshes.Add(&EditMesh);
		FMeshSimpleShapeApproximation Approximator;
		Approximator.InitializeSourceMeshes(InputMeshes);

		Approximator.bSimplifyHulls = (Options.SimplifyToFaceCount > 4);
		Approximator.HullTargetFaceCount = Options.SimplifyToFaceCount;

		Approximator.ConvexDecompositionMaxPieces = Options.NumHulls;
		Approximator.ConvexDecompositionSearchFactor = Options.SearchFactor;
		Approximator.ConvexDecompositionErrorTolerance = Options.ErrorTolerance;
		Approximator.ConvexDecompositionMinPartThickness = Options.MinPartThickness;

		FSimpleShapeSet3d Shapes;
		Approximator.Generate_ConvexHullDecompositions(Shapes);

		HullsMesh.EnableMatchingAttributes(EditMesh);
		FDynamicMeshEditor Editor(&HullsMesh);
		for (FConvexShape3d& Convex : Shapes.Convexes)
		{
			FMeshIndexMappings Mappings;
			Editor.AppendMesh(&Convex.Mesh, Mappings);
		}
	});

	if ( HullsMesh.TriangleCount() == 0 )
	{
		// todo: replace output with bounding box mesh?
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeMeshConvexDecomposition_Failed", "ComputeMeshConvexDecomposition: Hull Computation Failed"));
		CopyToMesh->ResetToCube();
	}
	else
	{
		FMeshNormals::InitializeOverlayToPerTriangleNormals(HullsMesh.Attributes()->PrimaryNormals());
		CopyToMesh->SetMesh(MoveTemp(HullsMesh));
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_ContainmentFunctions::ComputeMeshOrientedBox(
	UDynamicMesh* TargetMesh,
	FOrientedBox& OrientedBoxOut,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptFitOrientedBoxOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshOrientedBox_InvalidInput", "ComputeMeshOrientedBox: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&OrientedBoxOut, &Selection, &Options, Debug](const FDynamicMesh3& ProcessMesh)
		{
			TArray<int32> SelectedVertices;

			if (!Selection.IsEmpty())
			{
				Selection.ProcessByVertexID(ProcessMesh, 
					[&SelectedVertices](int32 VertexID) { SelectedVertices.Add(VertexID); }
				);
			}
			else
			{
				SelectedVertices.Reserve(ProcessMesh.VertexCount());
				for (int32 VID : ProcessMesh.VertexIndicesItr())
				{
					SelectedVertices.Add(VID);
				}
			}

			int32 VertexCount = SelectedVertices.Num();
			if (VertexCount == 0)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeMeshOrientedBox_FailedEmptyInput", "ComputeMeshOrientedBox: Cannot find bounds of zero vertices"));
				return;
			}

			FOrientedBox3d ResultBox;
			
			bool bUseExactComputationForBox = Options.FitMethod == FGeometryScriptFitOrientedBoxMethod::Precise;
			double MinDimension = FMath::Max(0., Options.MinBoxDimension);
			FMinVolumeBox3d MinBoxCalc;
			bool bMinBoxOK = MinBoxCalc.Solve(SelectedVertices.Num(),
				[&ProcessMesh, &SelectedVertices](int32 Index)
				{ return ProcessMesh.GetVertex(SelectedVertices[Index]); },
				bUseExactComputationForBox, nullptr
			);
				
			if (bMinBoxOK && MinBoxCalc.IsSolutionAvailable())
			{
				MinBoxCalc.GetResult(ResultBox);
				double MinHalfDimension = MinDimension * .5;
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					ResultBox.Extents[SubIdx] = FMath::Max(MinHalfDimension, ResultBox.Extents[SubIdx]);
				}
				ResultBox = ResultBox.ReparameterizedCloserToWorldFrame();
			}
			else
			{
				// Fitting failed; fall back to AABB
				ResultBox = ProcessMesh.GetBounds();
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeMeshOrientedBox_FailedFittingStep", "ComputeMeshOrientedBox: Failed to fit oriented box; falling back to axis-aligned box."));
			}
			
			// convert result to the BP-accessible oriented box type
			OrientedBoxOut = (FOrientedBox)ResultBox;
		}
	);

	return TargetMesh;
}



#undef LOCTEXT_NAMESPACE