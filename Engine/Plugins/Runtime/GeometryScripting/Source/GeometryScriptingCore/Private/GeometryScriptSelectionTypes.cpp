// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"
#include "UDynamicMesh.h"
#include "Misc/StringBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryScriptSelectionTypes)

using namespace UE::Geometry;

namespace UE::Private::GeometryScriptSelectionLocal
{
	// Assuming the selection is a triangle-edge selection, this enumerates all the *unique* mesh edge IDs
	//  -- accounting for the fact that the GeoSelection is effectively a half-edge representation that can include the same edge twice (per triangle that includes it)
	static void EnumerateUniqueEdgesInEdgeSelection(const FDynamicMesh3& Mesh, const FGeometrySelection& GeoSelection, TFunctionRef<void(int32)> PerEdgeFunc)
	{
		checkSlow(GeoSelection.ElementType == EGeometryElementType::Edge && GeoSelection.TopologyType == EGeometryTopologyType::Triangle);
		for (uint64 Item : GeoSelection.Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(Item).GeometryID);
			if (Mesh.IsTriangle(TriEdgeID.TriangleID))
			{
				int32 EdgeID = Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex);
				// Check if edge is also in the selection via a lower triangle ID, and if so rely on that element to call the PerEdgeFunc
				bool bAlsoInSelectionViaLowerTriangleID = false;
				Mesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&GeoSelection, &TriEdgeID, &bAlsoInSelectionViaLowerTriangleID](FMeshTriEdgeID CurTriEdge)
				{
					if (CurTriEdge.TriangleID < TriEdgeID.TriangleID && GeoSelection.Selection.Contains((uint64)CurTriEdge.Encoded()))
					{
						bAlsoInSelectionViaLowerTriangleID = true;
					}
				});
				if (!bAlsoInSelectionViaLowerTriangleID)
				{
					PerEdgeFunc(EdgeID);
				}
			}
		}
	}
}


FGeometryScriptMeshSelection::FGeometryScriptMeshSelection()
{
	GeoSelection = MakeShared<UE::Geometry::FGeometrySelection>();
	GeoSelection->InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
}

void FGeometryScriptMeshSelection::SetSelection(const FGeometryScriptMeshSelection& Selection)
{
	*GeoSelection = *Selection.GeoSelection;
}

void FGeometryScriptMeshSelection::SetSelection(const UE::Geometry::FGeometrySelection& Selection)
{
	*GeoSelection = Selection;
}

void FGeometryScriptMeshSelection::SetSelection(UE::Geometry::FGeometrySelection&& Selection)
{
	*GeoSelection = MoveTemp(Selection);
}


void FGeometryScriptMeshSelection::ClearSelection()
{
	GeoSelection->Reset();
}

bool FGeometryScriptMeshSelection::IsEmpty() const
{
	return GeoSelection->IsEmpty();
}

EGeometryScriptMeshSelectionType FGeometryScriptMeshSelection::GetSelectionType() const
{
	if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (GeoSelection->ElementType == EGeometryElementType::Face)
		{
			return EGeometryScriptMeshSelectionType::Polygroups;
		}
	}
	else if (GeoSelection->TopologyType == EGeometryTopologyType::Triangle)
	{
		if (GeoSelection->ElementType == EGeometryElementType::Face)
		{
			return EGeometryScriptMeshSelectionType::Triangles;
		}
		else if (GeoSelection->ElementType == EGeometryElementType::Vertex)
		{
			return EGeometryScriptMeshSelectionType::Vertices;
		}
		else if (GeoSelection->ElementType == EGeometryElementType::Edge)
		{
			return EGeometryScriptMeshSelectionType::Edges;
		}
	}
	UE_LOGF(LogGeometry, Warning, "GeometryScriptMeshSelection::GetSelectionType() - GeoSelection has invalid type for FGeometryScriptMeshSelection!");
	return EGeometryScriptMeshSelectionType::Triangles;
}


int32 FGeometryScriptMeshSelection::GetNumSelected() const
{
	return GeoSelection->Num();
}

int32 FGeometryScriptMeshSelection::GetNumUniqueSelected(const UE::Geometry::FDynamicMesh3& Mesh) const
{
	if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (GeoSelection->ElementType != EGeometryElementType::Face)
		{
			UE_LOGF(LogGeometry, Warning, "GeometryScriptMeshSelection::GetNumUniqueSelected() - GeoSelection has invalid type for FGeometryScriptMeshSelection!");
			return 0;
		}
		
		// The same group could be referenced from more than one triangle ID
		int32 Count = 0;
		TSet<int32> UniqueGroupIDs;
		for (uint64 Item : GeoSelection->Selection)
		{
			bool bWasInSet;
			UniqueGroupIDs.Add(FGeoSelectionID(Item).TopologyID, &bWasInSet);
			Count += (int32)!bWasInSet;
		}
		return Count;
	}
	else
	{
		// Note vertex and face selections should already be unique, so can just return the selection set size
		int32 Count = 0;
		switch (GeoSelection->ElementType)
		{
		case EGeometryElementType::Vertex:
			return GeoSelection->Selection.Num();
		case EGeometryElementType::Edge:
			// iterate to count unique edges
			UE::Private::GeometryScriptSelectionLocal::EnumerateUniqueEdgesInEdgeSelection(Mesh, *GeoSelection, [&Count](int32 EdgeID) {++Count;});
			return Count;
		case EGeometryElementType::Face:
			return GeoSelection->Selection.Num();
		}

		UE_LOGF(LogGeometry, Warning, "GeometryScriptMeshSelection::GetNumUniqueSelected() - GeoSelection has invalid type for FGeometryScriptMeshSelection!");
		return 0;
	}
	
}

void FGeometryScriptMeshSelection::DebugPrint() const
{
	bool bIsPolygroups = (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup);
	if (bIsPolygroups)
	{
		UE_LOGF(LogGeometry, Log, "FGeometryScriptSelectionList: Selection contains %d groups:", GeoSelection->Selection.Num());
	}
	else
	{
		UE_LOGF(LogGeometry, Log, "FGeometryScriptSelectionList: Selection contains %d %ls",
			GeoSelection->Selection.Num(), ( GeoSelection->ElementType == EGeometryElementType::Vertex ? TEXT("Vertices") : TEXT("Triangles") ) );
	}

	TStringBuilder<4096> StringBuilder;
	for (uint64 Item : GeoSelection->Selection)
	{
		StringBuilder.Appendf(TEXT(" %d"), (bIsPolygroups) ? FGeoSelectionID(Item).TopologyID : FGeoSelectionID(Item).GeometryID);
	}
	FString ResultString(StringBuilder.ToString());
	UE_LOGF(LogGeometry, Log, "Indices: %ls", *ResultString);
}




void FGeometryScriptMeshSelection::CombineSelectionInPlace(const FGeometryScriptMeshSelection& SelectionB, EGeometryScriptCombineSelectionMode CombineMode)
{
	if (ensure(GetSelectionType() == SelectionB.GetSelectionType()) == false)
	{
		return;
	}

	if (GetSelectionType() != EGeometryScriptMeshSelectionType::Polygroups)
	{
		if (CombineMode == EGeometryScriptCombineSelectionMode::Add)
		{
			for (uint64 ItemB : SelectionB.GeoSelection->Selection)
			{
				GeoSelection->Selection.Add(ItemB);
			}
		}
		else if (CombineMode == EGeometryScriptCombineSelectionMode::Subtract)
		{
			if (SelectionB.IsEmpty() == false)
			{
				for (uint64 ItemB : SelectionB.GeoSelection->Selection)
				{
					GeoSelection->Selection.Remove(ItemB);
				}
				GeoSelection->Selection.Compact();
			}
		}
		else if (CombineMode == EGeometryScriptCombineSelectionMode::Intersection)
		{
			TArray<uint64, TInlineAllocator<32>> ToRemove;
			for (uint64 ItemA : GeoSelection->Selection)
			{
				if (!SelectionB.GeoSelection->Selection.Contains(ItemA))
				{
					ToRemove.Add(ItemA);
				}
			}
			if (ToRemove.Num() > 0)
			{
				for (uint64 ItemA : ToRemove)
				{
					GeoSelection->Selection.Remove(ItemA);
				}
				GeoSelection->Selection.Compact();
			}
		}
	}
	else
	{
		// for Polygroup selections, we cannot rely on TSet operations because we have set an arbitrary Triangle ID 
		// as the 'geometry' key.
		if (CombineMode == EGeometryScriptCombineSelectionMode::Add)
		{
			for (uint64 ItemB : SelectionB.GeoSelection->Selection)
			{
				uint64 FoundItemA;
				if ( UE::Geometry::FindInSelectionByTopologyID(*GeoSelection, FGeoSelectionID(ItemB).TopologyID, FoundItemA) == false)
				{
					GeoSelection->Selection.Add(ItemB);
				}
			}
		}
		else if (CombineMode == EGeometryScriptCombineSelectionMode::Subtract)
		{
			if (SelectionB.IsEmpty() == false)
			{
				for (uint64 ItemB : SelectionB.GeoSelection->Selection)
				{
					uint64 FoundItemA;
					if (UE::Geometry::FindInSelectionByTopologyID(*GeoSelection, FGeoSelectionID(ItemB).TopologyID, FoundItemA))
					{
						GeoSelection->Selection.Remove(FoundItemA);
					}
				}
				GeoSelection->Selection.Compact();
			}
		}
		else if (CombineMode == EGeometryScriptCombineSelectionMode::Intersection)
		{
			TArray<uint64, TInlineAllocator<32>> ToRemove;
			for (uint64 ItemA : GeoSelection->Selection)
			{
				uint64 FoundItemB;
				if (UE::Geometry::FindInSelectionByTopologyID(*GeoSelection, FGeoSelectionID(ItemA).TopologyID, FoundItemB) == false)
				{
					ToRemove.Add(ItemA);
				}
			}
			if (ToRemove.Num() > 0)
			{
				for (uint64 ItemA : ToRemove)
				{
					GeoSelection->Selection.Remove(ItemA);
				}
				GeoSelection->Selection.Compact();
			}
		}
	}

}



EGeometryScriptIndexType FGeometryScriptMeshSelection::ConvertToMeshIndexArray(
	const FDynamicMesh3& Mesh, TArray<int32>& IndexListOut, EGeometryScriptIndexType ConvertToType) const
{
	IndexListOut.Reset();

	// utility fn to use below
	auto EnumerateGroupTriangles = [&Mesh, this](TFunctionRef<void(int&)> TriFunc)
	{
		TSet<int32> GroupIDs;
		for (uint64 Item : GeoSelection->Selection)
		{
			GroupIDs.Add(FGeoSelectionID(Item).TopologyID);
		}
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			if ( GroupIDs.Contains( Mesh.GetTriangleGroup(tid) )  )
			{
				TriFunc(tid);
			}
		}
	};

	bool bIsSameType =
		(ConvertToType == EGeometryScriptIndexType::Vertex && GeoSelection->TopologyType == EGeometryTopologyType::Triangle && GeoSelection->ElementType == EGeometryElementType::Vertex)
		|| (ConvertToType == EGeometryScriptIndexType::Edge && GeoSelection->TopologyType == EGeometryTopologyType::Triangle && GeoSelection->ElementType == EGeometryElementType::Edge)
		|| (ConvertToType == EGeometryScriptIndexType::Triangle && GeoSelection->TopologyType == EGeometryTopologyType::Triangle && GeoSelection->ElementType == EGeometryElementType::Face)
		|| (ConvertToType == EGeometryScriptIndexType::PolygroupID && GeoSelection->TopologyType == EGeometryTopologyType::Polygroup && GeoSelection->ElementType == EGeometryElementType::Face);

	if (bIsSameType || ConvertToType == EGeometryScriptIndexType::Any)
	{
		// keep existing type
		if (GeoSelection->TopologyType == EGeometryTopologyType::Triangle)
		{
			if (GeoSelection->ElementType == EGeometryElementType::Vertex)
			{
				for (uint64 Item : GeoSelection->Selection)
				{
					int32 VertexID = FGeoSelectionID(Item).GeometryID;
					if (Mesh.IsVertex(VertexID))
					{
						IndexListOut.Add( VertexID );
					}
				}
				return EGeometryScriptIndexType::Vertex;
			}
			else if (GeoSelection->ElementType == EGeometryElementType::Edge)
			{
				UE::Private::GeometryScriptSelectionLocal::EnumerateUniqueEdgesInEdgeSelection(Mesh, *GeoSelection, [&IndexListOut](int32 EdgeID)
				{
					IndexListOut.Add(EdgeID);
				});

				return EGeometryScriptIndexType::Edge;
			}
			else if (GeoSelection->ElementType == EGeometryElementType::Face)
			{
				for (uint64 Item : GeoSelection->Selection)
				{
					int32 TriangleID = FGeoSelectionID(Item).GeometryID;
					if (Mesh.IsTriangle(TriangleID))
					{
						IndexListOut.Add( TriangleID );
					}
				}
				return EGeometryScriptIndexType::Triangle;
			}
		}
		else if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup && GeoSelection->ElementType == EGeometryElementType::Face)
		{
			for (uint64 Item : GeoSelection->Selection)
			{
				int32 GroupID = FGeoSelectionID(Item).TopologyID;
				IndexListOut.Add( GroupID );		// too expensive to validate GroupID...
			}
			return EGeometryScriptIndexType::PolygroupID;
		}

	}
	else if (ConvertToType == EGeometryScriptIndexType::Triangle)
	{
		TSet<int32> UniqueTriangles;

		if (GeoSelection->TopologyType == EGeometryTopologyType::Triangle)
		{
			UE::Geometry::EnumerateSelectionTriangles(*GeoSelection, Mesh,
				[&](int32 TriangleID) { UniqueTriangles.Add(TriangleID); });
			for (int32 tid : UniqueTriangles)
			{
				IndexListOut.Add(tid);
			}
			return EGeometryScriptIndexType::Triangle;
		}
		else if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup && GeoSelection->ElementType == EGeometryElementType::Face)
		{
			// Currently doing this locally because the util functions don't support a GeoSelectionID w/ a Group ID but not a Triangle ID
			EnumerateGroupTriangles( [&](int32 TriangleID) { IndexListOut.Add(TriangleID); });
			return EGeometryScriptIndexType::Triangle;
		}

	}
	else if (ConvertToType == EGeometryScriptIndexType::Edge)
	{
		TSet<int32> UniqueEdges;
		if (GeoSelection->TopologyType == EGeometryTopologyType::Triangle)
		{
			UE::Geometry::EnumerateSelectionEdges(*GeoSelection, Mesh,
			[&Mesh, &UniqueEdges, &IndexListOut](int32 EdgeID)
			{
				bool bAlreadyInSet;
				UniqueEdges.Add(EdgeID, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					IndexListOut.Add(EdgeID);
				}
			});
			return EGeometryScriptIndexType::Edge;
		}
		else if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup && GeoSelection->ElementType == EGeometryElementType::Face)
		{
			// Currently doing this locally because the util functions don't support a GeoSelectionID w/ a Group ID but not a Triangle ID
			EnumerateGroupTriangles([&Mesh, &UniqueEdges, &IndexListOut](int32 TriangleID)
			{
				FIndex3i TriEdges = Mesh.GetTriEdges(TriangleID);
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					bool bAlreadyInSet;
					UniqueEdges.Add(TriEdges[SubIdx], &bAlreadyInSet);
					if (!bAlreadyInSet)
					{
						IndexListOut.Add(TriEdges[SubIdx]);
					}
				}
			});
			return EGeometryScriptIndexType::Edge;
		}
	}
	else if (ConvertToType == EGeometryScriptIndexType::Vertex)
	{
		TSet<int32> UniqueVertices;

		if (GeoSelection->TopologyType == EGeometryTopologyType::Triangle)
		{
			UE::Geometry::EnumerateTriangleSelectionVertices(*GeoSelection, Mesh, nullptr, [&UniqueVertices, &IndexListOut](uint64 VertexID, const FVector3d&)
			{
				int32 CastVID = (int32)VertexID;
				bool bAlreadyInSet;
				UniqueVertices.Add(CastVID, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					IndexListOut.Add(CastVID);
				}
			});
			return EGeometryScriptIndexType::Vertex;
		}
		else if (GeoSelection->TopologyType == EGeometryTopologyType::Polygroup && GeoSelection->ElementType == EGeometryElementType::Face)
		{
			// Currently doing this locally because the util functions don't support a GeoSelectionID w/ a Group ID but not a Triangle ID
			EnumerateGroupTriangles([&](int32 TriangleID) {
				FIndex3i TriV = Mesh.GetTriangle(TriangleID);
				UniqueVertices.Add(TriV.A);
				UniqueVertices.Add(TriV.B);
				UniqueVertices.Add(TriV.C);
			});
			for (int32 vid : UniqueVertices)
			{
				IndexListOut.Add(vid);
			}
			return EGeometryScriptIndexType::Vertex;
		}
	}
	else if (ConvertToType == EGeometryScriptIndexType::PolygroupID)
	{
		if ( (GeoSelection->TopologyType == EGeometryTopologyType::Triangle) &&
			(GeoSelection->ElementType == EGeometryElementType::Vertex || GeoSelection->ElementType == EGeometryElementType::Face) )
		{
			TSet<int32> UniqueGroups;
			UE::Geometry::EnumerateSelectionTriangles(*GeoSelection, Mesh,
			[&](int32 TriangleID) { 
				UniqueGroups.Add( Mesh.GetTriangleGroup(TriangleID) ); 
			});
			for (int32 gid : UniqueGroups)
			{
				IndexListOut.Add(gid);
			}
			return EGeometryScriptIndexType::PolygroupID;
		}
	}

	// indicates failure;
	UE_LOGF(LogGeometry, Warning, "FGeometryScriptSelectionListInternal: Conversion not currently supported");
	return EGeometryScriptIndexType::Any;
}




void FGeometryScriptMeshSelection::ProcessByTriangleID(const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> PerTriangleFunc,
	bool bProcessAllTrisIfSelectionEmpty) const
{
	if (IsEmpty() && bProcessAllTrisIfSelectionEmpty)
	{
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			PerTriangleFunc(TriangleID);
		}
		return;
	}

	if (GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles)
	{
		for (uint64 Item : GeoSelection->Selection)
		{
			int32 TriangleID = FGeoSelectionID(Item).GeometryID;
			if (Mesh.IsTriangle(TriangleID))
			{
				PerTriangleFunc(TriangleID);
			}
		}
	}
	else if (GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
	{
		TSet<int32> UniqueItems;
		for (uint64 Item : GeoSelection->Selection)
		{
			int32 VertexID = FGeoSelectionID(Item).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				Mesh.EnumerateVertexTriangles(VertexID, [&](int32 TriangleID)
				{
					bool bAlreadyInSet = false;
					UniqueItems.Add(TriangleID, &bAlreadyInSet);
					if (bAlreadyInSet == false)
					{
						PerTriangleFunc(TriangleID);
					}
				});
			}
		}
	}
	else if (GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
	{
		TSet<int32> UniqueItems;
		for (uint64 Item : GeoSelection->Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(Item).GeometryID);
			if (Mesh.IsTriangle(TriEdgeID.TriangleID))
			{
				int32 EdgeID = Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex);
				FIndex2i EdgeT = Mesh.GetEdgeT(EdgeID);
				for (int32 SubIdx = 0, NumT = EdgeT.B != FDynamicMesh3::InvalidID ? 2 : 1; SubIdx < NumT; ++SubIdx)
				{
					bool bAlreadyInSet = false;
					UniqueItems.Add(EdgeT[SubIdx], &bAlreadyInSet);
					if (!bAlreadyInSet)
					{
						PerTriangleFunc(EdgeT[SubIdx]);
					}
				}
			}
		}
	}
	else
	{
		TSet<int32> GroupIDs;
		for (uint64 Item : GeoSelection->Selection)
		{
			GroupIDs.Add(FGeoSelectionID(Item).TopologyID);
		}
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			if ( GroupIDs.Contains( Mesh.GetTriangleGroup(TriangleID) )  )
			{
				PerTriangleFunc(TriangleID);
			}
		}
	}
}


void FGeometryScriptMeshSelection::ProcessByVertexID(const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> PerVertexFunc,
	bool bProcessAllVertsIfSelectionEmpty) const
{
	if (IsEmpty() && bProcessAllVertsIfSelectionEmpty)
	{
		for (int32 VertexID : Mesh.VertexIndicesItr())
		{
			PerVertexFunc(VertexID);
		}
		return;
	}

	if (GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
	{
		for (uint64 Item : GeoSelection->Selection)
		{
			int32 VertexID = FGeoSelectionID(Item).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				PerVertexFunc(VertexID);
			}
		}
	}
	else 
	{
		TSet<int32> UniqueVertices;
		auto ProcessTriangle = [&Mesh, &PerVertexFunc, &UniqueVertices](int32 TriangleID)
		{
			FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
			for ( int32 j = 0; j < 3; ++j )
			{
				bool bAlreadyInSet = false;
				UniqueVertices.Add(Triangle[j], &bAlreadyInSet);
				if (bAlreadyInSet == false)
				{
					PerVertexFunc(Triangle[j]);
				}
			}
		};

		if (GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles)
		{
			for (uint64 Item : GeoSelection->Selection)
			{
				int32 TriangleID = FGeoSelectionID(Item).GeometryID;
				if (Mesh.IsTriangle(TriangleID))
				{
					ProcessTriangle(TriangleID);
				}
			}
		}
		else if (GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
		{
			for (uint64 Item : GeoSelection->Selection)
			{
				FMeshTriEdgeID TriEdgeID(FGeoSelectionID(Item).GeometryID);
				if (Mesh.IsTriangle(TriEdgeID.TriangleID))
				{
					int32 EdgeID = Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex);
					FIndex2i EdgeV = Mesh.GetEdgeV(EdgeID);
					for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
					{
						bool bAlreadyInSet = false;
						UniqueVertices.Add(EdgeV[SubIdx], &bAlreadyInSet);
						if (!bAlreadyInSet)
						{
							PerVertexFunc(EdgeV[SubIdx]);
						}
					}
				}
			}
		}
		else
		{
			TSet<int32> GroupIDs;
			for (uint64 Item : GeoSelection->Selection)
			{
				GroupIDs.Add(FGeoSelectionID(Item).TopologyID);
			}
			for (int32 TriangleID : Mesh.TriangleIndicesItr())
			{
				if ( GroupIDs.Contains( Mesh.GetTriangleGroup(TriangleID) )  )
				{
					ProcessTriangle(TriangleID);
				}
			}
		}
	}
}

void FGeometryScriptMeshSelection::ProcessByEdgeID(const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> PerEdgeFunc,
	bool bProcessAllEdgesIfSelectionEmpty) const
{
	if (IsEmpty())
	{
		if (bProcessAllEdgesIfSelectionEmpty)
		{
			for (int32 EdgeID : Mesh.EdgeIndicesItr())
			{
				PerEdgeFunc(EdgeID);
			}
		}
		return;
	}

	if (GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
	{
		UE::Private::GeometryScriptSelectionLocal::EnumerateUniqueEdgesInEdgeSelection(Mesh, *GeoSelection, PerEdgeFunc);
	}
	else
	{
		TSet<int32> UniqueEdges;
		auto ProcessTriangle = [&Mesh, &PerEdgeFunc, &UniqueEdges](int32 TriangleID)
		{
			FIndex3i TriEdges = Mesh.GetTriEdges(TriangleID);
			for (int32 j = 0; j < 3; ++j)
			{
				bool bAlreadyInSet = false;
				UniqueEdges.Add(TriEdges[j], &bAlreadyInSet);
				if (bAlreadyInSet == false)
				{
					PerEdgeFunc(TriEdges[j]);
				}
			}
		};

		if (GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles)
		{
			for (uint64 Item : GeoSelection->Selection)
			{
				int32 TriangleID = FGeoSelectionID(Item).GeometryID;
				if (Mesh.IsTriangle(TriangleID))
				{
					ProcessTriangle(TriangleID);
				}
			}
		}
		else if (GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
		{
			TSet<int32> UniqueItems;
			for (uint64 Item : GeoSelection->Selection)
			{
				int32 VertexID = FGeoSelectionID(Item).GeometryID;
				if (Mesh.IsVertex(VertexID))
				{
					Mesh.EnumerateVertexEdges(VertexID, [&UniqueItems, &PerEdgeFunc](int32 EdgeID)
						{
							bool bAlreadyInSet = false;
							UniqueItems.Add(EdgeID, &bAlreadyInSet);
							if (bAlreadyInSet == false)
							{
								PerEdgeFunc(EdgeID);
							}
						});
				}
			}
		}
		else // polygroups
		{
			TSet<int32> GroupIDs;
			for (uint64 Item : GeoSelection->Selection)
			{
				GroupIDs.Add(FGeoSelectionID(Item).TopologyID);
			}
			for (int32 TriangleID : Mesh.TriangleIndicesItr())
			{
				if (GroupIDs.Contains(Mesh.GetTriangleGroup(TriangleID)))
				{
					ProcessTriangle(TriangleID);
				}
			}
		}
	}
}

