// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryBase.h"

#include "Curve/GeneralPolygon2.h"
#include "CompGeom/ConvexDecomposition3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryScriptTypes)


static FString FindCurrentBPFunction()
{
#if WITH_EDITOR
	const FBlueprintContextTracker* ContextTracker = FBlueprintContextTracker::TryGet();
	if (ContextTracker && !ContextTracker->GetCurrentScriptStack().IsEmpty())
	{
		TStringBuilder<256> StringBuilder;
		ContextTracker->GetCurrentScriptStack().Last()->GetStackDescription(StringBuilder);
		return StringBuilder.ToString();
	}
#endif
	return {};
}

FGeometryScriptDebugMessage UE::Geometry::MakeScriptError(EGeometryScriptErrorType ErrorTypeIn, const FText& MessageIn)
{
	FString CurrentBPFunction = FindCurrentBPFunction();
	
	if (CurrentBPFunction.IsEmpty())
	{
		UE_LOGF(LogGeometry, Error, "%ls", *MessageIn.ToString() );
	}
	else
	{
		UE_LOGF(LogGeometry, Error, "%ls [Called from: %ls]", *MessageIn.ToString(), *CurrentBPFunction);
	}

	return FGeometryScriptDebugMessage{ EGeometryScriptDebugMessageType::ErrorMessage, ErrorTypeIn, MessageIn };
}

FGeometryScriptDebugMessage UE::Geometry::MakeScriptWarning(EGeometryScriptErrorType WarningTypeIn, const FText& MessageIn)
{
	FString CurrentBPFunction = FindCurrentBPFunction();
	
	if (CurrentBPFunction.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "%ls", *MessageIn.ToString() );
	}
	else
	{
		UE_LOGF(LogGeometry, Warning, "%ls [Called from: %ls]", *MessageIn.ToString(), *CurrentBPFunction);
	}
	
	return FGeometryScriptDebugMessage{ EGeometryScriptDebugMessageType::WarningMessage, WarningTypeIn, MessageIn };
}






void UE::Geometry::AppendError(UGeometryScriptDebug* Debug, EGeometryScriptErrorType ErrorTypeIn, const FText& MessageIn)
{
	FGeometryScriptDebugMessage Result = MakeScriptError(ErrorTypeIn, MessageIn);
	if (Debug != nullptr)
	{
		Debug->Append(Result);
	}
}

void UE::Geometry::AppendWarning(UGeometryScriptDebug* Debug, EGeometryScriptErrorType WarningTypeIn, const FText& MessageIn)
{
	FGeometryScriptDebugMessage Result = MakeScriptWarning(WarningTypeIn, MessageIn);
	if (Debug != nullptr)
	{
		Debug->Append(Result);
	}
}

void UE::Geometry::AppendError(TArray<FGeometryScriptDebugMessage>* DebugMessages, EGeometryScriptErrorType ErrorType, const FText& Message)
{
	FGeometryScriptDebugMessage Result = MakeScriptError(ErrorType, Message);
	if (DebugMessages != nullptr)
	{
		DebugMessages->Add(Result);
	}
}

void UE::Geometry::AppendWarning(TArray<FGeometryScriptDebugMessage>* DebugMessages, EGeometryScriptErrorType WarningType, const FText& Message)
{
	FGeometryScriptDebugMessage Result = MakeScriptWarning(WarningType, Message);
	if (DebugMessages != nullptr)
	{
		DebugMessages->Add(Result);
	}
}


void FGeometryScriptSphereCovering::Reset()
{
	if (Spheres.IsValid() == false)
	{
		Spheres = MakeShared<UE::Geometry::FSphereCovering>();
	}
	Spheres->Reset();
}


void FGeometryScriptGeneralPolygonList::Reset(int32 Num)
{
	if (!Polygons.IsValid())
	{
		Polygons = MakeShared<TArray<UE::Geometry::FGeneralPolygon2d>>();
	}
	Polygons->Reset(Num);
}
