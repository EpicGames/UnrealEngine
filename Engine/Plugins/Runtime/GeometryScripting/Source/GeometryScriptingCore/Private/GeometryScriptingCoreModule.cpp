// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScriptingCoreModule.h"
#include "UObject/CoreRedirects.h"

#define LOCTEXT_NAMESPACE "FGeometryScriptingCoreModule"

void FGeometryScriptingCoreModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	TArray<FCoreRedirect> Redirects;
	// UE5.0 - AppendRectangle and AppendRoundRectangle shipped in Preview 1 interpreted dimensions incorrectly
	Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("AppendRectangle"), TEXT("AppendRectangle_Compatibility_5_0"));
	Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("AppendRoundRectangle"), TEXT("AppendRoundRectangle_Compatibility_5_0"));

	// UE5.1 - added selection argument to various functions
	Redirects.Emplace(ECoreRedirectFlags::Type_Function, TEXT("ApplyMeshExtrude"), TEXT("ApplyMeshExtrude_Compatibility_5p0"));


	FCoreRedirects::AddRedirectList(Redirects, TEXT("GeometryScriptingCore"));
}

void FGeometryScriptingCoreModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryScriptingCoreModule, GeometryScriptingCore)