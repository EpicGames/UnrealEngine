// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/ListUtilityFunctions.h"
#include "VectorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ListUtilityFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_ListUtilityFunctions"


int UGeometryScriptLibrary_ListUtilityFunctions::GetIndexListLength(FGeometryScriptIndexList IndexList)
{
	return (IndexList.List.IsValid()) ? IndexList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetIndexListLastIndex(FGeometryScriptIndexList IndexList)
{
	return IndexList.List.IsValid() ? IndexList.List->Num()-1 : -1;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetIndexListItem(FGeometryScriptIndexList IndexList, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (IndexList.List.IsValid() && Index >= 0 && Index < IndexList.List->Num())
	{
		bIsValidIndex = true;
		return (*IndexList.List)[Index];
	}
	return -1;
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetIndexListItem(FGeometryScriptIndexList& IndexList, int Index, int NewValue, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (IndexList.List.IsValid() && Index >= 0 && Index < IndexList.List->Num())
	{
		bIsValidIndex = true;
		(*IndexList.List)[Index] = NewValue;
	}
}


void UGeometryScriptLibrary_ListUtilityFunctions::ConvertIndexListToArray(FGeometryScriptIndexList IndexList, TArray<int>& IndexArray)
{
	IndexArray.Reset();
	if (IndexList.List.IsValid())
	{
		IndexArray.Append(*IndexList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToIndexList(const TArray<int>& IndexArray, FGeometryScriptIndexList& IndexList, EGeometryScriptIndexType IndexType)
{
	IndexList.Reset(IndexType);
	IndexList.List->Append(IndexArray);
}

void UGeometryScriptLibrary_ListUtilityFunctions::DuplicateIndexList(FGeometryScriptIndexList IndexList, FGeometryScriptIndexList& DuplicateList)
{
	DuplicateList.Reset(IndexList.IndexType);
	*DuplicateList.List = *IndexList.List;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ClearIndexList(FGeometryScriptIndexList& IndexList, int ClearValue)
{
	int Num = GetIndexListLength(IndexList);
	IndexList.Reset(IndexList.IndexType);
	IndexList.List->Init(ClearValue, Num);
}




int UGeometryScriptLibrary_ListUtilityFunctions::GetTriangleListLength(FGeometryScriptTriangleList TriangleList)
{
	return (TriangleList.List.IsValid()) ? TriangleList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetTriangleListLastTriangle(FGeometryScriptTriangleList TriangleList)
{
	return (TriangleList.List.IsValid()) ? FMath::Max(TriangleList.List->Num()-1,0) : 0;
}

FIntVector UGeometryScriptLibrary_ListUtilityFunctions::GetTriangleListItem(FGeometryScriptTriangleList TriangleList, int Triangle, bool& bIsValidTriangle)
{
	bIsValidTriangle = false;
	if (TriangleList.List.IsValid() && Triangle >= 0 && Triangle < TriangleList.List->Num())
	{
		bIsValidTriangle = true;
		return (*TriangleList.List)[Triangle];
	}
	return FIntVector::NoneValue;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertTriangleListToArray(FGeometryScriptTriangleList TriangleList, TArray<FIntVector>& TriangleArray)
{
	TriangleArray.Reset();
	if (TriangleList.List.IsValid())
	{
		TriangleArray.Append(*TriangleList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToTriangleList(const TArray<FIntVector>& TriangleArray, FGeometryScriptTriangleList& TriangleList)
{
	TriangleList.Reset();
	TriangleList.List->Append(TriangleArray);
}





int UGeometryScriptLibrary_ListUtilityFunctions::GetScalarListLength(FGeometryScriptScalarList ScalarList)
{
	return (ScalarList.List.IsValid()) ? ScalarList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetScalarListLastIndex(FGeometryScriptScalarList ScalarList)
{
	return ScalarList.List.IsValid() ? ScalarList.List->Num()-1 : -1;
}

double UGeometryScriptLibrary_ListUtilityFunctions::GetScalarListItem(FGeometryScriptScalarList ScalarList, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (ScalarList.List.IsValid() && Index >= 0 && Index < ScalarList.List->Num())
	{
		bIsValidIndex = true;
		return (*ScalarList.List)[Index];
	}
	return 0.0;
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetScalarListItem(FGeometryScriptScalarList& ScalarList, int Index, double NewValue, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (ScalarList.List.IsValid() && Index >= 0 && Index < ScalarList.List->Num())
	{
		bIsValidIndex = true;
		(*ScalarList.List)[Index] = NewValue;
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertScalarListToArray(FGeometryScriptScalarList ScalarList, TArray<double>& ScalarArray)
{
	ScalarArray.Reset();
	if (ScalarList.List.IsValid())
	{
		ScalarArray.Append(*ScalarList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToScalarList(const TArray<double>& ScalarArray, FGeometryScriptScalarList& ScalarList)
{
	ScalarList.Reset();
	ScalarList.List->Append(ScalarArray);
}

void UGeometryScriptLibrary_ListUtilityFunctions::DuplicateScalarList(FGeometryScriptScalarList ScalarList, FGeometryScriptScalarList& DuplicateList)
{
	DuplicateList.Reset();
	*DuplicateList.List = *ScalarList.List;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ClearScalarList(FGeometryScriptScalarList& ScalarList, double ClearValue)
{
	int Num = GetScalarListLength(ScalarList);
	ScalarList.Reset();
	ScalarList.List->Init(ClearValue, Num);
}





int UGeometryScriptLibrary_ListUtilityFunctions::GetVectorListLength(FGeometryScriptVectorList VectorList)
{
	return (VectorList.List.IsValid()) ? VectorList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetVectorListLastIndex(FGeometryScriptVectorList VectorList)
{
	return VectorList.List.IsValid() ? VectorList.List->Num()-1 : -1;
}


FVector UGeometryScriptLibrary_ListUtilityFunctions::GetVectorListItem(FGeometryScriptVectorList VectorList, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (VectorList.List.IsValid() && Index >= 0 && Index < VectorList.List->Num())
	{
		bIsValidIndex = true;
		return (*VectorList.List)[Index];
	}
	return FVector::Zero();
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetVectorListItem(FGeometryScriptVectorList& VectorList, int Index, FVector NewValue, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (VectorList.List.IsValid() && Index >= 0 && Index < VectorList.List->Num())
	{
		bIsValidIndex = true;
		(*VectorList.List)[Index] = NewValue;
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertVectorListToArray(FGeometryScriptVectorList VectorList, TArray<FVector>& VectorArray)
{
	VectorArray.Reset();
	if (VectorList.List.IsValid())
	{
		VectorArray.Append(*VectorList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToVectorList(const TArray<FVector>& VectorArray, FGeometryScriptVectorList& VectorList)
{
	VectorList.Reset();
	VectorList.List->Append(VectorArray);
}

void UGeometryScriptLibrary_ListUtilityFunctions::DuplicateVectorList(FGeometryScriptVectorList VectorList, FGeometryScriptVectorList& DuplicateList)
{
	DuplicateList.Reset();
	*DuplicateList.List = *VectorList.List;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ClearVectorList(FGeometryScriptVectorList& VectorList, FVector ClearValue)
{
	int Num = GetVectorListLength(VectorList);
	VectorList.Reset();
	VectorList.List->Init(ClearValue, Num);
}



int UGeometryScriptLibrary_ListUtilityFunctions::GetUVListLength(FGeometryScriptUVList UVList)
{
	return (UVList.List.IsValid()) ? UVList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetUVListLastIndex(FGeometryScriptUVList UVList)
{
	return UVList.List.IsValid() ? UVList.List->Num()-1 : -1;
}

FVector2D UGeometryScriptLibrary_ListUtilityFunctions::GetUVListItem(FGeometryScriptUVList UVList, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (UVList.List.IsValid() && Index >= 0 && Index < UVList.List->Num())
	{
		bIsValidIndex = true;
		return (*UVList.List)[Index];
	}
	return FVector2D::ZeroVector;
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetUVListItem(FGeometryScriptUVList& UVList, int Index, FVector2D NewUV, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (UVList.List.IsValid() && Index >= 0 && Index < UVList.List->Num())
	{
		bIsValidIndex = true;
		(*UVList.List)[Index] = NewUV;
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertUVListToArray(FGeometryScriptUVList UVList, TArray<FVector2D>& UVArray)
{
	UVArray.Reset();
	if (UVList.List.IsValid())
	{
		UVArray.Append(*UVList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToUVList(const TArray<FVector2D>& UVArray, FGeometryScriptUVList& UVList)
{
	UVList.Reset();
	UVList.List->Append(UVArray);
}

void UGeometryScriptLibrary_ListUtilityFunctions::DuplicateUVList(FGeometryScriptUVList UVList, FGeometryScriptUVList& DuplicateList)
{
	DuplicateList.Reset();
	*DuplicateList.List = *UVList.List;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ClearUVList(FGeometryScriptUVList& UVList, FVector2D ClearUV)
{
	int Num = GetUVListLength(UVList);
	UVList.Reset();
	UVList.List->Init(ClearUV, Num);
}




int UGeometryScriptLibrary_ListUtilityFunctions::GetColorListLength(FGeometryScriptColorList ColorList)
{
	return (ColorList.List.IsValid()) ? ColorList.List->Num() : 0;
}

int UGeometryScriptLibrary_ListUtilityFunctions::GetColorListLastIndex(FGeometryScriptColorList ColorList)
{
	return ColorList.List.IsValid() ? ColorList.List->Num()-1 : -1;
}

FLinearColor UGeometryScriptLibrary_ListUtilityFunctions::GetColorListItem(FGeometryScriptColorList ColorList, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (ColorList.List.IsValid() && Index >= 0 && Index < ColorList.List->Num())
	{
		bIsValidIndex = true;
		return (*ColorList.List)[Index];
	}
	return FLinearColor::White;
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetColorListItem(FGeometryScriptColorList& ColorList, int Index, FLinearColor NewColor, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (ColorList.List.IsValid() && Index >= 0 && Index < ColorList.List->Num())
	{
		bIsValidIndex = true;
		(*ColorList.List)[Index] = NewColor;
	}
}


void UGeometryScriptLibrary_ListUtilityFunctions::ConvertColorListToArray(FGeometryScriptColorList ColorList, TArray<FLinearColor>& ColorArray)
{
	ColorArray.Reset();
	if (ColorList.List.IsValid())
	{
		ColorArray.Append(*ColorList.List);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ConvertArrayToColorList(const TArray<FLinearColor>& ColorArray, FGeometryScriptColorList& ColorList)
{
	ColorList.Reset();
	ColorList.List->Append(ColorArray);
}


void UGeometryScriptLibrary_ListUtilityFunctions::DuplicateColorList(FGeometryScriptColorList ColorList, FGeometryScriptColorList& DuplicateList)
{
	DuplicateList.Reset();
	*DuplicateList.List = *ColorList.List;
}

void UGeometryScriptLibrary_ListUtilityFunctions::ClearColorList(FGeometryScriptColorList& ColorList, FLinearColor ClearColor)
{
	int Num = GetColorListLength(ColorList);
	ColorList.Reset();
	ColorList.List->Init(ClearColor, Num);
}

namespace UE::ListUtilityFunctionsLocals
{
	//
	// helpers to access different list-array types in a consistent way, for template use
	//

	static double AccessListHelper(const TArray<double>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx];
	}
	static double& AccessListHelper(TArray<double>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx];
	}

	static double AccessListHelper(const TArray<FVector2D>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx][CompIdx];
	}
	static double& AccessListHelper(TArray<FVector2D>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx][CompIdx];
	}

	static double AccessListHelper(const TArray<FVector>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx][CompIdx];
	}
	static double& AccessListHelper(TArray<FVector>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx][CompIdx];
	}

	static float AccessListHelper(const TArray<FLinearColor>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx].Component(CompIdx);
	}
	static float& AccessListHelper(TArray<FLinearColor>& List, int32 ElIdx, int32 CompIdx)
	{
		return List[ElIdx].Component(CompIdx);
	}

	//
	// helpers to get the number of components per element of different list-array types, for template use
	//

	template<typename ElType>
	struct TNumComponents
	{};
	template<>
	struct TNumComponents<double>
	{
		constexpr static int32 Value = 1;
	};
	template<>
	struct TNumComponents<FVector2D>
	{
		constexpr static int32 Value = 2;
	};
	template<>
	struct TNumComponents<FVector>
	{
		constexpr static int32 Value = 3;
	};
	template<>
	struct TNumComponents<FLinearColor>
	{
		constexpr static int32 Value = 4;
	};

	// templated helper to copy a subset of element components values from one array to another (i.e. the Set Components methods)

	template<typename ToRealType, typename ToArrayType, typename FromArrayType>
	void SetArrayComponentsFromArrayHelper(
		ToArrayType& ToArr, const FromArrayType& FromArr, typename ToArrayType::ElementType DefaultVal, 
		int InToOutDimMapping[TNumComponents<typename ToArrayType::ElementType>::Value])
	{
		constexpr int32 ToDims = TNumComponents<typename ToArrayType::ElementType>::Value;
		constexpr int32 FromDims = TNumComponents<typename FromArrayType::ElementType>::Value;
		
		// Validate / clamp component indices
		for (int32 FromDim = 0; FromDim < FromDims; ++FromDim)
		{
			if (InToOutDimMapping[FromDim] < 0 || InToOutDimMapping[FromDim] >= ToDims)
			{
				UE::Geometry::MakeScriptWarning(EGeometryScriptErrorType::InvalidInputs,
					FText::Format(LOCTEXT("InvalidListElementComponentIndex", "Component Index {0} will be clamped to valid range (0-{1})"),
						InToOutDimMapping[FromDim], ToDims - 1));
			}
			InToOutDimMapping[FromDim] = FMath::Clamp(InToOutDimMapping[FromDim], 0, ToDims - 1);
		}
		int32 InitialToArrNum = ToArr.Num();
		int32 NoDefaultNeededNum = FMath::Min(ToArr.Num(), FromArr.Num());

		// Helper to copy across the requested components for a single element
		auto CopyElementComponents = [&InToOutDimMapping, &FromArr, &ToArr, FromDims](int32 Idx)
		{
			for (int32 FromDimIdx = 0; FromDimIdx < FromDims; ++FromDimIdx)
			{
				int32 ToDimIdx = InToOutDimMapping[FromDimIdx];
				AccessListHelper(ToArr, Idx, ToDimIdx) = (ToRealType)AccessListHelper(FromArr, Idx, FromDimIdx);
			}
		};

		// expand the 'to' array if needed
		if (ToArr.Num() < FromArr.Num())
		{
			ToArr.SetNumUninitialized(FromArr.Num());
		}
		// for range we didn't just add new elements, no default value is needed -- just write the requested components 
		for (int32 Idx = 0; Idx < NoDefaultNeededNum; ++Idx)
		{
			CopyElementComponents(Idx);
		}
		// for range we added, set the default value before writing requested components
		for (int32 Idx = NoDefaultNeededNum; Idx < FromArr.Num(); ++Idx)
		{
			ToArr[Idx] = DefaultVal;
			CopyElementComponents(Idx);
		}
	}

	// standard validation for the 'Set List from List' methods

	template<typename ListAType, typename ListBType>
	bool ValidateListsToSetAFromB(ListAType& ListA, const ListBType& ListB)
	{
		if (!ListB.List.IsValid())
		{
			return false;
		}
		if (!ListA.List.IsValid())
		{
			ListA.Reset();
		}
		return true;
	}

	// templated helper to extract a subset element components into a new array (i.e. the Extract methods)

	template<typename FromArrayType, typename ToArrayType>
	void ExtractArrayComponentsFromArrayHelper(
		
		const FromArrayType& FromArr, ToArrayType& ToArr, int FromComponent[TNumComponents<typename ToArrayType::ElementType>::Value])
	{
		constexpr int32 ToDims = TNumComponents<typename ToArrayType::ElementType>::Value;
		constexpr int32 FromDims = TNumComponents<typename FromArrayType::ElementType>::Value;
		
		// Validate / clamp component indices
		for (int32 ToDim = 0; ToDim < ToDims; ++ToDim)
		{
			if (FromComponent[ToDim] < 0 || FromComponent[ToDim] >= FromDims)
			{
				UE::Geometry::MakeScriptWarning(EGeometryScriptErrorType::InvalidInputs,
					FText::Format(LOCTEXT("InvalidListElementComponentIndex", "Component Index {0} will be clamped to valid range (0-{1})"), 
						FromComponent[ToDim], FromDims - 1));
			}
			FromComponent[ToDim] = FMath::Clamp(FromComponent[ToDim], 0, FromDims - 1);
		}

		// Extract requested components to output array
		ToArr.SetNumUninitialized(FromArr.Num());
		for (int32 Idx = 0; Idx < FromArr.Num(); ++Idx)
		{
			for (int32 ToDimIdx = 0; ToDimIdx < ToDims; ++ToDimIdx)
			{
				int32 FromDimIdx = FromComponent[ToDimIdx];
				AccessListHelper(ToArr, Idx, ToDimIdx) = (double)AccessListHelper(FromArr, Idx, FromDimIdx);
			}
		}
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ExtractColorListChannel(FGeometryScriptColorList ColorList, FGeometryScriptScalarList& ScalarList, int32 ChannelIndex)
{
	using namespace UE::ListUtilityFunctionsLocals;

	ScalarList.Reset();
	if (ColorList.List.IsValid())
	{
		int ChannelMap[1]{ ChannelIndex };
		ExtractArrayComponentsFromArrayHelper(*ColorList.List, *ScalarList.List, ChannelMap);
	}
}


void UGeometryScriptLibrary_ListUtilityFunctions::ExtractColorListChannels(
	FGeometryScriptColorList ColorList, FGeometryScriptVectorList& VectorList, 
	int32 XChannelIndex, int32 YChannelIndex, int32 ZChannelIndex)
{
	using namespace UE::ListUtilityFunctionsLocals;

	VectorList.Reset();
	if (ColorList.List.IsValid())
	{
		int ChannelMap[3]{ XChannelIndex, YChannelIndex, ZChannelIndex };
		ExtractArrayComponentsFromArrayHelper(*ColorList.List, *VectorList.List, ChannelMap);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ExtractUVListComponent(
	FGeometryScriptUVList UVList, FGeometryScriptScalarList& ScalarList, int32 ComponentIndex)
{
	using namespace UE::ListUtilityFunctionsLocals;

	ScalarList.Reset();
	if (UVList.List.IsValid())
	{
		int ComponentMap[1]{ ComponentIndex };
		ExtractArrayComponentsFromArrayHelper(*UVList.List, *ScalarList.List, ComponentMap);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ExtractVectorListComponent(
	FGeometryScriptVectorList VectorList, FGeometryScriptScalarList& ScalarList, int32 ComponentIndex)
{
	using namespace UE::ListUtilityFunctionsLocals;

	ScalarList.Reset();
	if (VectorList.List.IsValid())
	{
		int ComponentMap[1]{ ComponentIndex };
		ExtractArrayComponentsFromArrayHelper(*VectorList.List, *ScalarList.List, ComponentMap);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::ExtractVectorListComponentsAsUVs(
	FGeometryScriptVectorList VectorList, FGeometryScriptUVList& UVList, int32 UComponentIndex, int32 VComponentIndex)
{
	using namespace UE::ListUtilityFunctionsLocals;

	UVList.Reset();
	if (VectorList.List.IsValid())
	{
		int ComponentMap[2]{ UComponentIndex, VComponentIndex };
		ExtractArrayComponentsFromArrayHelper(*VectorList.List, *UVList.List, ComponentMap);
	}
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetColorListChannelFromScalars(
	FGeometryScriptColorList& ColorList, FGeometryScriptScalarList ScalarList, int32 ColorChannel, FLinearColor DefaultColor)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(ColorList, ScalarList))
	{
		return;
	}
	int InToOutDimMapping[1]{ ColorChannel };
	SetArrayComponentsFromArrayHelper<float>(*ColorList.List, *ScalarList.List, DefaultColor, InToOutDimMapping);
}


void UGeometryScriptLibrary_ListUtilityFunctions::SetColorListChannelsFromUVs(
	FGeometryScriptColorList& ColorList, FGeometryScriptUVList UVList,
	int32 UToChannelIndex, int32 VToChannelIndex, FLinearColor DefaultColor)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(ColorList, UVList))
	{
		return;
	}
	int InToOutDimMapping[2]{ UToChannelIndex, VToChannelIndex };
	SetArrayComponentsFromArrayHelper<float>(*ColorList.List, *UVList.List, DefaultColor, InToOutDimMapping);
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetColorListChannelsFromVectors(
	FGeometryScriptColorList& ColorList, FGeometryScriptVectorList VectorList,
	int32 XToChannelIndex, int32 YToChannelIndex, int32 ZToChannelIndex, FLinearColor DefaultColor)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(ColorList, VectorList))
	{
		return;
	}
	int InToOutDimMapping[3]{ XToChannelIndex, YToChannelIndex, ZToChannelIndex };
	SetArrayComponentsFromArrayHelper<float>(*ColorList.List, *VectorList.List, DefaultColor, InToOutDimMapping);
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetUVListComponentFromScalars(
	FGeometryScriptUVList& UVList, FGeometryScriptScalarList ScalarList, int32 UVComponent, FVector2D DefaultUV)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(UVList, ScalarList))
	{
		return;
	}
	int InToOutDimMapping[1]{ UVComponent };
	SetArrayComponentsFromArrayHelper<double>(*UVList.List, *ScalarList.List, DefaultUV, InToOutDimMapping);
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetVectorListComponentFromScalars(
	FGeometryScriptVectorList& VectorList, FGeometryScriptScalarList ScalarList, int32 VectorComponent, FVector DefaultVector)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(VectorList, ScalarList))
	{
		return;
	}
	int InToOutDimMapping[1]{ VectorComponent };
	SetArrayComponentsFromArrayHelper<double>(*VectorList.List, *ScalarList.List, DefaultVector, InToOutDimMapping);
}

void UGeometryScriptLibrary_ListUtilityFunctions::SetVectorListComponentsFromUVs(
	FGeometryScriptVectorList& VectorList, FGeometryScriptUVList UVList, int32 UToVectorComponent, int32 VToVectorComponent, FVector DefaultVector)
{
	using namespace UE::ListUtilityFunctionsLocals;

	if (!ValidateListsToSetAFromB(VectorList, UVList))
	{
		return;
	}
	int InToOutDimMapping[2]{ UToVectorComponent, VToVectorComponent };
	SetArrayComponentsFromArrayHelper<double>(*VectorList.List, *UVList.List, DefaultVector, InToOutDimMapping);
}

#undef LOCTEXT_NAMESPACE
