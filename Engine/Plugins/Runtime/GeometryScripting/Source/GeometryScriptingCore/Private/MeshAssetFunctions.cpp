// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshAssetFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "StaticMeshResources.h"
#include "UDynamicMesh.h"

#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkeletalMesh.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "RenderingThread.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshLODRenderDataToDynamicMesh.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "SkeletalMeshOperations.h"
#include "AssetUtils/StaticMeshMaterialUtil.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMesh/MeshNormals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshAssetFunctions)


#if WITH_EDITOR
#include "Editor.h"
#include "ScopedTransaction.h"
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshAssetFunctions"


static void ConvertGeometryScriptReadLOD(const FGeometryScriptMeshReadLOD& ReadLOD, UE::Conversion::EMeshLODType& OutLODType, int32& OutLODIndex)
{
	using namespace UE::Conversion;
	OutLODType = [](const EGeometryScriptLODType& LODType)
		{
			switch (LODType)
			{
			case EGeometryScriptLODType::MaxAvailable:
				return EMeshLODType::MaxAvailable;
			case EGeometryScriptLODType::HiResSourceModel:
				return EMeshLODType::HiResSourceModel;
			case EGeometryScriptLODType::SourceModel:
				return EMeshLODType::SourceModel;
			case EGeometryScriptLODType::RenderData:
				return EMeshLODType::RenderData;
			default:
				checkNoEntry();
				return EMeshLODType::RenderData;
			}
		}(ReadLOD.LODType);
	OutLODIndex = ReadLOD.LODIndex;
}

static void ConvertGeometryScriptWriteLOD(const FGeometryScriptMeshWriteLOD& WriteLOD, UE::Conversion::EMeshLODType& OutLODType, int32& OutLODIndex)
{
	using namespace UE::Conversion;
	OutLODType = WriteLOD.bWriteHiResSource ? EMeshLODType::HiResSourceModel : EMeshLODType::SourceModel;
	OutLODIndex = WriteLOD.LODIndex;
}


UDynamicMesh*  UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMeshV2(
	UStaticMesh* FromStaticMeshAsset, 
	UDynamicMesh* ToDynamicMesh, 
	FGeometryScriptCopyMeshFromAssetOptions AssetOptions,
	FGeometryScriptMeshReadLOD RequestedLOD,
	EGeometryScriptOutcomePins& Outcome,
	bool bUseSectionMaterials,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromAsset_InvalidInput1", "CopyMeshFromStaticMesh: FromStaticMeshAsset is Null"));
		return ToDynamicMesh;
	}
	if (ToDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromAsset_InvalidInput2", "CopyMeshFromStaticMesh: ToDynamicMesh is Null"));
		return ToDynamicMesh;
	}

	using namespace UE::Conversion;
	EMeshLODType LODType;
	int32 LODIndex;
	ConvertGeometryScriptReadLOD(RequestedLOD, LODType, LODIndex);
	FStaticMeshConversionOptions ConversionOptions;
	ConversionOptions.bApplyBuildSettings = AssetOptions.bApplyBuildSettings;
	ConversionOptions.bRequestTangents = AssetOptions.bRequestTangents;
	ConversionOptions.bIgnoreRemoveDegenerates = AssetOptions.bIgnoreRemoveDegenerates;
	ConversionOptions.bUseBuildScale = AssetOptions.bUseBuildScale;
	ConversionOptions.bUseSectionMaterialIndices = bUseSectionMaterials;
	ConversionOptions.bIncludeNonManifoldSrcInfo = true;

	FText ErrorMessage;

	FDynamicMesh3 NewMesh;
	bool bSuccess = StaticMeshToDynamicMesh(FromStaticMeshAsset, NewMesh, ErrorMessage, ConversionOptions, LODType, LODIndex);
	if (!bSuccess)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, ErrorMessage);
	}
	else
	{
		ToDynamicMesh->SetMesh(MoveTemp(NewMesh));
		Outcome = EGeometryScriptOutcomePins::Success;
	}
	return ToDynamicMesh;
}




UDynamicMesh*  UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(
	UDynamicMesh* FromDynamicMesh,
	UStaticMesh* ToStaticMeshAsset,
	FGeometryScriptCopyMeshToAssetOptions Options,
	FGeometryScriptMeshWriteLOD TargetLOD,
	EGeometryScriptOutcomePins& Outcome,
	bool bUseSectionMaterials,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToAsset_InvalidInput1", "CopyMeshToStaticMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}
	if (ToStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToAsset_InvalidInput2", "CopyMeshToStaticMesh: ToStaticMeshAsset is Null"));
		return FromDynamicMesh;
	}

#if WITH_EDITOR

	int32 UseLODIndex = FMath::Clamp(TargetLOD.LODIndex, 0, MAX_STATIC_MESH_LODS);

	// currently material updates are only applied when writing LODs
	if (Options.bReplaceMaterials && TargetLOD.bWriteHiResSource)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToStaticMesh_InvalidOptions1", "CopyMeshToStaticMesh: Can only Replace Materials when updating LODs"));
		return FromDynamicMesh;
	}

	// Don't allow built-in engine assets to be modified. However we do allow assets in /Engine/Transient/ to be updated because
	// this is a location that temporary assets in the Transient package will be created in, and in some cases we want to use
	// script functions on such asset (Datasmith does this for example)
	if ( ToStaticMeshAsset->GetPathName().StartsWith(TEXT("/Engine/")) && ToStaticMeshAsset->GetPathName().StartsWith(TEXT("/Engine/Transient")) == false )
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToAsset_EngineAsset", "CopyMeshToStaticMesh: Cannot modify built-in Engine asset"));
		return FromDynamicMesh;
	}

	// flush any pending rendering commands, which might want to touch this StaticMesh while we are rebuilding it
	FlushRenderingCommands();

	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("UpdateStaticMesh", "Update Static Mesh"));
	}

	// make sure transactional flag is on for the Asset
	ToStaticMeshAsset->SetFlags(RF_Transactional);
	// mark as modified
	ToStaticMeshAsset->Modify();

	// Decide whether to generate lightmap UVs by referencing the current asset settings (before they're modified below)
	bool bShouldGenerateLightmapUVs = false;
	if (Options.GenerateLightmapUVs == EGeometryScriptGenerateLightmapUVOptions::MatchTargetLODSetting)
	{
		const FStaticMeshSourceModel* UseReferenceSourceModel = nullptr;
		if (TargetLOD.bWriteHiResSource)
		{
			UseReferenceSourceModel = &ToStaticMeshAsset->GetHiResSourceModel();
		}
		else
		{
			if (ToStaticMeshAsset->IsSourceModelValid(TargetLOD.LODIndex))
			{
				UseReferenceSourceModel = &ToStaticMeshAsset->GetSourceModel(TargetLOD.LODIndex);
			}
			else if (ToStaticMeshAsset->IsSourceModelValid(0))
			{
				UseReferenceSourceModel = &ToStaticMeshAsset->GetSourceModel(0);
			}
		}
		if (UseReferenceSourceModel)
		{
			bShouldGenerateLightmapUVs = UseReferenceSourceModel->BuildSettings.bGenerateLightmapUVs;
		}
	}
	else
	{
		bShouldGenerateLightmapUVs = (Options.GenerateLightmapUVs == EGeometryScriptGenerateLightmapUVOptions::GenerateLightmapUVs);
	}
	
	auto ConfigureBuildSettingsFromOptions = [bShouldGenerateLightmapUVs](FStaticMeshSourceModel& SourceModel, FGeometryScriptCopyMeshToAssetOptions& Options) -> FVector
												{
													FMeshBuildSettings& BuildSettings = SourceModel.BuildSettings;
													BuildSettings.bRecomputeNormals  = Options.bEnableRecomputeNormals;
													BuildSettings.bRecomputeTangents = Options.bEnableRecomputeTangents;
													BuildSettings.bRemoveDegenerates = Options.bEnableRemoveDegenerates;
													BuildSettings.bGenerateLightmapUVs = bShouldGenerateLightmapUVs;
													if (!Options.bUseBuildScale) // if we're not using build scale, set asset BuildScale to 1,1,1
													{
														BuildSettings.BuildScale3D = FVector::OneVector;
													}
													return BuildSettings.BuildScale3D;
												};
	
	auto ApplyInverseBuildScale = [](FMeshDescription& MeshDescription, FVector BuildScale)
	{
		if (BuildScale.Equals(FVector::OneVector))
		{
			return;
		}
		FTransform InverseBuildScaleTransform = FTransform::Identity;
		FVector InverseBuildScale;
		// Safely invert BuildScale
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			InverseBuildScale[Idx] = FMath::IsNearlyZero(BuildScale[Idx], FMathd::Epsilon) ? 1.0 : 1.0 / BuildScale[Idx];
		}
		InverseBuildScaleTransform.SetScale3D(InverseBuildScale);
		FStaticMeshOperations::ApplyTransform(MeshDescription, InverseBuildScaleTransform, true /*use correct normal transforms*/);
	};

	if (TargetLOD.bWriteHiResSource)
	{
		// update model build settings
		FVector BuildScale = ConfigureBuildSettingsFromOptions(ToStaticMeshAsset->GetHiResSourceModel(), Options);

		ToStaticMeshAsset->ModifyHiResMeshDescription();
		FMeshDescription* NewHiResMD = ToStaticMeshAsset->CreateHiResMeshDescription();

		if (!ensure(NewHiResMD != nullptr))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToAsset_NullHiResMeshDescription", "CopyMeshToAsset: MeshDescription for HiRes is null?"));
			return FromDynamicMesh;
		}

		FConversionToMeshDescriptionOptions ConversionOptions;
		if (Options.bCleanAssignedMaterials)
		{
			ConversionOptions.MaxValidMaterialID = ToStaticMeshAsset->GetStaticMaterials().Num() - 1;
		}
		FDynamicMeshToMeshDescription Converter(ConversionOptions);
		FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				Converter.Convert(&ReadMesh, *NewHiResMD, !Options.bEnableRecomputeTangents);
			});

		ApplyInverseBuildScale(*NewHiResMD, BuildScale);

		ToStaticMeshAsset->CommitHiResMeshDescription();
	}
	else
	{ 

		if (ToStaticMeshAsset->GetNumSourceModels() < UseLODIndex+1)
		{
			ToStaticMeshAsset->SetNumSourceModels(UseLODIndex+1);
		}

		// update model build settings
		FVector BuildScale = ConfigureBuildSettingsFromOptions(ToStaticMeshAsset->GetSourceModel(UseLODIndex), Options);

		FMeshDescription* MeshDescription = ToStaticMeshAsset->GetMeshDescription(UseLODIndex);
		if (MeshDescription == nullptr)
		{
			MeshDescription = ToStaticMeshAsset->CreateMeshDescription(UseLODIndex);
		}

		// mark mesh description for modify
		ToStaticMeshAsset->ModifyMeshDescription(UseLODIndex);

		if (!ensure(MeshDescription != nullptr))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs,
				FText::Format(LOCTEXT("CopyMeshToAsset_NullMeshDescription", "CopyMeshToAsset: MeshDescription for LOD {0} is null?"), FText::AsNumber(UseLODIndex)));
			return FromDynamicMesh;
		}

		FConversionToMeshDescriptionOptions ConversionOptions;
		ConversionOptions.bConvertBackToNonManifold = Options.bUseOriginalVertexOrder;
		if (Options.bCleanAssignedMaterials)
		{
			if (Options.bReplaceMaterials)
			{
				ConversionOptions.MaxValidMaterialID = Options.NewMaterials.Num() - 1;
			}
			else
			{
				const FMeshSectionInfoMap& SectionMap = ToStaticMeshAsset->GetSectionInfoMap();
				int32 LODSectionNum = SectionMap.GetSectionNumber(UseLODIndex);
				ConversionOptions.MaxValidMaterialID = LODSectionNum - 1;
			}
		}
		
		FDynamicMeshToMeshDescription Converter(ConversionOptions);
		if (!bUseSectionMaterials && !Options.bReplaceMaterials)
		{
			TArray<int32> MaterialIDMap = UE::Conversion::GetPolygonGroupToMaterialIndexMap(ToStaticMeshAsset, UE::Conversion::EMeshLODType::SourceModel, UseLODIndex);
			Converter.SetMaterialIDMapFromInverseMap(MaterialIDMap);
		}
		FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			Converter.Convert(&ReadMesh, *MeshDescription, !Options.bEnableRecomputeTangents);
		});

		ApplyInverseBuildScale(*MeshDescription, BuildScale);

		// Setting to prevent the standard static mesh reduction from running and replacing the render LOD.
		FStaticMeshSourceModel& ThisSourceModel = ToStaticMeshAsset->GetSourceModel(UseLODIndex);
		ThisSourceModel.ResetReductionSetting();

		if (Options.bApplyNaniteSettings)
		{
			ToStaticMeshAsset->SetNaniteSettings(Options.NewNaniteSettings);
		}

		if (Options.bReplaceMaterials)
		{
			bool bHaveSlotNames = (Options.NewMaterialSlotNames.Num() == Options.NewMaterials.Num());

			auto SetMaterialSlotNames = [&MeshDescription](const TArray<FStaticMaterial>& NewMaterials)
			{
				// Set material slot names on the mesh description
				FStaticMeshAttributes Attributes(*MeshDescription);
				TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
				for (int32 SlotIdx = 0; SlotIdx < NewMaterials.Num(); ++SlotIdx)
				{
					if (SlotIdx < PolygonGroupImportedMaterialSlotNames.GetNumElements())
					{
						PolygonGroupImportedMaterialSlotNames.Set(SlotIdx, NewMaterials[SlotIdx].ImportedMaterialSlotName);
					}
				}
			};

			auto MakeNewMaterial = [&Options, bHaveSlotNames](const TArray<FStaticMaterial>& StaticMeshMaterials, int32 NewMatIdx) -> FStaticMaterial
			{
				FStaticMaterial NewMaterial;
				NewMaterial.MaterialInterface = Options.NewMaterials[NewMatIdx];
				FName UseSlotName = (bHaveSlotNames && Options.NewMaterialSlotNames[NewMatIdx] != NAME_None) ? Options.NewMaterialSlotNames[NewMatIdx] :
					UE::AssetUtils::GenerateNewMaterialSlotName(StaticMeshMaterials, NewMaterial.MaterialInterface, NewMatIdx);

				NewMaterial.MaterialSlotName = UseSlotName;
				NewMaterial.ImportedMaterialSlotName = UseSlotName;
				NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);		// this avoids an ensure in  UStaticMesh::GetUVChannelData
				return NewMaterial;
			};

			if (Options.NewMaterialUsage == ECopyMeshToAssetUseNewMaterialsOptions::ReplaceAssetMaterials)
			{
				TArray<FStaticMaterial> NewMaterials;
				for (int32 k = 0; k < Options.NewMaterials.Num(); ++k)
				{
					NewMaterials.Add(MakeNewMaterial(NewMaterials, k));
				}

				ToStaticMeshAsset->SetStaticMaterials(NewMaterials);

				SetMaterialSlotNames(NewMaterials);

				// Reset the section info map
				ToStaticMeshAsset->GetSectionInfoMap().Clear();
				ToStaticMeshAsset->GetOriginalSectionInfoMap().Clear();

				// Repopulate section info map
				FMeshSectionInfoMap SectionInfoMap;
				for (int32 LODIndex = 0, NumLODs = ToStaticMeshAsset->GetNumSourceModels(); LODIndex < NumLODs; ++LODIndex)
				{
					if (const FMeshDescription* Mesh = (LODIndex == UseLODIndex) ? MeshDescription : ToStaticMeshAsset->GetMeshDescription(LODIndex))
					{
						FStaticMeshConstAttributes MeshDescriptionAttributes(*Mesh);
						TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
						int32 SectionIndex = 0;
						for (FPolygonGroupID PolygonGroupID : Mesh->PolygonGroups().GetElementIDs())
						{
							// Material index is either from the matching material slot name or the section index if that name is not found
							int32 MaterialIndex = ToStaticMeshAsset->GetStaticMaterials().IndexOfByPredicate(
								[&MaterialSlotName = MaterialSlotNames[PolygonGroupID]](const FStaticMaterial& StaticMaterial) { return StaticMaterial.MaterialSlotName == MaterialSlotName; }
							);
							if (MaterialIndex == INDEX_NONE)
							{
								MaterialIndex = SectionIndex;
							}
							SectionInfoMap.Set(LODIndex, SectionIndex, FMeshSectionInfo(MaterialIndex));
							SectionIndex++;
						}
					}
				}
				ToStaticMeshAsset->GetSectionInfoMap().CopyFrom(SectionInfoMap);
				ToStaticMeshAsset->GetOriginalSectionInfoMap().CopyFrom(SectionInfoMap);
			}
			else
			{
				TArray<FStaticMaterial> ToUpdateMaterials = ToStaticMeshAsset->GetStaticMaterials();
				TMap<UMaterialInterface*, int32> MatToIndex;
				for (int32 OldMatIdx = 0; OldMatIdx < ToUpdateMaterials.Num(); ++OldMatIdx)
				{
					MatToIndex.Add(ToUpdateMaterials[OldMatIdx].MaterialInterface, OldMatIdx);
				}
				// 
				for (int32 NewMatIdx = 0; NewMatIdx < Options.NewMaterials.Num(); ++NewMatIdx)
				{
					UMaterialInterface* NewMat = Options.NewMaterials[NewMatIdx];
					if (!MatToIndex.Contains(NewMat))
					{
						int32 NewIdx = ToUpdateMaterials.Add(MakeNewMaterial(ToUpdateMaterials, NewMatIdx));
						MatToIndex.Add(NewMat, NewIdx);
					}
				}

				ToStaticMeshAsset->SetStaticMaterials(ToUpdateMaterials);

				SetMaterialSlotNames(ToUpdateMaterials);

				// update material sections map for the current LOD
				auto UpdateMaterialSectionInfo = [&MatToIndex, &Options](FMeshSectionInfoMap& SectionInfoMap, int32 LODIndex)
				{
					// clear old sections for this LOD
					int32 NumSection = SectionInfoMap.GetSectionNumber(LODIndex);
					for (int32 Idx = 0; Idx < NumSection; ++Idx)
					{
						SectionInfoMap.Remove(LODIndex, Idx);
					}
					// add new section per material in the NewMaterials array
					for (int32 NewMatIdx = 0; NewMatIdx < Options.NewMaterials.Num(); ++NewMatIdx)
					{
						SectionInfoMap.Set(LODIndex, NewMatIdx, FMeshSectionInfo(MatToIndex[Options.NewMaterials[NewMatIdx]]));
					}
				};
				UpdateMaterialSectionInfo(ToStaticMeshAsset->GetSectionInfoMap(), UseLODIndex);
				UpdateMaterialSectionInfo(ToStaticMeshAsset->GetOriginalSectionInfoMap(), UseLODIndex);
			}
		}

		ToStaticMeshAsset->CommitMeshDescription(UseLODIndex);
	}

	if (Options.bDeferMeshPostEditChange == false)
	{
		ToStaticMeshAsset->PostEditChange();
	}

	if (Options.bEmitTransaction && GEditor)
	{
		GEditor->EndTransaction();
	}

	Outcome = EGeometryScriptOutcomePins::Success;

#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToAsset_EditorOnly", "CopyMeshToStaticMesh: Not currently supported at Runtime"));
#endif

	return FromDynamicMesh;
}




bool UGeometryScriptLibrary_StaticMeshFunctions::CheckStaticMeshHasAvailableLOD(
	UStaticMesh* FromStaticMeshAsset,
	FGeometryScriptMeshReadLOD RequestedLOD,
	EGeometryScriptSearchOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptSearchOutcomePins::NotFound;
	if (FromStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CheckStaticMeshHasAvailableLOD_InvalidInput1", "CheckStaticMeshHasAvailableLOD: FromStaticMeshAsset is Null"));
		return false;
	}

	if (RequestedLOD.LODType == EGeometryScriptLODType::RenderData)
	{
		Outcome = (RequestedLOD.LODIndex >= 0 && RequestedLOD.LODIndex < FromStaticMeshAsset->GetNumLODs()) ?
			EGeometryScriptSearchOutcomePins::Found : EGeometryScriptSearchOutcomePins::NotFound;

#if !WITH_EDITOR
		if (FromStaticMeshAsset->bAllowCPUAccess == false)
		{
			Outcome = EGeometryScriptSearchOutcomePins::NotFound;
		}
#endif

		return (Outcome == EGeometryScriptSearchOutcomePins::Found);
	}

#if WITH_EDITOR
	bool bResult = false;
	if (RequestedLOD.LODType == EGeometryScriptLODType::HiResSourceModel)
	{
		bResult = FromStaticMeshAsset->IsHiResMeshDescriptionValid();
	}
	else if (RequestedLOD.LODType == EGeometryScriptLODType::SourceModel)
	{
		bResult = RequestedLOD.LODIndex >= 0
			&& RequestedLOD.LODIndex < FromStaticMeshAsset->GetNumSourceModels()
			&& FromStaticMeshAsset->IsSourceModelValid(RequestedLOD.LODIndex);
	}
	else if (RequestedLOD.LODType == EGeometryScriptLODType::MaxAvailable)
	{
		bResult = (FromStaticMeshAsset->GetNumSourceModels() > 0);
	}
	Outcome = (bResult) ? EGeometryScriptSearchOutcomePins::Found : EGeometryScriptSearchOutcomePins::NotFound;
	return bResult;

#else
	Outcome = EGeometryScriptSearchOutcomePins::NotFound;
	return false;
#endif
}



int UGeometryScriptLibrary_StaticMeshFunctions::GetNumStaticMeshLODsOfType(
	UStaticMesh* FromStaticMeshAsset,
	EGeometryScriptLODType LODType)
{
	if (FromStaticMeshAsset == nullptr) return 0;

#if WITH_EDITOR
	if (LODType == EGeometryScriptLODType::RenderData)
	{
		return FromStaticMeshAsset->GetNumLODs();
	}
	if (LODType == EGeometryScriptLODType::HiResSourceModel)
	{
		return FromStaticMeshAsset->IsHiResMeshDescriptionValid() ? 1 : 0;
	}
	if (LODType == EGeometryScriptLODType::SourceModel || LODType == EGeometryScriptLODType::MaxAvailable)
	{
		return FromStaticMeshAsset->GetNumSourceModels();
	}
#else
	if (LODType == EGeometryScriptLODType::RenderData && FromStaticMeshAsset->bAllowCPUAccess)
	{
		return FromStaticMeshAsset->GetNumLODs();
	}
#endif

	return 0;
}



void UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromStaticMesh(
	const UStaticMesh* FromStaticMeshAsset,
	TArray<UMaterialInterface*>& MaterialList,
	TArray<FName>& MaterialSlotNames,
	UGeometryScriptDebug* Debug)
{
	if (FromStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialListFromStaticMesh_InvalidInput1", "GetMaterialListFromStaticMesh: FromStaticMeshAsset is Null"));
		return;
	}

	const TArray<FStaticMaterial>& AssetMaterials = FromStaticMeshAsset->GetStaticMaterials();
	MaterialList.Reset(AssetMaterials.Num());
	MaterialSlotNames.Reset(AssetMaterials.Num());
	for (int32 k = 0; k < AssetMaterials.Num(); ++k)
	{
		MaterialList.Add(AssetMaterials[k].MaterialInterface);
		MaterialSlotNames.Add(AssetMaterials[k].MaterialSlotName);
	}
}

void UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromSkeletalMesh(
	const USkeletalMesh* FromSkeletalMeshAsset,
	TArray<UMaterialInterface*>& MaterialList,
	TArray<FName>& MaterialSlotNames,
	UGeometryScriptDebug* Debug)
{
	if (FromSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialListFromSkeletalMesh_InvalidInput1", "GetMaterialListFromSkeletalMesh: FromSkeletalMeshAsset is Null"));
		return;
	}

	const TArray<FSkeletalMaterial>& AssetMaterials = FromSkeletalMeshAsset->GetMaterials();
	MaterialList.Reset(AssetMaterials.Num());
	MaterialSlotNames.Reset(AssetMaterials.Num());
	for (int32 k = 0; k < AssetMaterials.Num(); ++k)
	{
		MaterialList.Add(AssetMaterials[k].MaterialInterface);
		MaterialSlotNames.Add(AssetMaterials[k].MaterialSlotName);
	}
}

void UGeometryScriptLibrary_StaticMeshFunctions::ConvertMaterialMapToMaterialList(const TMap<FName, UMaterialInterface*>& MaterialMap,
	TArray<UMaterialInterface*>& MaterialList,
	TArray<FName>& MaterialSlotNames)
{
	MaterialList.Reset(MaterialMap.Num());
	MaterialSlotNames.Reset(MaterialMap.Num());
	for (const TPair<FName, UMaterialInterface*> NameMat : MaterialMap)
	{
		MaterialList.Add(NameMat.Value);
		MaterialSlotNames.Add(NameMat.Key);
	}
}

TMap<FName, UMaterialInterface*> UGeometryScriptLibrary_StaticMeshFunctions::ConvertMaterialListToMaterialMap(const TArray<UMaterialInterface*>& MaterialList, const TArray<FName>& MaterialSlotNames)
{
	TMap<FName, UMaterialInterface*> ToRet;
	if (MaterialSlotNames.Num() != MaterialList.Num())
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMaterialListToMaterialMap: Number of Material Slot Names does not match number of Materials");
	}

	ToRet.Reserve(MaterialList.Num());
	for (int32 Idx = 0; Idx < MaterialList.Num(); ++Idx)
	{
		UMaterialInterface* Mat = MaterialList[Idx];
		// If we have fewer slot names than materials, we will have warned user via above AppendWarning, but make up a slot name so that we still have all materials in the map
		FName SlotName = MaterialSlotNames.IsValidIndex(Idx) ? MaterialSlotNames[Idx] : FName(FString::Printf(TEXT("%s_%d"), *((Mat) ? Mat->GetName() : TEXT("Material")), Idx));
		ToRet.Add(SlotName, Mat);
	}
	return ToRet;
}

void UGeometryScriptLibrary_StaticMeshFunctions::GetSectionMaterialListFromStaticMesh(
	UStaticMesh* FromStaticMeshAsset, 
	FGeometryScriptMeshReadLOD RequestedLOD,
	TArray<UMaterialInterface*>& MaterialList,
	TArray<int32>& MaterialIndex,
	TArray<FName>& MaterialSlotNames,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromStaticMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSectionMaterialListFromStaticMesh_InvalidInput1", "GetSectionMaterialListFromStaticMesh: FromStaticMeshAsset is Null"));
		return;
	}

	// RenderData mesh sections directly reference a Material Index, which is set as the MaterialID in CopyMeshFromStaticMesh_RenderData
	if (RequestedLOD.LODType == EGeometryScriptLODType::RenderData)
	{
		MaterialList.Reset();
		MaterialIndex.Reset();
		MaterialSlotNames.Reset();
		const TArray<FStaticMaterial>& AssetMaterials = FromStaticMeshAsset->GetStaticMaterials();
		for (int32 k = 0; k < AssetMaterials.Num(); ++k)
		{
			MaterialList.Add(AssetMaterials[k].MaterialInterface);
			MaterialIndex.Add(k);
			MaterialSlotNames.Add(AssetMaterials[k].MaterialSlotName);
		}

		Outcome = EGeometryScriptOutcomePins::Success;
		return;
	}

#if WITH_EDITOR

	if (RequestedLOD.LODType != EGeometryScriptLODType::MaxAvailable && RequestedLOD.LODType != EGeometryScriptLODType::SourceModel)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSectionMaterialListFromStaticMesh_LODNotAvailable", "GetSectionMaterialListFromStaticMesh: Requested LOD is not available"));
		return;
	}

	int32 UseLODIndex = FMath::Clamp(RequestedLOD.LODIndex, 0, FromStaticMeshAsset->GetNumSourceModels() - 1);

	MaterialList.Reset();
	MaterialIndex.Reset();
	MaterialSlotNames.Reset();
	if (UE::AssetUtils::GetStaticMeshLODMaterialListBySection(FromStaticMeshAsset, UseLODIndex, MaterialList, MaterialIndex, MaterialSlotNames) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSectionMaterialListFromStaticMesh_QueryFailed", "GetSectionMaterialListFromStaticMesh: Could not fetch Material Set from Asset"));
		return;
	}

	Outcome = EGeometryScriptOutcomePins::Success;

#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSectionMaterialListFromStaticMesh_EditorOnly", "GetSectionMaterialListFromStaticMesh: Source Models are not available at Runtime"));
#endif
}

void UGeometryScriptLibrary_StaticMeshFunctions::GetLODMaterialListFromSkeletalMesh(
	USkeletalMesh* FromSkeletalMeshAsset,
	FGeometryScriptMeshReadLOD RequestedLOD,
	TArray<UMaterialInterface*>& MaterialList,
	TArray<int32>& MaterialIndex,
	TArray<FName>& MaterialSlotNames,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetLODMaterialListFromSkeletalMesh_InvalidInput1", "GetLODMaterialListFromSkeletalMesh: FromSkeletalMeshAsset is Null"));
		return;
	}

#if WITH_EDITOR

	if (RequestedLOD.LODType == EGeometryScriptLODType::HiResSourceModel)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetLODMaterialListFromSkeletalMesh_LODNotAvailable", "GetLODMaterialListFromSkeletalMesh: Requested LOD is not available"));
		return;
	}

	int32 UseLODIndex = FMath::Clamp(RequestedLOD.LODIndex, 0, FromSkeletalMeshAsset->GetLODNum() - 1);

	const TArray<FSkeletalMaterial>& Mats = FromSkeletalMeshAsset->GetMaterials();
	const int32 NumMats = Mats.Num();

	// Get the material mapping via the LODInfo struct
	const FSkeletalMeshLODInfo* LODInfo = FromSkeletalMeshAsset->GetLODInfo(UseLODIndex);
	if (LODInfo && !LODInfo->LODMaterialMap.IsEmpty())
	{
		const TArray<int32>& Map = LODInfo->LODMaterialMap;
		const int32 NumSectionMat = Map.Num();
		MaterialList.Reset(NumSectionMat);
		MaterialIndex.Reset(NumSectionMat);
		MaterialSlotNames.Reset(NumSectionMat);
		for (int32 Idx = 0; Idx < NumSectionMat; ++Idx)
		{
			int32 MatIdx = Map[Idx];
			if (MatIdx == INDEX_NONE) // by convention, INDEX_NONE means the index is mapped to itself
			{
				MatIdx = FMath::Min(Idx, NumMats - 1);
			}
			MaterialIndex.Add(MatIdx);
			if (Mats.IsValidIndex(MatIdx))
			{
				MaterialList.Add(Mats[MatIdx].MaterialInterface);
				MaterialSlotNames.Add(Mats[MatIdx].MaterialSlotName);
			}
			else
			{
				MaterialList.Add(nullptr);
				MaterialSlotNames.Add(FName());
			}
		}
	}
	// if the LODMaterialMap is not there or is empty, materials are identity-mapped
	else
	{
		MaterialList.Reset(NumMats);
		MaterialIndex.Reset(NumMats);
		MaterialSlotNames.Reset(NumMats);
		for (int32 Idx = 0; Idx < NumMats; ++Idx)
		{
			MaterialIndex.Add(Idx);
			MaterialList.Add(Mats[Idx].MaterialInterface);
			MaterialSlotNames.Add(Mats[Idx].MaterialSlotName);
		}
	}

	Outcome = EGeometryScriptOutcomePins::Success;

#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetLODMaterialListFromSkeletalMesh_EditorOnly", "GetLODMaterialListFromSkeletalMesh: Not available at Runtime"));
#endif
}


namespace UELocal
{
	bool CopyMeshFromSkeletalMesh_RenderData(USkeletalMesh* FromSkeletalMeshAsset, FGeometryScriptCopyMeshFromAssetOptions AssetOptions, int32 LODIndex, UDynamicMesh* ToDynamicMesh, UGeometryScriptDebug* Debug)
	{
		

		if (FSkeletalMeshRenderData* RenderData = FromSkeletalMeshAsset->GetResourceForRendering())
		{
			int32 NumLODs = RenderData->LODRenderData.Num();
			if (NumLODs -1 < LODIndex )
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_RenderDataLDONotAvailable", "CopyMeshFromSkeletalMesh: Renderdata for specified LOD is not available"));
				return false;
			}

			FSkeletalMeshLODRenderData* SkeletalMeshLODRenderData = &(RenderData->LODRenderData[LODIndex]);

			UE::Geometry::FDynamicMesh3 NewMesh;
			
			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::ConversionOptions  ConversionOptions;
			ConversionOptions.bWantTangents = AssetOptions.bRequestTangents;

			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::Convert(SkeletalMeshLODRenderData, FromSkeletalMeshAsset->GetRefSkeleton(), ConversionOptions, NewMesh);
			ToDynamicMesh->SetMesh(MoveTemp(NewMesh));

			return true;
		}
	
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_RenderDataNotAvailable", "CopyMeshFromSkeletalMesh: Renderdata is not available"));
		return false;
		
	}
	
};

UDynamicMesh* UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromSkeletalMesh(
		USkeletalMesh* FromSkeletalMeshAsset, 
		UDynamicMesh* ToDynamicMesh,
		FGeometryScriptCopyMeshFromAssetOptions AssetOptions,
		FGeometryScriptMeshReadLOD RequestedLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_InvalidInput1", "CopyMeshFromSkeletalMesh: FromSkeletalMeshAsset is Null"));
		return ToDynamicMesh;
	}
	if (ToDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_InvalidInput2", "CopyMeshFromSkeletalMesh: ToDynamicMesh is Null"));
		return ToDynamicMesh;
	}

	// TODO: Consolidate this code with SkeletalMeshToolTarget::GetMeshDescription(..) 
	int32 UseLODIndex = RequestedLOD.LODIndex;
	EGeometryScriptLODType UseLODType = RequestedLOD.LODType;

#if WITH_EDITOR
	if (UseLODType == EGeometryScriptLODType::MaxAvailable || UseLODType == EGeometryScriptLODType::HiResSourceModel)
	{
		UseLODType = EGeometryScriptLODType::SourceModel;
	}

	if (UseLODType == EGeometryScriptLODType::SourceModel)
	{
		UseLODIndex = FMath::Clamp(RequestedLOD.LODIndex, 0, FromSkeletalMeshAsset->GetNumSourceModels() - 1);
		if (!FromSkeletalMeshAsset->GetSourceModel(UseLODIndex).HasMeshDescription())
		{
			UseLODType = EGeometryScriptLODType::RenderData;
		}
	}
#endif

	if (UseLODType == EGeometryScriptLODType::RenderData)
	{
		// TBD: Do we honor GetMinLodIdx?
		if (UELocal::CopyMeshFromSkeletalMesh_RenderData(FromSkeletalMeshAsset, AssetOptions, RequestedLOD.LODIndex, ToDynamicMesh, Debug))
		{
			Outcome = EGeometryScriptOutcomePins::Success;
		}
	}
	else
	{
#if WITH_EDITOR
		const FMeshDescription* SourceMesh = nullptr;

		// Check first if we have bulk data available and non-empty.
		if (FromSkeletalMeshAsset->HasMeshDescription(UseLODIndex))
		{
			SourceMesh = FromSkeletalMeshAsset->GetMeshDescription(UseLODIndex); 
		}
		if (SourceMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_LODNotAvailable", "CopyMeshFromSkeletalMesh: Requested LOD source mesh is not available"));
			return ToDynamicMesh;
		}

		FDynamicMesh3 NewMesh;
		FMeshDescriptionToDynamicMesh Converter;

		// Leave this on, since the set morph target node uses this. 
		Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
		
		Converter.Convert(SourceMesh, NewMesh, AssetOptions.bRequestTangents);

		// Notify of unused vertices
		if (NewMesh.HasUnusedVertices())
		{
			static const FText UnusedVerticesMsg = LOCTEXT("CopyMeshFromSkeletalMesh_UnusedVertices", "CopyMeshFromSkeletalMesh: ToDynamicMesh has unused vertices (not referenced by any triangle)");
			AppendWarning(Debug, EGeometryScriptErrorType::UnknownError, UnusedVerticesMsg);	
		}
	
		ToDynamicMesh->SetMesh(MoveTemp(NewMesh));
	
		Outcome = EGeometryScriptOutcomePins::Success;
#else
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromSkeletalMesh_SourceMesh_EditorOnly", "CopyMeshFromSkeletalMesh: Source Meshes are not available at Runtime"));
#endif
	}		
	
	return ToDynamicMesh;
}

#if WITH_EDITOR
namespace UELocal 
{

// this is identical to UE::AssetUtils::GenerateNewMaterialSlotName except it takes a TArray<FSkeletalMaterial>
// instead of a TArray<FStaticMaterial>. It seems likely that we will need SkeletalMeshMaterialUtil.h soon,
// at that point this function can be moved there
static FName GenerateNewMaterialSlotName(
	const TArray<FSkeletalMaterial>& ExistingMaterials,
	UMaterialInterface* SlotMaterial,
	int32 NewSlotIndex)
{
	FString MaterialName = (SlotMaterial) ? SlotMaterial->GetName() : TEXT("Material");
	FName BaseName(MaterialName);

	bool bFound = false;
	for (const FSkeletalMaterial& Mat : ExistingMaterials)
	{
		if (Mat.MaterialSlotName == BaseName || Mat.ImportedMaterialSlotName == BaseName)
		{
			bFound = true;
			break;
		}
	}
	if (bFound == false && SlotMaterial != nullptr)
	{
		return BaseName;
	}

	bFound = true;
	while (bFound)
	{
		bFound = false;

		BaseName = FName(FString::Printf(TEXT("%s_%d"), *MaterialName, NewSlotIndex++));
		for (const FSkeletalMaterial& Mat : ExistingMaterials)
		{
			if (Mat.MaterialSlotName == BaseName || Mat.ImportedMaterialSlotName == BaseName)
			{
				bFound = true;
				break;
			}
		}
	}

	return BaseName;
}
}
#endif


UDynamicMesh* UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToSkeletalMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FGeometryScriptCopyMeshToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToSkeletalMesh_InvalidInput1", "CopyMeshToSkeletalMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}
	if (ToSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToSkeletalMesh_InvalidInput2", "CopyMeshToSkeletalMesh: ToSkeletalMeshAsset is Null"));
		return FromDynamicMesh;
	}
	if (TargetLOD.bWriteHiResSource)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToSkeletalMesh_Unsupported", "CopyMeshToSkeletalMesh: Writing HiResSource LOD is not yet supported"));
		return FromDynamicMesh;
	}

	// TODO: Consolidate this code with SkeletalMeshToolTarget::CommitMeshDescription
#if WITH_EDITOR
	if (ToSkeletalMeshAsset->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		const FText Error = FText::Format(LOCTEXT("CopyMeshToSkeletalMesh_BuiltInAsset", "CopyMeshToSkeletalMesh: Cannot modify built-in engine asset: {0}"), FText::FromString(*ToSkeletalMeshAsset->GetPathName()));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		return FromDynamicMesh;
	}

	// flush any pending rendering commands, which might touch a component while we are rebuilding it's mesh
	FlushRenderingCommands();

	if (Options.bEmitTransaction)
	{
		GEditor->BeginTransaction(LOCTEXT("UpdateSkeletalMesh", "Update Skeletal Mesh"));
	}

	// If this option is set, override the bone hierarchy mismatch settings and warn the user that we're doing so.
	if (Options.bRemapBoneIndicesToMatchAsset)
	{
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs,
			LOCTEXT("RemapBoneIndicesToMatchAssetDeprecated", "The bRemapBoneIndicesToMatchAsset option is deprecated. Use BoneHierarchyMismatchHandling instead."));
		Options.BoneHierarchyMismatchHandling = EGeometryScriptBoneHierarchyMismatchHandling::RemapGeometryToReferenceSkeleton;
	}

	// make sure transactional flag is on for this asset
	ToSkeletalMeshAsset->SetFlags(RF_Transactional);

	ToSkeletalMeshAsset->Modify();

	if (Options.bDeferMeshPostEditChange == false)
	{
		ToSkeletalMeshAsset->PreEditChange(nullptr);
	}
	
	// Ensure we have enough LODInfos to cover up to the requested LOD.
	for (int32 LODIndex = ToSkeletalMeshAsset->GetLODNum(); LODIndex <= TargetLOD.LODIndex; LODIndex++)
	{
		FSkeletalMeshLODInfo& LODInfo = ToSkeletalMeshAsset->AddLODInfo();
		
		ToSkeletalMeshAsset->GetImportedModel()->LODModels.Add(new FSkeletalMeshLODModel);
		LODInfo.ReductionSettings.BaseLOD = 0;
	}

	FMeshDescription* MeshDescription = ToSkeletalMeshAsset->CreateMeshDescription(TargetLOD.LODIndex);

	if (MeshDescription == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToSkeletalMesh_TargetMeshDescription", "CopyMeshToSkeletalMesh: Failed to generate the mesh data for the Target LOD Index"));
		return FromDynamicMesh;
	}

	// Verify that the bones on the dynamic mesh are a proper subset of the bones on the skeletal mesh. The order is not important, since
	// we re-order as needed below. If the mesh has no bones, then we create, or get, the default skin weight attribute and bind everything 
	// to root, since we can't verify that any current skin binding is valid.
	TArray<int32> BoneRemapping;
	const FDynamicMesh3& Mesh{FromDynamicMesh->GetMeshRef()}; 
	if (Options.BoneHierarchyMismatchHandling == EGeometryScriptBoneHierarchyMismatchHandling::RemapGeometryToReferenceSkeleton && Mesh.HasAttributes() && Mesh.Attributes()->HasBones())
	{
		const FDynamicMeshBoneNameAttribute* SrcBoneNames = Mesh.Attributes()->GetBoneNames();
		TArray<FName> DstBoneNames = ToSkeletalMeshAsset->GetRefSkeleton().GetRawRefBoneNames();
		for (int32 SrcBoneIndex = 0; SrcBoneIndex < SrcBoneNames->Num(); ++SrcBoneIndex)
		{
			const FName SrcBoneName = SrcBoneNames->GetValue(SrcBoneIndex);
			const int32 DstBoneIndex = DstBoneNames.IndexOfByKey(SrcBoneName);
			if (DstBoneIndex == INDEX_NONE)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs,
					FText::Format(LOCTEXT("CopyMeshToSkeletalMesh_MissingBonesOnAsset", "CopyMeshToSkeletalMesh: Source geometry contains bone '{0}' which does not exist on the skeletal mesh asset ({1})."),
						FText::FromName(SrcBoneName), FText::FromString(ToSkeletalMeshAsset->GetPackage()->GetPathName())));
				return FromDynamicMesh;
			}

			BoneRemapping.Add(DstBoneIndex);
		}
	}
	
	FSkeletalMeshAttributes MeshAttributes(*MeshDescription);
	MeshAttributes.Register();

	ToSkeletalMeshAsset->ModifyMeshDescription(TargetLOD.LODIndex);
	
	FConversionToMeshDescriptionOptions ConversionOptions;
	ConversionOptions.bConvertBackToNonManifold = Options.bUseOriginalVertexOrder;

	TUniquePtr<TArray<int32>> MatIDInverseMap;

	// Update asset materials / set a corresponding material remap on the converter if needed
	if (Options.bReplaceMaterials)
	{
		bool bHaveSlotNames = (Options.NewMaterialSlotNames.Num() == Options.NewMaterials.Num());

		auto MakeNewMaterial = [&Options, bHaveSlotNames](TArray<FSkeletalMaterial>& SkelMaterials, int32 NewMatIdx) -> FSkeletalMaterial
		{
			FSkeletalMaterial NewMaterial;
			NewMaterial.MaterialInterface = Options.NewMaterials[NewMatIdx];
			FName UseSlotName = (bHaveSlotNames && Options.NewMaterialSlotNames[NewMatIdx] != NAME_None) ? Options.NewMaterialSlotNames[NewMatIdx] :
				UELocal::GenerateNewMaterialSlotName(SkelMaterials, NewMaterial.MaterialInterface, NewMatIdx);

			NewMaterial.MaterialSlotName = UseSlotName;
			NewMaterial.ImportedMaterialSlotName = UseSlotName;
			NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);		// this avoids an ensure in  UStaticMesh::GetUVChannelData
			return NewMaterial;
		};

		if (Options.NewMaterialUsage == ECopyMeshToAssetUseNewMaterialsOptions::ReplaceAssetMaterials)
		{
			TArray<FSkeletalMaterial> NewMaterials;
			for (int32 k = 0; k < Options.NewMaterials.Num(); ++k)
			{
				NewMaterials.Add(MakeNewMaterial(NewMaterials, k));
			}

			ToSkeletalMeshAsset->SetMaterials(NewMaterials);
		}
		else
		{
			TArray<FSkeletalMaterial> ToUpdateMaterials = ToSkeletalMeshAsset->GetMaterials();
			TMap<UMaterialInterface*, int32> MatToIndex;
			for (int32 OldMatIdx = 0; OldMatIdx < ToUpdateMaterials.Num(); ++OldMatIdx)
			{
				MatToIndex.Add(ToUpdateMaterials[OldMatIdx].MaterialInterface, OldMatIdx);
			}
			bool bUpdated = false;
			for (int32 NewMatIdx = 0; NewMatIdx < Options.NewMaterials.Num(); ++NewMatIdx)
			{
				UMaterialInterface* NewMat = Options.NewMaterials[NewMatIdx];
				if (!MatToIndex.Contains(NewMat))
				{
					bUpdated = true;
					int32 NewIdx = ToUpdateMaterials.Add(MakeNewMaterial(ToUpdateMaterials, NewMatIdx));
					MatToIndex.Add(NewMat, NewIdx);
				}
			}

			// build material remap s.t. dynamic mesh material indices map to where materials were set in the skeletal mesh asset
			MatIDInverseMap = MakeUnique<TArray<int32>>();
			MatIDInverseMap->Init(-1, ToUpdateMaterials.Num());
			for (int32 NewMatIdx = 0; NewMatIdx < Options.NewMaterials.Num(); ++NewMatIdx)
			{
				(*MatIDInverseMap)[MatToIndex[Options.NewMaterials[NewMatIdx]]] = NewMatIdx;
			}

			if (bUpdated)
			{
				ToSkeletalMeshAsset->SetMaterials(ToUpdateMaterials);
			}
		}
	}

	
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	if (MatIDInverseMap)
	{
		Converter.SetMaterialIDMapFromInverseMap(*MatIDInverseMap);
	}
	if (Options.bCleanAssignedMaterials)
	{
		if (Options.bReplaceMaterials)
		{
			if (Options.NewMaterialUsage == ECopyMeshToAssetUseNewMaterialsOptions::ReplaceAssetMaterials)
			{
				ConversionOptions.MaxValidMaterialID = Options.NewMaterials.Num() - 1;
			}
			else if (ensure(MatIDInverseMap))
			{
				ConversionOptions.MaxValidMaterialID = MatIDInverseMap->Num() - 1;
			}
		}
		else
		{
			FSkeletalMeshLODInfo* SkeletalLODInfo = ToSkeletalMeshAsset->GetLODInfo(TargetLOD.LODIndex);
			int32 LODNumMats = SkeletalLODInfo->LODMaterialMap.IsEmpty() ? ToSkeletalMeshAsset->GetNumMaterials() : SkeletalLODInfo->LODMaterialMap.Num();
			ConversionOptions.MaxValidMaterialID = LODNumMats - 1;
		}
	}

	FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		Converter.Convert(&ReadMesh, *MeshDescription, !Options.bEnableRecomputeTangents);
	});

	bool bForceRootBinding = false;
	
	switch (Options.BoneHierarchyMismatchHandling)
	{
	case EGeometryScriptBoneHierarchyMismatchHandling::DoNothing:
		break;
		
	case EGeometryScriptBoneHierarchyMismatchHandling::RemapGeometryToReferenceSkeleton:
		if (!BoneRemapping.IsEmpty())
		{
			FSkeletalMeshOperations::RemapBoneIndicesOnSkinWeightAttribute(*MeshDescription, BoneRemapping);
		}
		else
		{
			bForceRootBinding = true;
		}
		break;
	case EGeometryScriptBoneHierarchyMismatchHandling::CreateNewReferenceSkeleton:
		// We only grab ref skeleton from LOD 0. As of now, it is left to the user to ensure that bone hierarchies
		// for lower LODs are a strict subset of the ref skeleton (either being created). This restriction may be
		// lifted in the future and automatic fixing/rejection performed. 
		if (TargetLOD.LODIndex == 0)
		{
			static const FName RootBoneName("Root");
			FReferenceSkeleton RefSkeleton;

			{
				// Scoped here so that the modifier's destructor can complete the construction of the ref skeleton.
				FReferenceSkeletonModifier Modifier(RefSkeleton, nullptr);
				
				if (MeshAttributes.GetNumBones())
				{
					// For now, we assume the bone hierarchy is consistent and can construct a well-formed ref skeleton.
					FSkeletalMeshAttributesShared::FBoneNameAttributesConstRef BoneNames(MeshAttributes.GetBoneNames());
					FSkeletalMeshAttributesShared::FBoneParentIndexAttributesConstRef BoneParents(MeshAttributes.GetBoneParentIndices());
					FSkeletalMeshAttributesShared::FBonePoseAttributesConstRef BonePoses(MeshAttributes.GetBonePoses());

					for (int32 BoneIndex = 0; BoneIndex < MeshAttributes.GetNumBones(); BoneIndex++)
					{
						Modifier.Add(FMeshBoneInfo(BoneNames.Get(BoneIndex), BoneNames.Get(BoneIndex).ToString(), BoneParents.Get(BoneIndex)),
							BonePoses.Get(BoneIndex));
					}
				}
				else
				{
					Modifier.Add(FMeshBoneInfo(RootBoneName, RootBoneName.ToString(), INDEX_NONE), FTransform::Identity);
					bForceRootBinding = true;
				}
			}

			ToSkeletalMeshAsset->SetRefSkeleton(RefSkeleton);
			ToSkeletalMeshAsset->CalculateInvRefMatrices();
		}
		break;
	}

	if (bForceRootBinding)
	{
		MeshAttributes.BoneAttributes();
		
		using namespace UE::AnimationCore;
		FBoneWeight RootWeight(0, 1.0f);
		FBoneWeights RootBinding = FBoneWeights::Create({RootWeight});
		for (const FName AttributeName: MeshAttributes.GetSkinWeightProfileNames())
		{
			FSkinWeightsVertexAttributesRef SkinWeights(MeshAttributes.GetVertexSkinWeights(AttributeName));

			for (FVertexID VertexID: MeshDescription->Vertices().GetElementIDs())
			{
				SkinWeights.Set(VertexID, RootBinding);
			}
		}
	}
	
	FSkeletalMeshLODInfo* SkeletalLODInfo = ToSkeletalMeshAsset->GetLODInfo(TargetLOD.LODIndex);
	SkeletalLODInfo->BuildSettings.bRecomputeNormals = Options.bEnableRecomputeNormals;
	SkeletalLODInfo->BuildSettings.bRecomputeTangents = Options.bEnableRecomputeTangents;
	
	// Prevent decimation of this LOD.
	SkeletalLODInfo->ReductionSettings.NumOfTrianglesPercentage = 1.0;
	SkeletalLODInfo->ReductionSettings.NumOfVertPercentage = 1.0;
	SkeletalLODInfo->ReductionSettings.MaxNumOfTriangles = MAX_int32;
	SkeletalLODInfo->ReductionSettings.MaxNumOfVerts = MAX_int32; 
	SkeletalLODInfo->ReductionSettings.BaseLOD = TargetLOD.LODIndex;

	if (Options.bApplyNaniteSettings)
	{
		ToSkeletalMeshAsset->NaniteSettings = Options.NewNaniteSettings;
	}

	// Final pass of material updating
	if (Options.bReplaceMaterials)
	{
		// reset LODMaterialMap; it will be set up later with just the materials actually used
		SkeletalLODInfo->LODMaterialMap.Reset();
		
		// Set material slot names on the mesh description
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
		const TArray<FSkeletalMaterial>& UpdatedMaterials = ToSkeletalMeshAsset->GetMaterials();
		for (int32 SlotIdx = 0; SlotIdx < UpdatedMaterials.Num(); ++SlotIdx)
		{
			if (SlotIdx < PolygonGroupImportedMaterialSlotNames.GetNumElements())
			{
				PolygonGroupImportedMaterialSlotNames.Set(SlotIdx, UpdatedMaterials[SlotIdx].ImportedMaterialSlotName);
			}
		}
	}

	ToSkeletalMeshAsset->CommitMeshDescription(TargetLOD.LODIndex);

	bool bHasVertexColors = false;
	TVertexInstanceAttributesConstRef<FVector4f> VertexColors = MeshAttributes.GetVertexInstanceColors();
	for (const FVertexInstanceID VertexInstanceID: MeshDescription->VertexInstances().GetElementIDs())
	{
		if (!VertexColors.Get(VertexInstanceID).Equals(FVector4f::One()))
		{
			bHasVertexColors = true;
			break;
		}
	}
		
	// configure vertex color setup in the Asset
	ToSkeletalMeshAsset->SetHasVertexColors(bHasVertexColors);
#if WITH_EDITORONLY_DATA
	ToSkeletalMeshAsset->SetVertexColorGuid(bHasVertexColors ? FGuid::NewGuid() : FGuid() );
#endif
	
	if (Options.bDeferMeshPostEditChange == false)
	{
		ToSkeletalMeshAsset->PostEditChange();
	}

	if (Options.bEmitTransaction)
	{
		GEditor->EndTransaction();
	}
	
	Outcome = EGeometryScriptOutcomePins::Success;
#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToSkeletalMesh_EditorOnly", "CopyMeshToSkeletalMesh: Not currently supported at Runtime"));
#endif
	
	return FromDynamicMesh;
}


UDynamicMesh* UGeometryScriptLibrary_StaticMeshFunctions::CopyMorphTargetToSkeletalMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FName MorphTargetName,
		FGeometryScriptCopyMorphTargetToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (ToSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_InvalidInput1", "CopyMorphTargetToSkeletalMesh: ToSkeletalMeshAsset is Null"));
		return FromDynamicMesh;
	}
	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_InvalidInput2", "CopyMorphTargetToSkeletalMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}

#if WITH_EDITOR
	if (ToSkeletalMeshAsset->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		const FText Error = FText::Format(LOCTEXT("CopyMorphTargetToSkeletalMesh_BuiltInAsset", "CopyMorphTargetToSkeletalMesh: Cannot modify built-in engine asset: {0}"), FText::FromString(*ToSkeletalMeshAsset->GetPathName()));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		return FromDynamicMesh;
	}

	// flush any pending rendering commands, which might touch a component while we are rebuilding it's mesh
	FlushRenderingCommands();

	TUniquePtr<FScopedTransaction> Transaction;
	if (Options.bEmitTransaction)
	{
		Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("UpdateSkeletalMesh", "Update Skeletal Mesh"));
	}
	
	// make sure transactional flag is on for this asset
	ToSkeletalMeshAsset->SetFlags(RF_Transactional);

	ToSkeletalMeshAsset->Modify();
	
	// Ensure we have enough LODInfos to cover up to the requested LOD.
	for (int32 LODIndex = ToSkeletalMeshAsset->GetLODNum(); LODIndex <= TargetLOD.LODIndex; LODIndex++)
	{
		FSkeletalMeshLODInfo& LODInfo = ToSkeletalMeshAsset->AddLODInfo();
		
		ToSkeletalMeshAsset->GetImportedModel()->LODModels.Add(new FSkeletalMeshLODModel);
		LODInfo.ReductionSettings.BaseLOD = 0;
	}

	FMeshDescription* MeshDescription = ToSkeletalMeshAsset->GetMeshDescription(TargetLOD.LODIndex);

	if (MeshDescription == nullptr)
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_TargetMeshDescription", "CopyMorphTargetToSkeletalMesh: Failed to generate the mesh data for the Target LOD Index"));
		return FromDynamicMesh;
	}

	// If the dynamic mesh has non-manifold information, use that to figure out what the original vertex count was.
	// Otherwise, we assume that they have a 1:1 match.
	const FDynamicMesh3& SourceMesh = FromDynamicMesh->GetMeshRef();
	const FNonManifoldMappingSupport NonManifoldMappingSupport(SourceMesh);
	int32 SourceVertexCount;

	if (NonManifoldMappingSupport.IsNonManifoldVertexInSource())
	{
		TSet<int32> UniqueVertices;
		for (int32 SourceVID = 0; SourceVID < SourceMesh.VertexCount(); ++SourceVID)
		{
			UniqueVertices.Add(NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(SourceVID));
		}

		SourceVertexCount = UniqueVertices.Num();
	}
	else
	{
		SourceVertexCount = SourceMesh.VertexCount();
	}
	if (MeshDescription->Vertices().Num() != SourceVertexCount)
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_InvalidMorphTargetGeometry", "CopyMorphTargetToSkeletalMesh: Morph target mesh doesnt have the same number of vertices as the skeletal mesh."));
		return FromDynamicMesh;
	}

	const FDynamicMeshNormalOverlay* SourceNormals = SourceMesh.Attributes()->PrimaryNormals();
	FMeshNormals Normals;
	if (Options.bCopyNormals && !SourceNormals)
	{
		Normals = FMeshNormals(&SourceMesh);
		Normals.ComputeVertexNormals();
	}
	
	FSkeletalMeshAttributes MeshAttributes(*MeshDescription);
	constexpr bool bKeepExistingAttributes = true;
	MeshAttributes.Register(bKeepExistingAttributes);

	ToSkeletalMeshAsset->ModifyMeshDescription(TargetLOD.LODIndex);

	if (MeshAttributes.GetMorphTargetNames().Contains(MorphTargetName))
	{
		if (!Options.bOverwriteExistingTarget) // only throw error if we dont want to overwrite the existing target 
		{
			if (Transaction)
			{
				Transaction->Cancel();
			}
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_InvalidMorphTargetName1", "CopyMorphTargetToSkeletalMesh: Morph target name already exists"));
			return FromDynamicMesh;
		}

		// Remove existing attribute so that we start with a clean slate.
		MeshAttributes.UnregisterMorphTargetAttribute(MorphTargetName);
	}

	// call RegisterMorphTargetAttribute to make sure normals are registered / unregistered when needed
	if (!MeshAttributes.RegisterMorphTargetAttribute(MorphTargetName, Options.bCopyNormals))
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_InvalidMorphTargetName2", "CopyMorphTargetToSkeletalMesh: Morph target name is invalid."));
		return FromDynamicMesh;
	}

	const FSkeletalMeshLODInfo* LODInfo = ToSkeletalMeshAsset->GetLODInfo(TargetLOD.LODIndex);
	const float MorphThresholdSquared = LODInfo->BuildSettings.MorphThresholdPosition * LODInfo->BuildSettings.MorphThresholdPosition;
	
	TVertexAttributesRef<FVector3f> MorphPositionDelta = MeshAttributes.GetVertexMorphPositionDelta(MorphTargetName);
	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();

	TVertexInstanceAttributesRef<FVector3f> MorphNormalDelta = MeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName);
	TVertexInstanceAttributesConstRef<FVector3f> VertexNormals = MeshAttributes.GetVertexInstanceNormals();
	
	bool bMorphTargetIsEmpty = true;
	
	TArray<int32> ElementIndexes;
	for (int32 SourceVID = 0; SourceVID < SourceMesh.VertexCount(); ++SourceVID)
	{
		const int32 TargetVID = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(SourceVID);
		
		const FVector3d V0 = SourceMesh.GetVertex(SourceVID);
		const FVector3f V1 = VertexPositions[TargetVID];

		const FVector3f Delta = FVector3f{V0} - V1;
		if (Delta.SquaredLength() > MorphThresholdSquared)
		{
			bMorphTargetIsEmpty = false;
			MorphPositionDelta.Set(TargetVID, Delta);

			if (Options.bCopyNormals)
			{
				FVector3f N0;
				if (SourceNormals)
				{
					// For now, we average the normals. In the future, we should detect discontinuous normals and transfer them
					// exactly to the target mesh, using vertex matching for the triangles to figure out which vertex instance goes
					// where.
					N0 = FVector3f::ZeroVector;
					ElementIndexes.Reset();
					SourceNormals->GetVertexElements(SourceVID, ElementIndexes);
					for (int32 ElementIdx: ElementIndexes)
					{
						N0 += SourceNormals->GetElement(ElementIdx);
					}
					N0.Normalize();
				}
				else
				{
					N0 = FVector3f{Normals[SourceVID]};
				}
				for (const FVertexInstanceID VertexInstanceID : MeshDescription->GetVertexVertexInstanceIDs(TargetVID))
				{
					const FVector3f N1 = VertexNormals.Get(VertexInstanceID);
					const FVector3f NDelta(N0-N1);
					MorphNormalDelta.Set(VertexInstanceID, NDelta);
				}
			}
		}
	}

	if (bMorphTargetIsEmpty)
	{
		MeshAttributes.UnregisterMorphTargetAttribute(MorphTargetName);
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("CopyMorphTargetToSkeletalMesh_EmptyMorphTarget", "CopyMorphTargetToSkeletalMesh: Morph target is empty since it does not differ from the base mesh's vertex position."));
		return FromDynamicMesh;
	}

#if WITH_EDITORONLY_DATA
	if (FSkeletalMeshLODInfo* LodInfo = ToSkeletalMeshAsset->GetLODInfo(TargetLOD.LODIndex))
	{
		constexpr bool bGeneratedByEngineTrue = true;
		LodInfo->ImportedMorphTargetSourceFilename.FindOrAdd(MorphTargetName.ToString()).SetGeneratedByEngine(bGeneratedByEngineTrue);
	}
#endif //WITH_EDITORONLY_DATA

	ToSkeletalMeshAsset->CommitMeshDescription(TargetLOD.LODIndex);

	if (Options.bDeferMeshPostEditChange == false)
	{
		ToSkeletalMeshAsset->PostEditChange();
	}

	Outcome = EGeometryScriptOutcomePins::Success;
#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMorphTargetToSkeletalMesh_EditorOnly", "CopyMorphTargetToSkeletalMesh: Not currently supported at Runtime"));
#endif
	
	return FromDynamicMesh;
}


UDynamicMesh* UGeometryScriptLibrary_StaticMeshFunctions::CopySkinWeightProfileToSkeletalMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FName TargetProfileName,
		FName SourceProfileName,
		FGeometryScriptCopySkinWeightProfileToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (ToSkeletalMeshAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_InvalidInput1", "CopySkinWeightProfileToSkeletalMesh: ToSkeletalMeshAsset is Null"));
		return FromDynamicMesh;
	}
	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_InvalidInput2", "CopySkinWeightProfileToSkeletalMesh: FromDynamicMesh is Null"));
		return FromDynamicMesh;
	}

#if WITH_EDITOR
	if (ToSkeletalMeshAsset->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		const FText Error = FText::Format(LOCTEXT("CopySkinWeightProfileToSkeletalMesh_BuiltInAsset", "CopySkinWeightProfileToSkeletalMesh: Cannot modify built-in engine asset: {0}"), FText::FromString(*ToSkeletalMeshAsset->GetPathName()));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		return FromDynamicMesh;
	}

	TUniquePtr<FScopedTransaction> Transaction;
	if (Options.bEmitTransaction)
	{
		Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("UpdateSkeletalMesh", "Update Skeletal Mesh"));
	}
	
	// flush any pending rendering commands, which might touch a component while we are rebuilding it's mesh
	FlushRenderingCommands();

	// make sure transactional flag is on for this asset
	ToSkeletalMeshAsset->SetFlags(RF_Transactional);

	ToSkeletalMeshAsset->Modify();

	// Ensure we have enough LODInfos to cover up to the requested LOD.
	for (int32 LODIndex = ToSkeletalMeshAsset->GetLODNum(); LODIndex <= TargetLOD.LODIndex; LODIndex++)
	{
		FSkeletalMeshLODInfo& LODInfo = ToSkeletalMeshAsset->AddLODInfo();
		
		ToSkeletalMeshAsset->GetImportedModel()->LODModels.Add(new FSkeletalMeshLODModel);
		LODInfo.ReductionSettings.BaseLOD = 0;
	}

	FMeshDescription* MeshDescription = ToSkeletalMeshAsset->GetMeshDescription(TargetLOD.LODIndex);
	if (MeshDescription == nullptr)
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_TargetMeshDescription", "CopySkinWeightProfileToSkeletalMesh: Failed to generate the mesh data for the Target LOD Index"));
		return FromDynamicMesh;
	}

	if (TargetProfileName.IsNone())
	{
		TargetProfileName = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
	}
	if (SourceProfileName.IsNone())
	{
		SourceProfileName = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
	}
	
	const FDynamicMesh3& SourceMesh = FromDynamicMesh->GetMeshRef();
	if (!SourceMesh.HasAttributes() || !SourceMesh.Attributes()->HasSkinWeightsAttribute(SourceProfileName))
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_InvalidSourceProfile", "CopySkinWeightProfileToSkeletalMesh: The requested skin weight profile does not exist on the source mesh."));
		return FromDynamicMesh;
	}
	
	const FNonManifoldMappingSupport NonManifoldMappingSupport(SourceMesh);
	int32 SourceVertexCount;

	// If the dynamic mesh has non-manifold information, use that to figure out what the original vertex count was.
	// Otherwise, we assume that they have a 1:1 match.
	if (NonManifoldMappingSupport.IsNonManifoldVertexInSource())
	{
		TSet<int32> UniqueVertices;
		for (int32 SourceVID = 0; SourceVID < SourceMesh.VertexCount(); ++SourceVID)
		{
			UniqueVertices.Add(NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(SourceVID));
		}

		SourceVertexCount = UniqueVertices.Num();
	}
	else
	{
		SourceVertexCount = SourceMesh.VertexCount();
	}
	if (MeshDescription->Vertices().Num() != SourceVertexCount)
	{
		if (Transaction)
		{
			Transaction->Cancel();
		}
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_InvalidAssetGeometry", "CopySkinWeightProfileToSkeletalMesh: Target skeletal mesh doesnt have the same number of vertices as the source mesh."));
		return FromDynamicMesh;
	}
	
	FSkeletalMeshAttributes MeshAttributes(*MeshDescription);
	constexpr bool bKeepExistingAttributes = true;
	MeshAttributes.Register(bKeepExistingAttributes);

	ToSkeletalMeshAsset->ModifyMeshDescription(TargetLOD.LODIndex);

	if (TargetProfileName != FSkeletalMeshAttributes::DefaultSkinWeightProfileName)
	{
		if (!Options.bOverwriteExistingProfile && MeshAttributes.GetSkinWeightProfileNames().Contains(TargetProfileName))
		{
			if (Transaction)
			{
				Transaction->Cancel();
			}
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_CantOverrideProfile", "CopySkinWeightProfileToSkeletalMesh: Skin profile name already exists on the target mesh."));
			return FromDynamicMesh;
		}
		
		if (!MeshAttributes.RegisterSkinWeightAttribute(TargetProfileName))
		{
			if (Transaction)
			{
				Transaction->Cancel();
			}
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_InvalidProfileName", "CopySkinWeightProfileToSkeletalMesh: Cannot create target skin weight profile with the given profile name."));
			return FromDynamicMesh;
		}
	}

	const FDynamicMeshVertexSkinWeightsAttribute* SourceProfileAttribute = SourceMesh.Attributes()->GetSkinWeightsAttribute(SourceProfileName);
	FSkinWeightsVertexAttributesRef TargetProfileAttribute = MeshAttributes.GetVertexSkinWeights(TargetProfileName);

	for (int32 SourceVID = 0; SourceVID < SourceMesh.VertexCount(); ++SourceVID)
	{
		const int32 TargetVID = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(SourceVID);

		UE::AnimationCore::FBoneWeights BoneWeights;
		SourceProfileAttribute->GetValue(SourceVID, BoneWeights);
		TargetProfileAttribute.Set(TargetVID, BoneWeights);
	}

	ToSkeletalMeshAsset->CommitMeshDescription(TargetLOD.LODIndex);
	ToSkeletalMeshAsset->InvalidateDeriveDataCacheGUID();

	if (Options.bDeferMeshPostEditChange == false)
	{
		ToSkeletalMeshAsset->PostEditChange();
	}

	Outcome = EGeometryScriptOutcomePins::Success;
#else
	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopySkinWeightProfileToSkeletalMesh_EditorOnly", "CopySkinWeightProfileToSkeletalMesh: Not currently supported at Runtime"));
#endif
	
	return FromDynamicMesh;
}

#undef LOCTEXT_NAMESPACE
