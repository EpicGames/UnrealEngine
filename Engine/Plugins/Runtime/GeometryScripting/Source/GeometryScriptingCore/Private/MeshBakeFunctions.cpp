// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshBakeFunctions.h"
#include "UDynamicMesh.h"
#include "GameFramework/Actor.h"

#include "ModelingToolTargetUtil.h" // UE::ToolTarget functions
#include "Scene/SceneCapturePhotoSet.h"
#include "Baking/RenderCaptureFunctions.h"

#include "Sampling/MeshBakerCommon.h"
#include "Sampling/MeshMapBaker.h"
#include "Sampling/MeshVertexBaker.h"
#include "Sampling/MeshCurvatureMapEvaluator.h"
#include "Sampling/MeshOcclusionMapEvaluator.h"
#include "Sampling/MeshNormalMapEvaluator.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "Sampling/MeshUVShellMapEvaluator.h"
#include "Sampling/MeshResampleImageEvaluator.h"
#include "Sampling/MeshHeightMapEvaluator.h"

#include "DynamicMesh/MeshTransforms.h"

#include "AssetUtils/Texture2DBuilder.h"
#include "AssetUtils/Texture2DUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBakeFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshBakeFunctions"

namespace GeometryScriptBakeLocals
{
	FImageDimensions GetDimensions(const EGeometryScriptBakeResolution Resolution)
	{
		int Dimension = 256;
		switch(Resolution)
		{
		case EGeometryScriptBakeResolution::Resolution16:
			Dimension = 16;
			break;
		case EGeometryScriptBakeResolution::Resolution32:
			Dimension = 32;
			break;
		case EGeometryScriptBakeResolution::Resolution64:
			Dimension = 64;
			break;
		case EGeometryScriptBakeResolution::Resolution128:
			Dimension = 128;
			break;
		case EGeometryScriptBakeResolution::Resolution256:
			Dimension = 256;
			break;
		case EGeometryScriptBakeResolution::Resolution512:
			Dimension = 512;
			break;
		case EGeometryScriptBakeResolution::Resolution1024:
			Dimension = 1024;
			break;
		case EGeometryScriptBakeResolution::Resolution2048:
			Dimension = 2048;
			break;
		case EGeometryScriptBakeResolution::Resolution4096:
			Dimension = 4096;
			break;
		case EGeometryScriptBakeResolution::Resolution8192:
			Dimension = 8192;
			break;
		}
		return FImageDimensions(Dimension, Dimension);
	}

	int GetSamplesPerPixel(EGeometryScriptBakeSamplesPerPixel SamplesPerPixel)
	{
		int Samples = 1;
		switch(SamplesPerPixel)
		{
		case EGeometryScriptBakeSamplesPerPixel::Sample1:
			Samples = 1;
			break;
		case EGeometryScriptBakeSamplesPerPixel::Sample4:
			Samples = 4;
			break;
		case EGeometryScriptBakeSamplesPerPixel::Sample16:
			Samples = 16;
			break;
		case EGeometryScriptBakeSamplesPerPixel::Sample64:
			Samples = 64;
			break;
		case EGeometryScriptBakeSamplesPerPixel::Samples256:
			Samples = 256;
			break;
		}
		return Samples;
	}

	FMeshMapBaker::EBakeFilterType GetFilter(EGeometryScriptBakeFilteringType FilteringType)
	{
		switch (FilteringType)
		{
		case EGeometryScriptBakeFilteringType::Box:
			return FMeshMapBaker::EBakeFilterType::Box;
		case EGeometryScriptBakeFilteringType::BSpline:
		default:
			return FMeshMapBaker::EBakeFilterType::BSpline;
		}
	}


	FTexture2DBuilder::ETextureType GetTextureType(const FMeshMapEvaluator* Evaluator, const EGeometryScriptBakeBitDepth MapFormat)
	{
		FTexture2DBuilder::ETextureType TexType = FTexture2DBuilder::ETextureType::Color;
		switch (Evaluator->Type())
		{
		default:
			checkNoEntry();
			break;
		case EMeshMapEvaluatorType::Normal:
		{
			TexType = FTexture2DBuilder::ETextureType::NormalMap;
			break;
		}
		case EMeshMapEvaluatorType::Occlusion:
		{
			const FMeshOcclusionMapEvaluator* OcclusionEval = static_cast<const FMeshOcclusionMapEvaluator*>(Evaluator); 
			if (static_cast<bool>(OcclusionEval->OcclusionType & EMeshOcclusionMapType::AmbientOcclusion))
			{
				ensure(OcclusionEval->OcclusionType == EMeshOcclusionMapType::AmbientOcclusion);
				TexType = FTexture2DBuilder::ETextureType::AmbientOcclusion;
			}
			else if (static_cast<bool>(OcclusionEval->OcclusionType & EMeshOcclusionMapType::BentNormal))
			{
				ensure(OcclusionEval->OcclusionType == EMeshOcclusionMapType::BentNormal);
				TexType = FTexture2DBuilder::ETextureType::NormalMap;
			}
			break;
		}
		case EMeshMapEvaluatorType::Property:
		{
			const FMeshPropertyMapEvaluator* PropertyEval = static_cast<const FMeshPropertyMapEvaluator*>(Evaluator);
			switch (PropertyEval->Property)
			{
			case EMeshPropertyMapType::Normal:
			case EMeshPropertyMapType::FacetNormal:
			case EMeshPropertyMapType::Position:
			case EMeshPropertyMapType::UVPosition:
				TexType = FTexture2DBuilder::ETextureType::ColorLinear;
				break;
			case EMeshPropertyMapType::VertexColor:
			case EMeshPropertyMapType::MaterialID:
			case EMeshPropertyMapType::PolyGroupID:
				TexType = FTexture2DBuilder::ETextureType::Color;
				break;
			}
			break;
		}
		case EMeshMapEvaluatorType::Height:
		{
			TexType = FTexture2DBuilder::ETextureType::ColorLinear;
			break;
		}
		case EMeshMapEvaluatorType::UVShell:
		{
			TexType = FTexture2DBuilder::ETextureType::Color;
			break;
		}
		case EMeshMapEvaluatorType::Curvature:
		{
			TexType = FTexture2DBuilder::ETextureType::ColorLinear;
			break;
		}
		case EMeshMapEvaluatorType::ResampleImage:
		case EMeshMapEvaluatorType::MultiResampleImage:
		{
			// For texture output with 16-bit source data, output HDR texture
			if (MapFormat == EGeometryScriptBakeBitDepth::ChannelBits16)
			{
				TexType = FTexture2DBuilder::ETextureType::EmissiveHDR;
			}
			else
			{
				TexType = FTexture2DBuilder::ETextureType::Color;
			}
			break;
		}
		case EMeshMapEvaluatorType::Constant:
		{
			TexType = FTexture2DBuilder::ETextureType::ColorLinear;
			break;
		}
		}
		return TexType;
	}

	FMeshCurvatureMapEvaluator::ECurvatureType GetCurvatureType(EGeometryScriptBakeCurvatureTypeMode CurvatureType)
	{
		FMeshCurvatureMapEvaluator::ECurvatureType Result = FMeshCurvatureMapEvaluator::ECurvatureType::Mean;
		switch(CurvatureType)
		{
		case EGeometryScriptBakeCurvatureTypeMode::Mean:
			Result = FMeshCurvatureMapEvaluator::ECurvatureType::Mean;
			break;
		case EGeometryScriptBakeCurvatureTypeMode::Gaussian:
			Result = FMeshCurvatureMapEvaluator::ECurvatureType::Gaussian;
			break;
		case EGeometryScriptBakeCurvatureTypeMode::Min:
			Result = FMeshCurvatureMapEvaluator::ECurvatureType::MinPrincipal;
			break;
		case EGeometryScriptBakeCurvatureTypeMode::Max:
			Result = FMeshCurvatureMapEvaluator::ECurvatureType::MaxPrincipal;
			break;
		}
		return Result;
	}

	FMeshCurvatureMapEvaluator::EColorMode GetCurvatureColorMode(EGeometryScriptBakeCurvatureColorMode ColorMode)
	{
		FMeshCurvatureMapEvaluator::EColorMode Result = FMeshCurvatureMapEvaluator::EColorMode::BlackGrayWhite;
		switch(ColorMode)
		{
		case EGeometryScriptBakeCurvatureColorMode::Grayscale:
			Result = FMeshCurvatureMapEvaluator::EColorMode::BlackGrayWhite;
			break;
		case EGeometryScriptBakeCurvatureColorMode::RedGreenBlue:
			Result = FMeshCurvatureMapEvaluator::EColorMode::RedGreenBlue;
			break;
		case EGeometryScriptBakeCurvatureColorMode::RedBlue:
			Result = FMeshCurvatureMapEvaluator::EColorMode::RedBlue;
			break;
		}
		return Result;
	}

	FMeshCurvatureMapEvaluator::EClampMode GetCurvatureClampMode(EGeometryScriptBakeCurvatureClampMode ClampMode)
	{
		FMeshCurvatureMapEvaluator::EClampMode Result = FMeshCurvatureMapEvaluator::EClampMode::FullRange;
		switch(ClampMode)
		{
		case EGeometryScriptBakeCurvatureClampMode::None:
			Result = FMeshCurvatureMapEvaluator::EClampMode::FullRange;
			break;
		case EGeometryScriptBakeCurvatureClampMode::OnlyNegative:
			Result = FMeshCurvatureMapEvaluator::EClampMode::Negative;
			break;
		case EGeometryScriptBakeCurvatureClampMode::OnlyPositive:
			Result = FMeshCurvatureMapEvaluator::EClampMode::Positive;
			break;
		}
		return Result;
	}

	FMeshHeightMapEvaluator::EHeightRangeMode GetHeightRangeMode(EGeometryScriptBakeHeightRangeMode RangeMode)
	{
		FMeshHeightMapEvaluator::EHeightRangeMode Result = FMeshHeightMapEvaluator::EHeightRangeMode::Absolute;
		switch (RangeMode)
		{
		case EGeometryScriptBakeHeightRangeMode::Absolute:
			Result = FMeshHeightMapEvaluator::EHeightRangeMode::Absolute;
			break;
		case EGeometryScriptBakeHeightRangeMode::RelativeBounds:
			Result = FMeshHeightMapEvaluator::EHeightRangeMode::RelativeBounds;
			break;
		}
		return Result;
	}

	bool GetMeshTangents(const FDynamicMesh3* Mesh, TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe>& Tangents)
	{
		if (!Tangents)
		{
			Tangents = MakeShared<FMeshTangentsd, ESPMode::ThreadSafe>(Mesh);
			Tangents->CopyTriVertexTangents(*Mesh);

			// Validate the tangents
			if (!FDynamicMeshTangents(Mesh).HasValidTangents(true))
			{
				return false;
			}
		}
		return true;
	};

	TUniquePtr<TImageBuilder<FVector4f>> GetSampleFilterMask(
		const FGeometryScriptBakeTextureOptions& Options,
		const FText& DebugPrefix,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		if (Options.SampleFilterMask)
		{
			TUniquePtr<TImageBuilder<FVector4f>> Result = MakeUnique<TImageBuilder<FVector4f>>();
			if (!UE::AssetUtils::ReadTexture(Options.SampleFilterMask, *Result, true))
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("Bake_InvalidSampleFilterMask", "{0}: Failed to read SampleFilterMask"), DebugPrefix));
				return nullptr;
			}
			return Result;
		}
		return nullptr;
	}

	struct FEvaluatorState
	{
		const FDynamicMesh3* TargetMesh = nullptr;
		const FDynamicMesh3* SourceMesh = nullptr;
		IMeshBakerDetailSampler* DetailSampler = nullptr;
		TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> TargetMeshTangents;
		TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> SourceMeshTangents;
		TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> SourceTexture;
		TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> SourceNormalMap;
		bool bSupportsSourceNormalMap = false;
	};

	bool GetSourceNormalMap(
		FEvaluatorState& EvalState,
		const FGeometryScriptBakeSourceMeshOptions& SourceOptions,
		const FText& DebugPrefix,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		if (!EvalState.bSupportsSourceNormalMap || !SourceOptions.SourceNormalMap)
		{
			return false;
		}
		
		EvalState.SourceNormalMap = MakeShared<TImageBuilder<FVector4f>>();
		if (!UE::AssetUtils::ReadTexture(SourceOptions.SourceNormalMap, *EvalState.SourceNormalMap, false))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("Bake_InvalidSourceNormalMap", "{0}: Failed to read SourceNormalMap"), DebugPrefix));
			return false;
		}
		else
		{
			EvalState.DetailSampler->SetNormalTextureMap(EvalState.SourceMesh,
			IMeshBakerDetailSampler::FBakeDetailNormalTexture(
					EvalState.SourceNormalMap.Get(),
					SourceOptions.SourceNormalUVLayer,
					SourceOptions.SourceNormalSpace == EGeometryScriptBakeNormalSpace::Tangent ? IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent : IMeshBakerDetailSampler::EBakeDetailNormalSpace::Object));

			if (!GetMeshTangents(EvalState.SourceMesh, EvalState.SourceMeshTangents))
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("Bake_InvalidSourceTangents", "{0}: Source Mesh tangents are invalid."), DebugPrefix));
				return false;
			}
			EvalState.DetailSampler->SetTangents(EvalState.SourceMesh, EvalState.SourceMeshTangents.Get());
		}
		return true;
	}

	TSharedPtr<FMeshMapEvaluator> CreateEvaluator(
		FEvaluatorState& EvalState,
		const TFunctionRef<bool(EGeometryScriptBakeTypes, TArray<FGeometryScriptDebugMessage>*)> IsValidType,
		const FGeometryScriptBakeTypeOptions Options,
		const FText& DebugPrefix,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		// Channel evaluators only support a subset of bake types
		if (!IsValidType(Options.BakeType, Debug))
		{
			return nullptr;
		}

		auto GetTargetMeshTangents = [Debug, &DebugPrefix](FEvaluatorState& State)
		{
			const bool bSuccess = GetMeshTangents(State.TargetMesh, State.TargetMeshTangents); 
			if (!bSuccess)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("Bake_InvalidTargetTangents", "{0}: Target Mesh tangents are invalid."), DebugPrefix));
			}
			return bSuccess;
		};
		
		TSharedPtr<FMeshMapEvaluator> Result;
		switch(Options.BakeType)
		{
		case EGeometryScriptBakeTypes::TangentSpaceNormal:
		{
			TSharedPtr<FMeshNormalMapEvaluator> NormalEval = MakeShared<FMeshNormalMapEvaluator>();
			if (!GetTargetMeshTangents(EvalState))
			{
				return nullptr;
			}
			EvalState.bSupportsSourceNormalMap = true;
			Result = NormalEval;
			break;
		}
		case EGeometryScriptBakeTypes::ObjectSpaceNormal:
		{
			TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
			PropertyEval->Property = EMeshPropertyMapType::Normal;
			EvalState.bSupportsSourceNormalMap = true;
			Result = PropertyEval;
			break;
		}
		case EGeometryScriptBakeTypes::FaceNormal:
		{
			TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
			PropertyEval->Property = EMeshPropertyMapType::FacetNormal;
			Result = PropertyEval;
			break;
		}
		case EGeometryScriptBakeTypes::BentNormal:
		{
			FGeometryScriptBakeType_Occlusion* OcclusionOptions = static_cast<FGeometryScriptBakeType_Occlusion*>(Options.Options.Get());
			TSharedPtr<FMeshOcclusionMapEvaluator> OcclusionEval = MakeShared<FMeshOcclusionMapEvaluator>();
			OcclusionEval->OcclusionType = EMeshOcclusionMapType::BentNormal;
			OcclusionEval->NumOcclusionRays = OcclusionOptions->OcclusionRays;
			OcclusionEval->MaxDistance = (OcclusionOptions->MaxDistance == 0) ? TNumericLimits<float>::Max() : OcclusionOptions->MaxDistance;
			OcclusionEval->SpreadAngle = OcclusionOptions->SpreadAngle;
			switch (OcclusionOptions->NormalSpace)
			{
			case EGeometryScriptBakeNormalSpace::Tangent:
				OcclusionEval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Tangent;
				break;
			case EGeometryScriptBakeNormalSpace::Object:
				OcclusionEval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Object;
				break;
			}			
			if (!GetTargetMeshTangents(EvalState))
			{
				return nullptr;
			}
			Result = OcclusionEval;
			break;
		}
		case EGeometryScriptBakeTypes::Position:
		{
			TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
			PropertyEval->Property = EMeshPropertyMapType::Position;
			Result = PropertyEval;
			break;
		}
		case EGeometryScriptBakeTypes::Curvature:
		{
			FGeometryScriptBakeType_Curvature* CurvatureOptions = static_cast<FGeometryScriptBakeType_Curvature*>(Options.Options.Get());
			TSharedPtr<FMeshCurvatureMapEvaluator> CurvatureEval = MakeShared<FMeshCurvatureMapEvaluator>();
			CurvatureEval->UseCurvatureType = GetCurvatureType(CurvatureOptions->CurvatureType);
			CurvatureEval->UseColorMode = GetCurvatureColorMode(CurvatureOptions->ColorMapping);
			CurvatureEval->RangeScale = CurvatureOptions->ColorRangeMultiplier;
			CurvatureEval->MinRangeScale = CurvatureOptions->MinRangeMultiplier;
			CurvatureEval->UseClampMode = GetCurvatureClampMode(CurvatureOptions->Clamping);
			Result = CurvatureEval;
			break;
		}
		case EGeometryScriptBakeTypes::AmbientOcclusion:
		{
			FGeometryScriptBakeType_Occlusion* OcclusionOptions = static_cast<FGeometryScriptBakeType_Occlusion*>(Options.Options.Get());
			TSharedPtr<FMeshOcclusionMapEvaluator> OcclusionEval = MakeShared<FMeshOcclusionMapEvaluator>();
			OcclusionEval->OcclusionType = EMeshOcclusionMapType::AmbientOcclusion;
			OcclusionEval->NumOcclusionRays = OcclusionOptions->OcclusionRays;
			OcclusionEval->MaxDistance = (OcclusionOptions->MaxDistance == 0) ? TNumericLimits<float>::Max() : OcclusionOptions->MaxDistance;
			OcclusionEval->SpreadAngle = OcclusionOptions->SpreadAngle;
			OcclusionEval->BiasAngleDeg = OcclusionOptions->BiasAngle;
			Result = OcclusionEval;
			break;
		}
		case EGeometryScriptBakeTypes::Texture:
		{
			FGeometryScriptBakeType_Texture* TextureOptions = static_cast<FGeometryScriptBakeType_Texture*>(Options.Options.Get());
			TSharedPtr<FMeshResampleImageEvaluator> TextureEval = MakeShared<FMeshResampleImageEvaluator>();

			// TODO: Add support for sampling different texture maps per Texture evaluator in a single pass. 
			if (!EvalState.SourceTexture && TextureOptions->SourceTexture)
			{
				EvalState.SourceTexture = MakeShared<TImageBuilder<FVector4f>>();
				if (!UE::AssetUtils::ReadTexture(TextureOptions->SourceTexture, *EvalState.SourceTexture, false))
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeTexture_InvalidSourceTexture", "BakeTexture: Failed to read SourceTexture"));
				}
				else
				{
					EvalState.DetailSampler->SetTextureMap(EvalState.SourceMesh, IMeshBakerDetailSampler::FBakeDetailTexture(EvalState.SourceTexture.Get(), TextureOptions->SourceUVLayer));
				}
			}
			Result = TextureEval;
			break;
		}
		case EGeometryScriptBakeTypes::MultiTexture:
		{
			FGeometryScriptBakeType_MultiTexture* TextureOptions = static_cast<FGeometryScriptBakeType_MultiTexture*>(Options.Options.Get());
			TSharedPtr<FMeshMultiResampleImageEvaluator> TextureEval = MakeShared<FMeshMultiResampleImageEvaluator>();

			if (TextureOptions->MaterialIDSourceTextures.Num())
			{
				TextureEval->MultiTextures.SetNum(TextureOptions->MaterialIDSourceTextures.Num());
				for (int32 MaterialId = 0; MaterialId < TextureEval->MultiTextures.Num(); ++MaterialId)
				{
					if (UTexture2D* Texture = TextureOptions->MaterialIDSourceTextures[MaterialId])
					{
						TextureEval->MultiTextures[MaterialId] = MakeShared<TImageBuilder<FVector4f>>();
						if (!UE::AssetUtils::ReadTexture(Texture, *TextureEval->MultiTextures[MaterialId], false))
						{
							UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("Bake_InvalidMultiTexture", "{0}: Failed to read MaterialIDSourceTexture"), DebugPrefix));
						}
					}
				}
			}
			Result = TextureEval;
			break;
		}
		case EGeometryScriptBakeTypes::VertexColor:
		{
			TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
			PropertyEval->Property = EMeshPropertyMapType::VertexColor;
			Result = PropertyEval;
			break;
		}
		case EGeometryScriptBakeTypes::MaterialID:
		{
			TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
			PropertyEval->Property = EMeshPropertyMapType::MaterialID;
			Result = PropertyEval;
			break;
		}
		case EGeometryScriptBakeTypes::Height:
		{
			FGeometryScriptBakeType_Height* HeightOptions = static_cast<FGeometryScriptBakeType_Height*>(Options.Options.Get());
			TSharedPtr<FMeshHeightMapEvaluator> HeightEval = MakeShared<FMeshHeightMapEvaluator>();
			HeightEval->RangeMode = GetHeightRangeMode(HeightOptions->HeightRangeMode);
			HeightEval->Range = FInterval1f::MakeFromUnordered(HeightOptions->InnerDistance, HeightOptions->OuterDistance);
			Result = HeightEval;
			break;
		}
		case EGeometryScriptBakeTypes::UVShell:
		{
			FGeometryScriptBakeType_UVShell* UVShellOptions = static_cast<FGeometryScriptBakeType_UVShell*>(Options.Options.Get());
			TSharedPtr<FMeshUVShellMapEvaluator> UVShellEval = MakeShared<FMeshUVShellMapEvaluator>();
			UVShellEval->UVLayer = UVShellOptions->SourceUVLayer;
			UVShellEval->WireframeThickness = UVShellOptions->WireframeThickness;
			UVShellEval->WireframeColor = UVShellOptions->WireframeColor;
			UVShellEval->ShellColor = UVShellOptions->ShellColor;
			UVShellEval->BackgroundColor = UVShellOptions->BackgroundColor;
			Result = UVShellEval;
			break;
		}
		case EGeometryScriptBakeTypes::Constant:
		{
			FGeometryScriptBakeType_Constant* ConstantOptions = static_cast<FGeometryScriptBakeType_Constant*>(Options.Options.Get());
			TSharedPtr<FMeshConstantMapEvaluator> ConstantEval = MakeShared<FMeshConstantMapEvaluator>(ConstantOptions->Value);
			Result = ConstantEval;
			break;
		}
		default:
			break;
		}
		return Result;
	}

	// Verify that every enabled per-triangle overlay on the supplied mesh has element-triangle
	// storage covering the parent's MaxTriangleID.
	bool ValidateOverlayTriangleStorage(
		const UDynamicMesh* Mesh,
		const FText& MeshLabel,
		const FText& DebugPrefix,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		if (Mesh == nullptr)
		{
			return true; // null-mesh checks are handled by the callers
		}
		const FDynamicMesh3* MeshPtr = Mesh->GetMeshPtr();
		if (MeshPtr == nullptr || !MeshPtr->HasAttributes())
		{
			return true; // no overlays => nothing to validate here
		}

		const FDynamicMeshAttributeSet* Attributes = MeshPtr->Attributes();
		const int32 MaxTriangleID = MeshPtr->MaxTriangleID();

		auto ReportError = [Debug, &DebugPrefix, &MeshLabel, MaxTriangleID](const FText& OverlayLabel)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs,
				FText::Format(LOCTEXT("Bake_OverlayCoverageMismatch",
					"{0}: {1} {2} is structurally inconsistent with its parent mesh "
					"(MaxTriangleID={3}). The overlay's element-triangle storage does not cover "
					"every triangle in the parent. This usually means the mesh was constructed via "
					"a path that bypassed AppendTriangle/InsertTriangle (direct triangle-table "
					"writes, custom merges, or a deserialization mismatch). Baking this mesh would "
					"read out of bounds; aborting."),
					DebugPrefix, MeshLabel, OverlayLabel, FText::AsNumber(MaxTriangleID)));
		};

		bool bAllValid = true;

		const int32 NumUVLayers = Attributes->NumUVLayers();
		for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; ++LayerIdx)
		{
			const FDynamicMeshUVOverlay* UVLayer = Attributes->GetUVLayer(LayerIdx);
			if (UVLayer && !UVLayer->IsTriangleStorageValid())
			{
				ReportError(FText::Format(LOCTEXT("Bake_OverlayKindUV",
					"UV channel {0}"), FText::AsNumber(LayerIdx)));
				bAllValid = false;
			}
		}

		const int32 NumNormalLayers = Attributes->NumNormalLayers();
		for (int32 LayerIdx = 0; LayerIdx < NumNormalLayers; ++LayerIdx)
		{
			const FDynamicMeshNormalOverlay* NormalLayer = Attributes->GetNormalLayer(LayerIdx);
			if (NormalLayer && !NormalLayer->IsTriangleStorageValid())
			{
				ReportError(FText::Format(LOCTEXT("Bake_OverlayKindNormal",
					"Normal channel {0}"), FText::AsNumber(LayerIdx)));
				bAllValid = false;
			}
		}

		if (const FDynamicMeshColorOverlay* ColorLayer = Attributes->PrimaryColors())
		{
			if (!ColorLayer->IsTriangleStorageValid())
			{
				ReportError(LOCTEXT("Bake_OverlayKindColor", "Color overlay"));
				bAllValid = false;
			}
		}

		return bAllValid;
	}

	TUniquePtr<FMeshMapBaker> BakeTextureImpl(
		UDynamicMesh* TargetMesh,
		FTransform TargetTransform,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		UDynamicMesh* SourceMesh,
		FTransform SourceTransform,
		FGeometryScriptBakeSourceMeshOptions SourceOptions,
		const TArray<FGeometryScriptBakeTypeOptions>& BakeTypes,
		FGeometryScriptBakeTextureOptions BakeOptions,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeTexture_InvalidTargetMesh", "BakeTexture: TargetMesh is Null"));
			return nullptr;
		}
		if (SourceMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeTexture_InvalidSourceMesh", "BakeTexture: SourceMesh is Null"));
			return nullptr;
		}
		if (BakeTypes.Num() == 0)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeTexture_BakeTypesEmpty", "BakeTexture: BakeTypes is empty"));
			return nullptr;
		}

		const FText BakeTexturePrefix = LOCTEXT("BakeTexture_Prefix", "BakeTexture");

		const FText TargetMeshLabel = LOCTEXT("Bake_TargetMeshLabel", "TargetMesh");
		const FText SourceMeshLabel = LOCTEXT("Bake_SourceMeshLabel", "SourceMesh");
		bool bAllStorageValid = ValidateOverlayTriangleStorage(TargetMesh, TargetMeshLabel, BakeTexturePrefix, Debug);
		if (TargetMesh != SourceMesh)
		{
			bAllStorageValid &= ValidateOverlayTriangleStorage(SourceMesh, SourceMeshLabel, BakeTexturePrefix, Debug);
		}
		if (!bAllStorageValid)
		{
			return nullptr;
		}

		const bool bIsBakeToSelf = (TargetMesh == SourceMesh);

		FDynamicMesh3 SourceMeshCopy;
		const FDynamicMesh3* SourceMeshOriginal = SourceMesh->GetMeshPtr();
		const FDynamicMesh3* SourceMeshToUse = SourceMeshOriginal;
		if (BakeOptions.bProjectionInWorldSpace && !bIsBakeToSelf)
		{
			// Transform the SourceMesh into TargetMesh local space using a copy (oof)
			// TODO: Remove this once we have support for transforming rays in the core bake loop
			SourceMeshCopy = *SourceMeshOriginal;
			const FTransformSRT3d SourceToWorld = SourceTransform;
			MeshTransforms::ApplyTransform(SourceMeshCopy, SourceToWorld, true);
			const FTransformSRT3d TargetToWorld = TargetTransform;
			MeshTransforms::ApplyTransformInverse(SourceMeshCopy, TargetToWorld, true);
			SourceMeshToUse = &SourceMeshCopy;
		}

		FImageDimensions BakeDimensions = GetDimensions(BakeOptions.Resolution);
		const FDynamicMeshAABBTree3 DetailSpatial(SourceMeshToUse);
		FMeshBakerDynamicMeshSampler DetailSampler(SourceMeshToUse, &DetailSpatial);

		TUniquePtr<FMeshMapBaker> Result = MakeUnique<FMeshMapBaker>();
		FMeshMapBaker& Baker = *Result;
		Baker.SetTargetMesh(TargetMesh->GetMeshPtr());
		Baker.SetTargetMeshUVLayer(TargetOptions.TargetUVLayer);
		Baker.SetDetailSampler(&DetailSampler);
		Baker.SetDimensions(BakeDimensions);
		Baker.SetProjectionDistance(BakeOptions.ProjectionDistance);
		Baker.SetSamplesPerPixel(GetSamplesPerPixel(BakeOptions.SamplesPerPixel));
		Baker.SetFilter(GetFilter(BakeOptions.FilteringType));
		Baker.SetGutterEnabled(BakeOptions.GutterSize > 0);
		Baker.SetGutterSize(BakeOptions.GutterSize);
		TUniquePtr<TImageBuilder<FVector4f>> SampleFilterMask = GetSampleFilterMask(BakeOptions, BakeTexturePrefix, Debug);
		if (SampleFilterMask)
		{
			Baker.SampleFilterF = [&SampleFilterMask](const FVector2i& ImageCoords, const FVector2d& UV, int32 TriID)
			{
				const FVector4f Mask = SampleFilterMask->BilinearSampleUV<float>(UV, FVector4f::One());
				return (Mask.X + Mask.Y + Mask.Z) / 3;
			};
		}
		if (bIsBakeToSelf)
		{
			Baker.SetCorrespondenceStrategy(FMeshBaseBaker::ECorrespondenceStrategy::Identity);
		}

		auto IsValidBakeType = [&BakeTexturePrefix](EGeometryScriptBakeTypes BakeType, TArray<FGeometryScriptDebugMessage>* Debug)
		{
			const bool bIsValid = BakeType != EGeometryScriptBakeTypes::None;
			if (!bIsValid)
			{
				const FText BakeTypeName = FText::FromName(StaticEnum<EGeometryScriptBakeTypes>()->GetNameByIndex(static_cast<int32>(BakeType)));
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("BakeTexture_InvalidBakeType", "{0}: {1} bake type is not a supported RGBA evaluator."), BakeTexturePrefix, BakeTypeName));
			}
			return bIsValid;
		};

		FEvaluatorState EvalState;
		EvalState.TargetMesh = TargetMesh->GetMeshPtr();
		EvalState.SourceMesh = SourceMeshToUse;
		EvalState.DetailSampler = &DetailSampler;
		for (const FGeometryScriptBakeTypeOptions& Options : BakeTypes)
		{
			TSharedPtr<FMeshMapEvaluator> Eval = CreateEvaluator(EvalState, IsValidBakeType, Options, BakeTexturePrefix, Debug);
			if (!Eval)
			{
				// Abort if any evaluators failed to build.
				return nullptr;
			}

			// Post evaluator construction handling
			switch (Options.BakeType)
			{
			case EGeometryScriptBakeTypes::UVShell:
			{
				FMeshUVShellMapEvaluator* UVShellEval = static_cast<FMeshUVShellMapEvaluator*>(Eval.Get());
				UVShellEval->TexelSize = BakeDimensions.GetTexelSize();
				break;
			}
			default:
				break;
			}
			
			Baker.AddEvaluator(Eval);
		}

		if (EvalState.bSupportsSourceNormalMap && SourceOptions.SourceNormalMap)
		{
			GetSourceNormalMap(EvalState, SourceOptions, BakeTexturePrefix, Debug);
		}

		if (EvalState.TargetMeshTangents)
		{
			Baker.SetTargetMeshTangents(EvalState.TargetMeshTangents);
		}

		Baker.Bake();

		return MoveTemp(Result);
	}

	void GetTexturesFromBaker(FMeshMapBaker* Baker, const EGeometryScriptBakeBitDepth BakeBitDepth, TArray<UTexture2D*>& Textures)
	{
		if (!Baker)
		{
			return;
		}
		
		const FImageDimensions BakeDimensions = Baker->GetDimensions();
		const int NumEval = Baker->NumEvaluators();
		for (int EvalIdx = 0; EvalIdx < NumEval; ++EvalIdx)
		{
			// For 8-bit color textures, ensure that the source data is in sRGB.
			const FTexture2DBuilder::ETextureType TexType = GetTextureType(Baker->GetEvaluator(EvalIdx), BakeBitDepth);
			const bool bConvertToSRGB = (TexType == FTexture2DBuilder::ETextureType::Color);
			const ETextureSourceFormat SourceDataFormat = (BakeBitDepth == EGeometryScriptBakeBitDepth::ChannelBits16 ? TSF_RGBA16F : TSF_BGRA8);

			constexpr int ResultIdx = 0;
			FTexture2DBuilder TextureBuilder;
			TextureBuilder.Initialize(TexType, BakeDimensions);
			TextureBuilder.Copy(*Baker->GetBakeResults(EvalIdx)[ResultIdx], bConvertToSRGB);
			TextureBuilder.Commit(false);

			// Copy image to source data after commit. This will avoid incurring
			// the cost of hitting the DDC for texture compile while iterating on
			// bake settings. Since this dirties the texture, the next time the texture
			// is used after accepting the final texture, the DDC will trigger and
			// properly recompile the platform data.
			const bool bConvertSourceToSRGB = bConvertToSRGB && SourceDataFormat == TSF_BGRA8;
			TextureBuilder.CopyImageToSourceData(*Baker->GetBakeResults(EvalIdx)[ResultIdx], SourceDataFormat, bConvertSourceToSRGB);
			Textures.Add(TextureBuilder.GetTexture2D());
		}
	}

	TUniquePtr<FMeshVertexBaker> BakeVertexImpl(
		UDynamicMesh* TargetMesh,
		const FTransform& TargetTransform,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		UDynamicMesh* SourceMesh,
		const FTransform& SourceTransform,
		FGeometryScriptBakeSourceMeshOptions SourceOptions,
		FGeometryScriptBakeOutputType BakeTypes,
		FGeometryScriptBakeVertexOptions BakeOptions,
		TArray<FGeometryScriptDebugMessage>* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeVertex_InvalidTargetMesh", "BakeVertex: TargetMesh is Null"));
			return nullptr;
		}
		if (SourceMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeVertex_InvalidSourceMesh", "BakeVertex: SourceMesh is Null"));
			return nullptr;
		}

		const FText BakeVertexPrefix = LOCTEXT("BakeVertex_Prefix", "BakeVertex");

		const FText TargetMeshLabel = LOCTEXT("Bake_TargetMeshLabel", "TargetMesh");
		const FText SourceMeshLabel = LOCTEXT("Bake_SourceMeshLabel", "SourceMesh");
		bool bAllStorageValid = ValidateOverlayTriangleStorage(TargetMesh, TargetMeshLabel, BakeVertexPrefix, Debug);
		if (TargetMesh != SourceMesh)
		{
			bAllStorageValid &= ValidateOverlayTriangleStorage(SourceMesh, SourceMeshLabel, BakeVertexPrefix, Debug);
		}
		if (!bAllStorageValid)
		{
			return nullptr;
		}

		const bool bIsBakeToSelf = (TargetMesh == SourceMesh);

		// Initialize the source mesh
		// This must precede target mesh generation in case SourceMesh & TargetMesh are the same mesh
		// since we mutate the target mesh color topology.
		FDynamicMesh3 SourceMeshCopy = *SourceMesh->GetMeshPtr();
		const FDynamicMesh3* SourceMeshToUse = &SourceMeshCopy;
		if (BakeOptions.bProjectionInWorldSpace && !bIsBakeToSelf)
		{
			// Transform the SourceMesh into TargetMesh local space using a copy (oof)
			// TODO: Remove this once we have support for transforming rays in the core bake loop
			const FTransformSRT3d SourceToWorld = SourceTransform;
			MeshTransforms::ApplyTransform(SourceMeshCopy, SourceToWorld, true);
			const FTransformSRT3d TargetToWorld = TargetTransform;
			MeshTransforms::ApplyTransformInverse(SourceMeshCopy, TargetToWorld, true);
		}

		const FDynamicMeshAABBTree3 DetailSpatial(SourceMeshToUse);
		FMeshBakerDynamicMeshSampler DetailSampler(SourceMeshToUse, &DetailSpatial);

		// Initialize the color overlay on the TargetMesh
		FDynamicMesh3& TargetMeshRef = TargetMesh->GetMeshRef();
		if (BakeOptions.TopologyMode == EGeometryScriptBakeVertexTopology::CreateNew)
		{
			TargetMeshRef.EnableAttributes();
			TargetMeshRef.Attributes()->EnablePrimaryColors();
			TargetMeshRef.Attributes()->PrimaryColors()->ClearElements();

			FDynamicMeshNormalOverlay* NormalOverlay = TargetMeshRef.Attributes()->PrimaryNormals();
			FDynamicMeshUVOverlay* UVOverlay = TargetMeshRef.Attributes()->PrimaryUV();
			TargetMeshRef.Attributes()->PrimaryColors()->CreateFromPredicate(
				[&BakeOptions, NormalOverlay, UVOverlay](int ParentVID, int TriIDA, int TriIDB) -> bool
				{
					auto OverlayCanShare = [TriIDA, TriIDB] (auto Overlay) -> bool
					{
						return Overlay ? Overlay->AreTrianglesConnected(TriIDA, TriIDB) : true;
					};
				
					bool bCanShare = true;
					if (BakeOptions.bSplitAtNormalSeams)
					{
						bCanShare = bCanShare && OverlayCanShare(NormalOverlay);
					}
					if (BakeOptions.bSplitAtUVSeams)
					{
						bCanShare = bCanShare && OverlayCanShare(UVOverlay);
					}
					return bCanShare;
				}, 0.0f);
		}
		else
		{
			if (!TargetMeshRef.Attributes() ||
				!TargetMeshRef.Attributes()->PrimaryColors())
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BakeVertex_InvalidVertexColorTopology", "BakeVertex: Topology Mode was set to UseExisting, but existing colors were not found."));
				return nullptr;
			}
		}

		if (bIsBakeToSelf || BakeTypes.OutputMode == EGeometryScriptBakeOutputMode::PerChannel)
		{
			// Copy source vertex colors onto new color overlay topology for identity bakes
			// This is necessary when sampling vertex color data.
			//
			// Also copy source vertex colors for PerChannel bakes so users can opt to bake
			// a single channel of an existing vertex color dataset.
			const FDynamicMeshColorOverlay* SourceColorOverlay = SourceMeshToUse->HasAttributes() ? SourceMeshToUse->Attributes()->PrimaryColors() : nullptr;
			FDynamicMeshColorOverlay* TargetColorOverlay = TargetMeshRef.Attributes()->PrimaryColors(); 
			if (SourceColorOverlay)
			{
				for (int VId : TargetMeshRef.VertexIndicesItr())
				{
					TargetMeshRef.EnumerateVertexTriangles(VId, [VId, SourceColorOverlay, TargetColorOverlay](int32 TriID)
					{
						const FVector4f TargetColor = SourceColorOverlay->GetElementAtVertex(TriID, VId);
						const int ElemId = TargetColorOverlay->GetElementIDAtVertex(TriID, VId);
						TargetColorOverlay->SetElement(ElemId, TargetColor);
					});
				}
			}
		}

		TUniquePtr<FMeshVertexBaker> Result = MakeUnique<FMeshVertexBaker>();
		FMeshVertexBaker& Baker = *Result;
		Baker.BakeMode = BakeTypes.OutputMode == EGeometryScriptBakeOutputMode::RGBA ? FMeshVertexBaker::EBakeMode::RGBA : FMeshVertexBaker::EBakeMode::PerChannel;
		Baker.SetTargetMesh(TargetMesh->GetMeshPtr());
		Baker.SetTargetMeshUVLayer(TargetOptions.TargetUVLayer);
		Baker.SetDetailSampler(&DetailSampler);
		Baker.SetProjectionDistance(BakeOptions.ProjectionDistance);
		if (bIsBakeToSelf)
		{
			Baker.SetCorrespondenceStrategy(FMeshBaseBaker::ECorrespondenceStrategy::Identity);
		}

		FEvaluatorState EvalState;
		EvalState.TargetMesh = TargetMesh->GetMeshPtr();
		EvalState.SourceMesh = SourceMeshToUse;
		EvalState.DetailSampler = &DetailSampler;

		if (BakeTypes.OutputMode == EGeometryScriptBakeOutputMode::RGBA)
		{
			auto IsValidBakeType = [&BakeVertexPrefix](EGeometryScriptBakeTypes BakeType, TArray<FGeometryScriptDebugMessage>* Debug)
			{
				const bool bIsValid = (BakeType != EGeometryScriptBakeTypes::None && BakeType != EGeometryScriptBakeTypes::UVShell);
				if (!bIsValid)
				{
					const FText BakeTypeName = FText::FromName(StaticEnum<EGeometryScriptBakeTypes>()->GetNameByIndex(static_cast<int32>(BakeType)));
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("BakeVertex_InvalidBakeType", "{0}: {1} bake type is not a supported RGBA evaluator."), BakeVertexPrefix, BakeTypeName));
				}
				return bIsValid;
			};
			
			Baker.BakeMode = FMeshVertexBaker::EBakeMode::RGBA;
			Baker.ColorEvaluator = CreateEvaluator(EvalState, IsValidBakeType, BakeTypes.RGBA, BakeVertexPrefix, Debug);
		}
		else
		{
			ensure(BakeTypes.OutputMode == EGeometryScriptBakeOutputMode::PerChannel);

			auto IsValidBakeType = [&BakeVertexPrefix](EGeometryScriptBakeTypes BakeType, TArray<FGeometryScriptDebugMessage>* Debug)
			{
				// The None bake type is permitted for PerChannel bakes since it allows users to specify a channel that should not be overwritten.
				const bool bIsValid = (BakeType == EGeometryScriptBakeTypes::AmbientOcclusion
					|| BakeType == EGeometryScriptBakeTypes::Curvature
					|| BakeType == EGeometryScriptBakeTypes::Height
					|| BakeType == EGeometryScriptBakeTypes::Constant
					|| BakeType == EGeometryScriptBakeTypes::None);
				if (!bIsValid)
				{
					const FText BakeTypeName = FText::FromName(StaticEnum<EGeometryScriptBakeTypes>()->GetNameByIndex(static_cast<int32>(BakeType)));
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("BakeVertex_InvalidChannelEval", "{0}: {1} bake type is not a supported per-channel evaluator."), BakeVertexPrefix, BakeTypeName));
				}
				return bIsValid;
			};
			
			Baker.BakeMode = FMeshVertexBaker::EBakeMode::PerChannel;
			Baker.ChannelEvaluators[0] = CreateEvaluator(EvalState, IsValidBakeType, BakeTypes.R, BakeVertexPrefix, Debug);
			Baker.ChannelEvaluators[1] = CreateEvaluator(EvalState, IsValidBakeType, BakeTypes.G, BakeVertexPrefix, Debug);
			Baker.ChannelEvaluators[2] = CreateEvaluator(EvalState, IsValidBakeType, BakeTypes.B, BakeVertexPrefix, Debug);
			Baker.ChannelEvaluators[3] = CreateEvaluator(EvalState, IsValidBakeType, BakeTypes.A, BakeVertexPrefix, Debug);
		}

		TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> SourceNormalMap;
		TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> SourceMeshTangents;
		if (EvalState.bSupportsSourceNormalMap && SourceOptions.SourceNormalMap)
		{
			GetSourceNormalMap(EvalState, SourceOptions, BakeVertexPrefix, Debug);
		}

		if (EvalState.TargetMeshTangents)
		{
			Baker.SetTargetMeshTangents(EvalState.TargetMeshTangents);
		}

		Baker.Bake();
		
		return MoveTemp(Result);
	}

	bool ApplyVertexBakeToMesh(const FMeshVertexBaker* Baker, const FGeometryScriptBakeOutputType& BakeTypes, UDynamicMesh* Mesh)
	{
		if (!Baker || !Mesh)
		{
			return false;
		}

		if (BakeTypes.OutputMode == EGeometryScriptBakeOutputMode::PerChannel)
		{
			// Precompute scale vectors for source and image pixel data to merge
			// the data according to the populated channels.
			FVector4f SrcScale = FVector4f::Zero();
			FVector4f ImgScale = FVector4f::Zero();

			const bool bOutputR = BakeTypes.R.BakeType != EGeometryScriptBakeTypes::None;
			const bool bOutputG = BakeTypes.G.BakeType != EGeometryScriptBakeTypes::None;
			const bool bOutputB = BakeTypes.B.BakeType != EGeometryScriptBakeTypes::None;
			const bool bOutputA = BakeTypes.A.BakeType != EGeometryScriptBakeTypes::None;

			SrcScale[0] = static_cast<float>(!bOutputR);
			ImgScale[0] = static_cast<float>(bOutputR);

			SrcScale[1] = static_cast<float>(!bOutputG);
			ImgScale[1] = static_cast<float>(bOutputG);

			SrcScale[2] = static_cast<float>(!bOutputB);
			ImgScale[2] = static_cast<float>(bOutputB);

			SrcScale[3] = static_cast<float>(!bOutputA);
			ImgScale[3] = static_cast<float>(bOutputA);

			FDynamicMesh3& MeshRef = Mesh->GetMeshRef();
			const int NumColors = MeshRef.Attributes()->PrimaryColors()->ElementCount();
			const TImageBuilder<FVector4f>* ImageResult = Baker->GetBakeResult();
			check(NumColors == ImageResult->GetDimensions().GetWidth());
			for (int Idx = 0; Idx < NumColors; ++Idx)
			{
				if (const FDynamicMeshColorOverlay* ColorOverlay = MeshRef.Attributes()->PrimaryColors())
				{
					FVector4f Pixel;
					ColorOverlay->GetElement(Idx, Pixel);
					Pixel *= SrcScale;

					// Swizzle the ImageResult pixels based on the requested channels.
					const FVector4f& ImagePixel = ImageResult->GetPixel(Idx);
					Pixel += ImagePixel * ImgScale;
					MeshRef.Attributes()->PrimaryColors()->SetElement(Idx, Pixel);
				}
			}
		}
		else
		{
			FDynamicMesh3& MeshRef = Mesh->GetMeshRef();		
			const TImageBuilder<FVector4f>* ImageResult = Baker->GetBakeResult();
			const int NumColors = MeshRef.Attributes()->PrimaryColors()->ElementCount();
			check(NumColors == ImageResult->GetDimensions().GetWidth());
			for (int Idx = 0; Idx < NumColors; ++Idx)
			{
				const FVector4f& Pixel = ImageResult->GetPixel(Idx);
				MeshRef.Attributes()->PrimaryColors()->SetElement(Idx, Pixel);
			}
		}
		return true;
	}

	// Verbose but more robust than casting
	EBakeTextureResolution ConvertEnum(EGeometryScriptBakeResolution Input)
	{
		switch(Input)
		{
			case EGeometryScriptBakeResolution::Resolution16:   return EBakeTextureResolution::Resolution16;
			case EGeometryScriptBakeResolution::Resolution32:   return EBakeTextureResolution::Resolution32;
			case EGeometryScriptBakeResolution::Resolution64:   return EBakeTextureResolution::Resolution64;
			case EGeometryScriptBakeResolution::Resolution128:  return EBakeTextureResolution::Resolution128;
			case EGeometryScriptBakeResolution::Resolution256:  return EBakeTextureResolution::Resolution256;
			case EGeometryScriptBakeResolution::Resolution512:  return EBakeTextureResolution::Resolution512;
			case EGeometryScriptBakeResolution::Resolution1024: return EBakeTextureResolution::Resolution1024;
			case EGeometryScriptBakeResolution::Resolution2048: return EBakeTextureResolution::Resolution2048;
			case EGeometryScriptBakeResolution::Resolution4096: return EBakeTextureResolution::Resolution4096;
			case EGeometryScriptBakeResolution::Resolution8192: return EBakeTextureResolution::Resolution8192;
		}
		ensure(false);
		return EBakeTextureResolution::Resolution16;
	}

	// Verbose but more robust than casting
	EBakeTextureSamplesPerPixel ConvertEnum(EGeometryScriptBakeSamplesPerPixel Input)
	{
		switch(Input)
		{
			case EGeometryScriptBakeSamplesPerPixel::Sample1:   return EBakeTextureSamplesPerPixel::Sample1;
			case EGeometryScriptBakeSamplesPerPixel::Sample4:   return EBakeTextureSamplesPerPixel::Sample4;
			case EGeometryScriptBakeSamplesPerPixel::Sample16:  return EBakeTextureSamplesPerPixel::Sample16;
			case EGeometryScriptBakeSamplesPerPixel::Sample64:  return EBakeTextureSamplesPerPixel::Sample64;
			case EGeometryScriptBakeSamplesPerPixel::Samples256: return EBakeTextureSamplesPerPixel::Sample256;
		}
		ensure(false);
		return EBakeTextureSamplesPerPixel::Sample1;
	}


	FSceneCaptureConfig GetSceneCaptureConfig(FGeometryScriptBakeRenderCaptureOptions BakeOptions)
	{
		FSceneCaptureConfig Parameters;

		Parameters.Flags.bBaseColor = BakeOptions.bBaseColorMap;
		Parameters.Flags.bWorldNormal = BakeOptions.bNormalMap;
		Parameters.Flags.bEmissive = BakeOptions.bEmissiveMap;
		Parameters.Flags.bOpacity = BakeOptions.bOpacityMap;
		Parameters.Flags.bSubsurfaceColor = BakeOptions.bSubsurfaceColorMap;

		// TODO Maybe its better UX if we ignore the precondition below, it makes sense to do it in the BakeRC tool but
		// in GS it is easy to add it manually and users may be surprised to not get the textures they requested

		// Enforce the PackedMRS precondition here
		Parameters.Flags.bCombinedMRS = BakeOptions.bPackedMRSMap;
		Parameters.Flags.bMetallic    = BakeOptions.bPackedMRSMap ? false : BakeOptions.bMetallicMap;
		Parameters.Flags.bRoughness   = BakeOptions.bPackedMRSMap ? false : BakeOptions.bRoughnessMap;
		Parameters.Flags.bSpecular    = BakeOptions.bPackedMRSMap ? false : BakeOptions.bSpecularMap;

		// Only compute the device depth if we compute at least one other channel, the DeviceDepth is used to eliminate
		// occlusion artefacts from the other channels
		Parameters.Flags.bDeviceDepth =
			(BakeOptions.CleanupTolerance > 0) && (Parameters.Flags != FRenderCaptureTypeFlags::None());

		Parameters.RenderCaptureImageSize = GetDimensions(BakeOptions.RenderCaptureResolution).GetHeight();
		Parameters.bAntiAliasing = BakeOptions.bRenderCaptureAntiAliasing;
		Parameters.FieldOfViewDegrees = BakeOptions.FieldOfViewDegrees;
		Parameters.NearPlaneDist = BakeOptions.NearPlaneDist;

		return Parameters;
	}
} // end namespace GeometryScriptBakeLocals



int UGeometryScriptLibrary_MeshBakeFunctions::ConvertBakeResolutionToInt(EGeometryScriptBakeResolution BakeResolution)
{
	FImageDimensions Dimensions = GeometryScriptBakeLocals::GetDimensions(BakeResolution);

	int Output;
	Output = Dimensions.GetWidth();
	checkSlow(Output == Dimensions.GetHeight());
	return Output;
}


FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeTangentNormal()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::TangentSpaceNormal;
	return Output;
}


FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeObjectNormal()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::ObjectSpaceNormal;
	return Output;
}


FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeFaceNormal()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::FaceNormal;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeBentNormal(
	int OcclusionRays,
	float MaxDistance,
	float SpreadAngle,
	EGeometryScriptBakeNormalSpace NormalSpace)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::BentNormal;
	const TSharedPtr<FGeometryScriptBakeType_Occlusion> OcclusionOptions = MakeShared<FGeometryScriptBakeType_Occlusion>();
	OcclusionOptions->OcclusionRays = OcclusionRays;
	OcclusionOptions->MaxDistance = MaxDistance;
	OcclusionOptions->SpreadAngle = SpreadAngle;
	OcclusionOptions->NormalSpace = NormalSpace;
	Output.Options = OcclusionOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypePosition()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::Position;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeCurvature(
	EGeometryScriptBakeCurvatureTypeMode CurvatureType,
	EGeometryScriptBakeCurvatureColorMode ColorMapping,
	float ColorRangeMultiplier,
	float MinRangeMultiplier,
	EGeometryScriptBakeCurvatureClampMode Clamping)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::Curvature;
	const TSharedPtr<FGeometryScriptBakeType_Curvature> CurvatureOptions = MakeShared<FGeometryScriptBakeType_Curvature>();
	CurvatureOptions->CurvatureType = CurvatureType;
	CurvatureOptions->ColorMapping = ColorMapping;
	CurvatureOptions->ColorRangeMultiplier = ColorRangeMultiplier;
	CurvatureOptions->MinRangeMultiplier = MinRangeMultiplier;
	Output.Options = CurvatureOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeAmbientOcclusion(
	int OcclusionRays,
	float MaxDistance,
	float SpreadAngle,
	float BiasAngle)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::AmbientOcclusion;
	const TSharedPtr<FGeometryScriptBakeType_Occlusion> OcclusionOptions = MakeShared<FGeometryScriptBakeType_Occlusion>();
	OcclusionOptions->OcclusionRays = OcclusionRays;
	OcclusionOptions->MaxDistance = MaxDistance;
	OcclusionOptions->SpreadAngle = SpreadAngle;
	OcclusionOptions->BiasAngle = BiasAngle;
	Output.Options = OcclusionOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeTexture(
	UTexture2D* SourceTexture,
	int SourceUVLayer)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::Texture;
	const TSharedPtr<FGeometryScriptBakeType_Texture> TextureOptions = MakeShared<FGeometryScriptBakeType_Texture>();
	TextureOptions->SourceTexture = SourceTexture;
	TextureOptions->SourceUVLayer = SourceUVLayer;
	Output.Options = TextureOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeMultiTexture(
	const TArray<UTexture2D*>& MaterialIDSourceTextures,
	int SourceUVLayer)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::MultiTexture;
	const TSharedPtr<FGeometryScriptBakeType_MultiTexture> MultiTextureOptions = MakeShared<FGeometryScriptBakeType_MultiTexture>();
	MultiTextureOptions->MaterialIDSourceTextures = MaterialIDSourceTextures;
	MultiTextureOptions->SourceUVLayer = SourceUVLayer;
	Output.Options = MultiTextureOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeVertexColor()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::VertexColor;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeMaterialID()
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::MaterialID;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeUVShell(
	int SourceUVLayer,
	float WireframeThickness,
	FLinearColor WireframeColor,
	FLinearColor ShellColor,
	FLinearColor BackgroundColor)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::UVShell;
	const TSharedPtr<FGeometryScriptBakeType_UVShell> UVShellOptions = MakeShared<FGeometryScriptBakeType_UVShell>();
	UVShellOptions->SourceUVLayer = SourceUVLayer;
	UVShellOptions->WireframeThickness = WireframeThickness;
	UVShellOptions->WireframeColor = WireframeColor;
	UVShellOptions->ShellColor = ShellColor;
	UVShellOptions->BackgroundColor = BackgroundColor;
	Output.Options = UVShellOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeHeight(EGeometryScriptBakeHeightRangeMode RangeMode, float InnerDistance, float OuterDistance)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::Height;
	const TSharedPtr<FGeometryScriptBakeType_Height> HeightOptions = MakeShared<FGeometryScriptBakeType_Height>();
	HeightOptions->HeightRangeMode = RangeMode;
	HeightOptions->InnerDistance = InnerDistance;
	HeightOptions->OuterDistance = OuterDistance;
	Output.Options = HeightOptions;
	return Output;
}

FGeometryScriptBakeTypeOptions UGeometryScriptLibrary_MeshBakeFunctions::MakeBakeTypeConstant(
	float Value)
{
	FGeometryScriptBakeTypeOptions Output;
	Output.BakeType = EGeometryScriptBakeTypes::Constant;
	const TSharedPtr<FGeometryScriptBakeType_Constant> ConstantOptions = MakeShared<FGeometryScriptBakeType_Constant>();
	ConstantOptions->Value = Value;
	Output.Options = ConstantOptions;
	return Output;
}

TArray<UTexture2D*> UGeometryScriptLibrary_MeshBakeFunctions::BakeTexture(
	UDynamicMesh* TargetMesh,
	FTransform TargetTransform,
	FGeometryScriptBakeTargetMeshOptions TargetOptions,
	UDynamicMesh* SourceMesh,
	FTransform SourceTransform,
	FGeometryScriptBakeSourceMeshOptions SourceOptions,
	const TArray<FGeometryScriptBakeTypeOptions>& BakeTypes,
	FGeometryScriptBakeTextureOptions BakeOptions,
	UGeometryScriptDebug* Debug)
{
	const TUniquePtr<FMeshMapBaker> Baker = GeometryScriptBakeLocals::BakeTextureImpl(
		TargetMesh,
		TargetTransform,
		TargetOptions,
		SourceMesh,
		SourceTransform,
		SourceOptions,
		BakeTypes,
		BakeOptions,
		Debug ? &Debug->Messages : nullptr);

	TArray<UTexture2D*> TextureOutput;
	GeometryScriptBakeLocals::GetTexturesFromBaker(Baker.Get(), BakeOptions.BitDepth, TextureOutput);

	return TextureOutput;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBakeFunctions::BakeVertex(
		UDynamicMesh* TargetMesh,
		FTransform TargetTransform,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		UDynamicMesh* SourceMesh,
		FTransform SourceTransform,
		FGeometryScriptBakeSourceMeshOptions SourceOptions,
		FGeometryScriptBakeOutputType BakeTypes,
		FGeometryScriptBakeVertexOptions BakeOptions,
		UGeometryScriptDebug* Debug)
{
	const TUniquePtr<FMeshVertexBaker> Baker = GeometryScriptBakeLocals::BakeVertexImpl(
		TargetMesh,
		TargetTransform,
		TargetOptions,
		SourceMesh,
		SourceTransform,
		SourceOptions,
		BakeTypes,
		BakeOptions,
		Debug ? &Debug->Messages : nullptr);

	// Extract the vertex bake data and apply to target mesh.
	GeometryScriptBakeLocals::ApplyVertexBakeToMesh(Baker.Get(), BakeTypes, TargetMesh);

	return TargetMesh;
}


FGeometryScriptRenderCaptureTextures UGeometryScriptLibrary_MeshBakeFunctions::BakeTextureFromRenderCaptures(
	UDynamicMesh* TargetMesh,
	FTransform TargetLocalToWorld,
	FGeometryScriptBakeTargetMeshOptions TargetOptions,
	const TArray<AActor*> SourceActors,
	FGeometryScriptBakeRenderCaptureOptions BakeOptions,
	UGeometryScriptDebug* Debug)
{
	// Its possible to pass nullptrs in SourceActors so we filter these out here
	TArray<TObjectPtr<AActor>> ValidSourceActors;
	for (AActor* Actor : SourceActors)
	{
		if (Actor != nullptr)
		{
			ValidSourceActors.Add(Actor);
		}
	}

	if (ValidSourceActors.IsEmpty())
	{
		return {};
	}

	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs,
			LOCTEXT("BakeTextureFromRenderCaptures_InvalidTargetMesh", "BakeTextureFromRenderCaptures: TargetMesh is Null"));
		return {};
	}

	// Transform the mesh, and undo it before we return
	MeshTransforms::ApplyTransform(TargetMesh->GetMeshRef(), TargetLocalToWorld);
	ON_SCOPE_EXIT {
		MeshTransforms::ApplyTransformInverse(TargetMesh->GetMeshRef(), TargetLocalToWorld);
	};

	const auto HasDegenerateUVs = [&TargetMesh, &TargetOptions]
	{
		FDynamicMeshUVOverlay* UVOverlay = TargetMesh->GetMeshRef().Attributes()->GetUVLayer(TargetOptions.TargetUVLayer);
		FAxisAlignedBox2f Bounds = FAxisAlignedBox2f::Empty();
		for (const int Index : UVOverlay->ElementIndicesItr())
		{
			FVector2f UV;
			UVOverlay->GetElement(Index, UV);
			Bounds.Contain(UV);
		}
		return Bounds.Min == Bounds.Max;
	};

	if (TargetMesh->GetMeshRef().Attributes()->GetUVLayer(TargetOptions.TargetUVLayer) == nullptr)
	{
		AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs,
			LOCTEXT("BakeTextureFromRenderCaptures_TargetMeshMissingUVs", "BakeTextureFromRenderCaptures: TargetMesh UV layer is missing"));
		return {};
	}

	if (HasDegenerateUVs())
	{
		AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs,
			LOCTEXT("BakeTextureFromRenderCaptures_TargetMeshDegenerateUVs", "BakeTextureFromRenderCaptures: TargetMesh UV layer is degenerate"));
		return {};
	}

	if (BakeOptions.bNormalMap && !FDynamicMeshTangents(TargetMesh->GetMeshPtr()).HasValidTangents(true))
	{
		AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs,
			LOCTEXT("BakeTextureFromRenderCaptures_InvalidMeshTangents", "BakeTextureFromRenderCaptures: TargetMesh has invalid tangents so the requested normal map cannot be baked"));
		return {};
	}

	TUniquePtr<FSceneCapturePhotoSet> SceneCapture = MakeUnique<FSceneCapturePhotoSet>();
	const FSceneCaptureConfig Options = GeometryScriptBakeLocals::GetSceneCaptureConfig(BakeOptions);

	// TODO This block is similar to ConfigureSceneCapture function but handles an explicit Cameras array, when the BakeRC
	// tool also supports explicit Cameras then we can re-unify the code paths
	{
		ForEachCaptureType([&SceneCapture, &Options](ERenderCaptureType CaptureType)
		{
			const bool bCaptureTypeEnabled = Options.Flags[CaptureType];
			SceneCapture->SetCaptureTypeEnabled(CaptureType, bCaptureTypeEnabled);

			FRenderCaptureConfig Config;
			Config.bAntiAliasing = (CaptureType == ERenderCaptureType::DeviceDepth ? false : Options.bAntiAliasing);
			SceneCapture->SetCaptureConfig(CaptureType, Config);
		});

		SceneCapture->SetCaptureSceneActors(ValidSourceActors[0]->GetWorld(), ValidSourceActors);

		if (BakeOptions.Cameras.IsEmpty())
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs,
				LOCTEXT("BakeTextureFromRenderCaptures_EmptyCameraSet", "BakeTextureFromRenderCaptures: Cameras array was empty so a default camera set was used. Please update your blueprint by providing an explicit array e.g., by using ComputeRenderCaptureCamerasForBox"));
		
			TArray<FSpatialPhotoParams> SpatialParams = ComputeStandardExteriorSpatialPhotoParameters(
				nullptr,
				ValidSourceActors,
				TArray<UActorComponent*>(),
				FImageDimensions(Options.RenderCaptureImageSize, Options.RenderCaptureImageSize),
				Options.FieldOfViewDegrees,
				Options.NearPlaneDist,
				true, true, true, true, true);
			SceneCapture->SetSpatialPhotoParams(SpatialParams);
		}
		else
		{
			TArray<FSpatialPhotoParams> SpatialParams;
			for (const FGeometryScriptRenderCaptureCamera& Camera : BakeOptions.Cameras)
			{
				FSpatialPhotoParams Params;
				Params.NearPlaneDist = Camera.NearPlaneDist;
				Params.HorzFOVDegrees = Camera.FieldOfViewDegrees;
				Params.Dimensions = FImageDimensions(FMath::Max(1, Camera.Resolution), FMath::Max(1, Camera.Resolution));
				Params.Frame.AlignAxis(0, Camera.ViewDirection);
				Params.Frame.ConstrainedAlignAxis(2, FVector3d::UnitZ(), Params.Frame.X());
				Params.Frame.Origin = Camera.ViewPosition;
				SpatialParams.Add(Params);
			}
			SceneCapture->SetSpatialPhotoParams(SpatialParams);
		}
	}

	SceneCapture->Compute();

	const FDynamicMeshAABBTree3 TargetMeshSpatial(TargetMesh->GetMeshPtr());
	TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> TargetMeshTangents = MakeShared<FMeshTangentsd>(TargetMesh->GetMeshPtr());
	TargetMeshTangents->CopyTriVertexTangents(TargetMesh->GetMeshRef());
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> TargetMeshUVCharts = MakeShared<TArray<int32>>();
	FMeshMapBaker::ComputeUVCharts(TargetMesh->GetMeshRef(), *TargetMeshUVCharts);

	FSceneCapturePhotoSetSampler Sampler(
		SceneCapture.Get(),
		BakeOptions.CleanupTolerance,
		TargetMesh->GetMeshPtr(),
		&TargetMeshSpatial,
		TargetMeshTangents.Get());

	FRenderCaptureOcclusionHandler OcclusionHandler(GeometryScriptBakeLocals::GetDimensions(BakeOptions.Resolution));

	const FRenderCaptureTypeFlags PendingBake = Options.Flags; // All specified channels need baking
	TUniquePtr<FMeshMapBaker> Baker = MakeRenderCaptureBaker(
		TargetMesh->GetMeshPtr(),
		TargetMeshTangents,
		TargetMeshUVCharts,
		SceneCapture.Get(),
		&Sampler,
		PendingBake,
		TargetOptions.TargetUVLayer,
		GeometryScriptBakeLocals::ConvertEnum(BakeOptions.Resolution),
		GeometryScriptBakeLocals::ConvertEnum(BakeOptions.SamplesPerPixel),
		&OcclusionHandler);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshBakeFunctions_BakeTextureFromRenderCaptures_Bake);
		Baker->Bake();
	}

	FRenderCaptureTextures TexturesOut;
	GetTexturesFromRenderCaptureBaker(*Baker, TexturesOut);

	// Update source data
	if (TexturesOut.BaseColorMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.BaseColorMap, FTexture2DBuilder::ETextureType::Color);
	}
	if (TexturesOut.NormalMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.NormalMap, FTexture2DBuilder::ETextureType::NormalMap);
	}
	if (TexturesOut.PackedMRSMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.PackedMRSMap, FTexture2DBuilder::ETextureType::ColorLinear);
	}
	if (TexturesOut.MetallicMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.MetallicMap, FTexture2DBuilder::ETextureType::Metallic);
	}
	if (TexturesOut.RoughnessMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.RoughnessMap, FTexture2DBuilder::ETextureType::Roughness);
	}
	if (TexturesOut.SpecularMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.SpecularMap, FTexture2DBuilder::ETextureType::Specular);
	}
	if (TexturesOut.EmissiveMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.EmissiveMap, FTexture2DBuilder::ETextureType::EmissiveHDR);
	}
	if (TexturesOut.OpacityMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.OpacityMap, FTexture2DBuilder::ETextureType::ColorLinear);
	}
	if (TexturesOut.SubsurfaceColorMap)
	{
		FTexture2DBuilder::CopyPlatformDataToSourceData(TexturesOut.SubsurfaceColorMap, FTexture2DBuilder::ETextureType::Color);
	}

	FGeometryScriptRenderCaptureTextures Result;

	Result.BaseColorMap = TexturesOut.BaseColorMap;
	Result.NormalMap    = TexturesOut.NormalMap;
	Result.PackedMRSMap = TexturesOut.PackedMRSMap;
	Result.MetallicMap  = TexturesOut.MetallicMap;
	Result.RoughnessMap = TexturesOut.RoughnessMap;
	Result.SpecularMap  = TexturesOut.SpecularMap;
	Result.EmissiveMap  = TexturesOut.EmissiveMap;
	Result.OpacityMap   = TexturesOut.OpacityMap;
	Result.SubsurfaceColorMap  = TexturesOut.SubsurfaceColorMap;

	Result.bHasBaseColorMap = (Result.BaseColorMap != nullptr);
	Result.bHasNormalMap    = (Result.NormalMap    != nullptr);
	Result.bHasPackedMRSMap = (Result.PackedMRSMap != nullptr);
	Result.bHasMetallicMap  = (Result.MetallicMap  != nullptr);
	Result.bHasRoughnessMap = (Result.RoughnessMap != nullptr);
	Result.bHasSpecularMap  = (Result.SpecularMap  != nullptr);
	Result.bHasEmissiveMap  = (Result.EmissiveMap  != nullptr);
	Result.bHasOpacityMap   = (Result.OpacityMap   != nullptr);
	Result.bHasSubsurfaceColorMap  = (Result.SubsurfaceColorMap  != nullptr);

	return Result;
}


#undef LOCTEXT_NAMESPACE

