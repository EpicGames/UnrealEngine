// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshBasicEditFunctions.h"
#include "UDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "TransformSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBasicEditFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshBasicEditFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DiscardMeshAttributes(UDynamicMesh* TargetMesh, bool bDeferChangeNotifications)
{
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh.DiscardAttributes();
			EditMesh.DiscardVertexNormals();

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::SetVertexPosition(UDynamicMesh* TargetMesh, int32 VertexID, FVector NewPosition, bool& bIsValidVertex, bool bDeferChangeNotifications)
{
	bIsValidVertex = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsVertex(VertexID))
			{
				bIsValidVertex = true;
				EditMesh.SetVertex(VertexID, (FVector3d)NewPosition);
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::SetAllMeshVertexPositions(
	UDynamicMesh* TargetMesh,
	FGeometryScriptVectorList PositionList,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllMeshVertexPositions_InvalidMesh", "SetAllMeshVertexPositions: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PositionList.List.IsValid() == false || PositionList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllMeshVertexPositions_InvalidList", "SetAllMeshVertexPositions: List is empty"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		const TArray<FVector>& VertexPositions = *PositionList.List;
		if (VertexPositions.Num() < EditMesh.MaxVertexID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllMeshVertexPositions_IncorrectCount", "SetAllMeshVertexPositions: size of provided PositionList is smaller than MaxVertexID of Mesh"));
		}
		else
		{
			for (int32 VertexID : EditMesh.VertexIndicesItr())
			{
				EditMesh.SetVertex( VertexID, VertexPositions[VertexID] );
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AddVertexToMesh(
	UDynamicMesh* TargetMesh,
	FVector NewPosition,
	int32& NewVertexIndex,
	bool bDeferChangeNotifications)
{
	NewVertexIndex = INDEX_NONE;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			NewVertexIndex = EditMesh.AppendVertex(NewPosition);

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AddVerticesToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptVectorList NewPositionsList, 
	FGeometryScriptIndexList& NewIndicesList,
	bool bDeferChangeNotifications)
{
	if (NewPositionsList.List.IsValid() == false || NewPositionsList.List->Num() == 0)
	{
		return TargetMesh;
	}

	NewIndicesList.Reset(EGeometryScriptIndexType::Vertex);
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			for (FVector Position : *NewPositionsList.List)
			{
				int32 NewVertexIndex = EditMesh.AppendVertex(Position);
				NewIndicesList.List->Add(NewVertexIndex);
			}
		
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DeleteVertexFromMesh(
	UDynamicMesh* TargetMesh,
	int VertexID,
	bool& bWasVertexDeleted,
	bool bDeferChangeNotifications)
{
	bWasVertexDeleted = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EMeshResult Result = EditMesh.RemoveVertex(VertexID);
			bWasVertexDeleted = (Result == EMeshResult::Ok);

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DeleteVerticesFromMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptIndexList VertexList,
	int& NumDeleted,
	bool bDeferChangeNotifications)
{
	if (VertexList.List.IsValid() == false || VertexList.List->Num() == 0 || VertexList.IsCompatibleWith(EGeometryScriptIndexType::Vertex) == false)
	{
		return TargetMesh;
	}

	NumDeleted = 0;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			for (int32 VertexID : *VertexList.List)
			{
				EMeshResult Result = EditMesh.RemoveVertex(VertexID);
				if (Result == EMeshResult::Ok)
				{
					NumDeleted++;
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AddTriangleToMesh(
	UDynamicMesh* TargetMesh,
	FIntVector NewTriangle,
	int32& NewTriangleIndex,
	int32 NewTriangleGroupID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	NewTriangleIndex = INDEX_NONE;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			NewTriangleIndex = EditMesh.AppendTriangle((FIndex3i)NewTriangle, NewTriangleGroupID);

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

		if (NewTriangleIndex < 0)
		{
			if (NewTriangleIndex == FDynamicMesh3::NonManifoldID)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTriangleToMesh_NonManifold", "AddTriangleToMesh: Triangle cannot be added because it would create invalid Non-Manifold Mesh Topology"));
			}
			else if (NewTriangleIndex == FDynamicMesh3::DuplicateTriangleID)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTriangleToMesh_Duplicate", "AddTriangleToMesh: Triangle cannot be added because it is a duplicate of an existing Triangle"));
			}
			else
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AddTriangleToMesh_Unknown", "AddTriangleToMesh: adding Triangle Failed"));
			}
			NewTriangleIndex = INDEX_NONE;
		}
	}
	else
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTriangleToMesh_InvalidMesh", "AddTriangleToMesh: TargetMesh is Null"));
	}
	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AddTrianglesToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptTriangleList NewTrianglesList,
	FGeometryScriptIndexList& NewIndicesList,
	int32 NewTriangleGroupID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (NewTrianglesList.List.IsValid() == false || NewTrianglesList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTrianglesToMesh_InvalidList", "AddTrianglesToMesh: NewTrianglesList is empty"));
		return TargetMesh;
	}

	NewIndicesList.Reset(EGeometryScriptIndexType::Triangle);
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			for (FIntVector Triangle : *NewTrianglesList.List)
			{
				int32 NewTriangleIndex = EditMesh.AppendTriangle((FIndex3i)Triangle, NewTriangleGroupID);
				NewIndicesList.List->Add(NewTriangleIndex);
			}

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

		for (int32& NewTriangleIndex : *NewIndicesList.List)
		{
			if (NewTriangleIndex < 0)
			{
				if (NewTriangleIndex == FDynamicMesh3::NonManifoldID)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTrianglesToMesh_NonManifold", "AddTrianglesToMesh: Triangle cannot be added because it would create invalid Non-Manifold Mesh Topology"));
				}
				else if (NewTriangleIndex == FDynamicMesh3::DuplicateTriangleID)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTrianglesToMesh_Duplicate", "AddTrianglesToMesh: Triangle cannot be added because it is a duplicate of an existing Triangle"));
				}
				else
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AddTrianglesToMesh_Unknown", "AddTrianglesToMesh: adding Triangle Failed"));
				}
				NewTriangleIndex = INDEX_NONE;
			}
		}
	}
	else
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddTrianglesToMesh_InvalidMesh", "AddTriangleToMesh: TargetMesh is Null"));
	}
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DeleteTriangleFromMesh(
	UDynamicMesh* TargetMesh,
	int TriangleID,
	bool& bWasTriangleDeleted,
	bool bDeferChangeNotifications)
{
	bWasTriangleDeleted = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EMeshResult Result = EditMesh.RemoveTriangle(TriangleID);
			bWasTriangleDeleted = (Result == EMeshResult::Ok);

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DeleteTrianglesFromMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptIndexList TriangleList,
	int& NumDeleted,
	bool bDeferChangeNotifications)
{
	if (TriangleList.List.IsValid() == false || TriangleList.List->Num() == 0)
	{
		return TargetMesh;
	}

	NumDeleted = 0;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			for (int32 TriangleID : *TriangleList.List)
			{
				EMeshResult Result = EditMesh.RemoveTriangle(TriangleID);
				if (Result == EMeshResult::Ok)
				{
					NumDeleted++;
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::DeleteSelectedTrianglesFromMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	int& NumDeleted,
	bool bDeferChangeNotifications)
{
	if (TargetMesh && Selection.IsEmpty() == false )
	{
		NumDeleted = 0;
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			TArray<int32> Triangles;
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
			for (int32 TriangleID : Triangles)
			{
				EMeshResult Result = EditMesh.RemoveTriangle(TriangleID);
				if (Result == EMeshResult::Ok)
				{
					NumDeleted++;
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::MergeMeshVertexPair(
	UDynamicMesh* TargetMesh,
	int32 VertexKeep,
	int32 VertexDiscard,
	FGeometryScriptMergeVertexOptions Options,
	bool& bSuccess,
	double InterpParam,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	bSuccess = false;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MergeMeshVertexPair_InvalidInput", "MergeMeshVertexPair: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([VertexKeep, VertexDiscard, InterpParam, Options, &bSuccess](FDynamicMesh3& Mesh)
	{
		if (Options.bOnlyBoundary)
		{
			if (!Mesh.IsBoundaryVertex(VertexKeep) || !Mesh.IsBoundaryVertex(VertexDiscard))
			{
				return;
			}
		}

		FDynamicMesh3::FMergeVerticesOptions MergeOptions;
		MergeOptions.bAllowNonBoundaryBowtieCreation = Options.bAllowNonBoundaryBowties;
		FDynamicMesh3::FMergeVerticesInfo MergeInfo;
		EMeshResult Result = Mesh.MergeVertices(VertexKeep, VertexDiscard, InterpParam, MergeOptions, MergeInfo);
		bSuccess = Result == EMeshResult::Ok;
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::MergeMeshVerticesInSelections(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection SelectionKeep,
	FGeometryScriptMeshSelection SelectionDiscard,
	FGeometryScriptMergeVertexOptions Options,
	int& NumMerged,
	double InterpParam,
	double DistanceThreshold,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	NumMerged = 0;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MergeMeshVerticesInSelections_InvalidInput", "MergeMeshVerticesInSelections: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&SelectionKeep, &SelectionDiscard, InterpParam, Options, &NumMerged, DistanceThreshold](FDynamicMesh3& Mesh)
	{
		TSet<int32> VertSets[2];
		SelectionKeep.ProcessByVertexID(Mesh, [&Mesh, &VertSets, Options](int32 VID)
		{
			if (Options.bOnlyBoundary && !Mesh.IsBoundaryVertex(VID))
			{
				return;
			}
			VertSets[0].Add(VID);
		});
		SelectionDiscard.ProcessByVertexID(Mesh, [&Mesh, &SelectionKeep, &VertSets, Options](int32 VID)
		{
			if (Options.bOnlyBoundary && !Mesh.IsBoundaryVertex(VID))
			{
				return;
			}
			if (VertSets[0].Contains(VID))
			{
				return;
			}
			VertSets[1].Add(VID);
		});

		constexpr int32 MaxNumPasses = 2;
		for (int32 Pass = 0; Pass < MaxNumPasses; ++Pass)
		{
			// Number of closest vertex matches that we've discarded due to finding a better reverse match
			// Finding these discarded matches indicates that there may be additional matches to make if we run a second matching pass
			int32 DiscardedMatches = 0;
			// Mapping from DiscardVID to the best KeepVID match found so far
			TMap<int32, TPair<int32, double>> BestMatches;
			double ThreshSq = DistanceThreshold * DistanceThreshold;
			for (int32 KeepVID : VertSets[0])
			{
				int32 BestMatch = INDEX_NONE;
				double BestDistSq = ThreshSq;
				FVector3d KeepPos = Mesh.GetVertex(KeepVID);

				for (int32 DiscardVID : VertSets[1])
				{
					FVector3d DiscardPos = Mesh.GetVertex(DiscardVID);
					double DistSq = FVector3d::DistSquared(KeepPos, DiscardPos);
					if (DistSq < BestDistSq)
					{
						BestDistSq = DistSq;
						BestMatch = DiscardVID;
					}
				}

				if (BestMatch != INDEX_NONE)
				{
					TPair<int32, double>* PrevMatch = BestMatches.Find(BestMatch);
					if (PrevMatch)
					{
						// Two 'keep' vertices have the same 'discard' vertex as their best match
						// Use the best of these two matches, and track that we discarded the other match
						DiscardedMatches++;
						if (PrevMatch->Value > BestDistSq)
						{
							PrevMatch->Key = KeepVID;
							PrevMatch->Value = BestDistSq;
						}
					}
					else
					{
						BestMatches.Add(BestMatch, TPair<int32, double>(KeepVID, BestDistSq));
					}
				}
			}

			const bool bLastPass = Pass + 1 == MaxNumPasses || !DiscardedMatches;
			for (TPair<int32, TPair<int32, double>> Match : BestMatches)
			{
				int32 KeepVertex = Match.Value.Key;
				int32 DiscardVertex = Match.Key;
				FDynamicMesh3::FMergeVerticesOptions MergeOptions;
				MergeOptions.bAllowNonBoundaryBowtieCreation = Options.bAllowNonBoundaryBowties;
				FDynamicMesh3::FMergeVerticesInfo MergeInfo;
				EMeshResult Result = Mesh.MergeVertices(KeepVertex, DiscardVertex, InterpParam, MergeInfo);
				NumMerged += (int32)(Result == EMeshResult::Ok);
				if (!bLastPass)
				{
					VertSets[0].Remove(KeepVertex);
					VertSets[1].Remove(DiscardVertex);
				}
			}

			if (!DiscardedMatches)
			{
				break;
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}


void FGeometryScriptAppendMeshOptions::UpdateAttributesForCombineMode(FDynamicMesh3& Target, const FDynamicMesh3& Source)
{
	if (CombineMode == EGeometryScriptCombineAttributesMode::EnableAllMatching)
	{
		Target.EnableMatchingAttributes(Source, false, false);
	}
	else if (CombineMode == EGeometryScriptCombineAttributesMode::UseSource)
	{
		Target.EnableMatchingAttributes(Source, false, true);
	}
	// else the mode is UseTarget, which already corresponds to the default behavior for AppendMesh
}

namespace UE::Private::AppendHelpers
{
	static void AppendMaterials(const TArray<UMaterialInterface*>& TargetMeshMaterialList, const TArray<UMaterialInterface*>& AppendMeshMaterialList, bool bCompactAppendedMaterials,
		TArray<int32>& OutAppendedMaterialRemap, TArray<UMaterialInterface*>& OutMaterialList)
	{
		OutMaterialList = TargetMeshMaterialList;
		const int32 NumAppend = AppendMeshMaterialList.Num();
		OutAppendedMaterialRemap.Init(-1, NumAppend);

		for (int32 Idx = 0; Idx < NumAppend; ++Idx)
		{
			int32 FoundIdx;
			if (bCompactAppendedMaterials && TargetMeshMaterialList.Find(AppendMeshMaterialList[Idx], FoundIdx))
			{
				OutAppendedMaterialRemap[Idx] = FoundIdx;
			}
			else
			{
				OutAppendedMaterialRemap[Idx] = OutMaterialList.Add(AppendMeshMaterialList[Idx]);
			}
		}
	}

	static void ApplyMaterialRemap(const FMeshIndexMappings& Mappings, const TArray<int32>& AppendMaterialRemap, FDynamicMesh3& Mesh)
	{
		if (!Mesh.HasAttributes() || !Mesh.Attributes()->HasMaterialID())
		{
			return;
		}

		FDynamicMeshMaterialAttribute* MaterialAttrib = Mesh.Attributes()->GetMaterialID();
		for (const TPair<int32, int32>& MapTID : Mappings.GetTriangleMap().GetForwardMap())
		{
			int32 OrigMID = MaterialAttrib->GetValue(MapTID.Value);
			if (AppendMaterialRemap.IsValidIndex(OrigMID))
			{
				int32 RemapMID = AppendMaterialRemap[OrigMID];
				MaterialAttrib->SetValue(MapTID.Value, RemapMID);
			}
		}
	}

	static UDynamicMesh* AppendMesh(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* AppendMesh,
		TArray<int32>* AppendMaterialRemap,
		FTransform AppendTransform,
		bool bDeferChangeNotifications,
		FGeometryScriptAppendMeshOptions AppendOptions,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMesh_InvalidInput1", "AppendMesh: TargetMesh is Null"));
			return TargetMesh;
		}
		if (AppendMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMesh_InvalidInput2", "AppendMesh: AppendMesh is Null"));
			return TargetMesh;
		}

		TargetMesh->EditMesh([&AppendOptions, &AppendMaterialRemap, &AppendMesh, &AppendTransform](FDynamicMesh3& AppendToMesh)
		{
			AppendMesh->ProcessMesh([&AppendOptions, &AppendMaterialRemap, &AppendToMesh, &AppendTransform](const FDynamicMesh3& OtherMesh)
			{
				AppendOptions.UpdateAttributesForCombineMode(AppendToMesh, OtherMesh);
				FTransformSRT3d XForm(AppendTransform);
				FMeshIndexMappings TmpMappings;
				FDynamicMeshEditor Editor(&AppendToMesh);
				const FDynamicMesh3* UseOtherMesh = &OtherMesh;
				FDynamicMesh3 TmpMesh;
				if (UseOtherMesh == &AppendToMesh)
				{
					TmpMesh = OtherMesh;	// need  to make a copy if we are appending to ourself
					UseOtherMesh = &TmpMesh;
				}
				Editor.AppendMesh(UseOtherMesh, TmpMappings,
					[&XForm](int, const FVector3d& Position) { return XForm.TransformPosition(Position); },
					[&XForm](int, const FVector3d& Normal) { return XForm.TransformNormal(Normal); },
					XForm.GetDeterminant() < 0);
				if (AppendMaterialRemap)
				{
					ApplyMaterialRemap(TmpMappings, *AppendMaterialRemap, AppendToMesh);
				}
			});
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

		return TargetMesh;
	}

	static UDynamicMesh* AppendMeshTransformed(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* AppendMesh,
		TArray<int32>* AppendMaterialRemap,
		const TArray<FTransform>& AppendTransforms, 
		FTransform ConstantTransform,
		bool bConstantTransformIsRelative,
		bool bDeferChangeNotifications,
		FGeometryScriptAppendMeshOptions AppendOptions,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshTransformed_InvalidInput1", "AppendMeshTransformed: TargetMesh is Null"));
			return TargetMesh;
		}
		if (AppendMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshTransformed_InvalidInput2", "AppendMeshTransformed: AppendMesh is Null"));
			return TargetMesh;
		}
		if (AppendTransforms.IsEmpty())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshTransformed_NoTransforms", "AppendMeshTransformed: AppendTransforms array is empty"));
			return TargetMesh;
		}

		TargetMesh->EditMesh([&AppendOptions, &AppendMaterialRemap, &AppendMesh, &AppendTransforms, bConstantTransformIsRelative, &ConstantTransform](FDynamicMesh3& AppendToMesh)
		{
			AppendMesh->ProcessMesh([&AppendOptions, &AppendMaterialRemap, &AppendToMesh, &AppendTransforms, bConstantTransformIsRelative, &ConstantTransform](const FDynamicMesh3& OtherMesh)
			{
				AppendOptions.UpdateAttributesForCombineMode(AppendToMesh, OtherMesh);
				FMeshIndexMappings TmpMappings;
				FDynamicMeshEditor Editor(&AppendToMesh);
				const FDynamicMesh3* UseOtherMesh = &OtherMesh;
				FDynamicMesh3 TmpMesh;
				if (UseOtherMesh == &AppendToMesh)
				{
					TmpMesh = OtherMesh;	// need  to make a copy if we are appending to ourself
					UseOtherMesh = &TmpMesh;
				}
				for (FTransform AppendTransform : AppendTransforms)
				{
					FTransformSequence3d TransformSequence;

					if (bConstantTransformIsRelative)
					{
						TransformSequence.Append(ConstantTransform);
						TransformSequence.Append(AppendTransform);
					}
					else
					{
						// want to apply the constant transform's rotate/scale after
						// the main transform rotate/scale, so the main positioning 
						// translation has to be deferred until after that
						FVector Translation = AppendTransform.GetLocation();
						AppendTransform.SetTranslation(FVector::Zero());

						TransformSequence.Append(AppendTransform);
						TransformSequence.Append(ConstantTransform);
						TransformSequence.Append(FTransform(Translation));
					}

					Editor.AppendMesh(UseOtherMesh, TmpMappings,
						[&TransformSequence](int, const FVector3d& Position) { return TransformSequence.TransformPosition(Position); },
						[&TransformSequence](int, const FVector3d& Normal) { return TransformSequence.TransformNormal(Normal); },
						TransformSequence.WillInvert());
					if (AppendMaterialRemap)
					{
						ApplyMaterialRemap(TmpMappings, *AppendMaterialRemap, AppendToMesh);
					}
					TmpMappings.Reset();
				}
			});
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

		return TargetMesh;
	}

	static UDynamicMesh* AppendMeshRepeated(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* AppendMesh,
		TArray<int32>* AppendMaterialRemap,
		FTransform AppendTransform,
		int32 RepeatCount,
		bool bApplyTransformToFirstInstance,
		bool bDeferChangeNotifications,
		FGeometryScriptAppendMeshOptions AppendOptions,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshRepeated_InvalidInput1", "AppendMeshRepeated: TargetMesh is Null"));
			return TargetMesh;
		}
		if (AppendMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshRepeated_InvalidInput2", "AppendMeshRepeated: AppendMesh is Null"));
			return TargetMesh;
		}
		if (RepeatCount > 0)
		{
			FTransformSRT3d XForm(AppendTransform);
			FDynamicMesh3 TmpMesh;
			AppendMesh->ProcessMesh([&TmpMesh](const FDynamicMesh3& OtherMesh) { TmpMesh.Copy(OtherMesh); });
			if (AppendMaterialRemap && TmpMesh.HasAttributes() && TmpMesh.Attributes()->HasMaterialID())
			{
				FDynamicMeshMaterialAttribute* MaterialAttrib = TmpMesh.Attributes()->GetMaterialID();
				for (int32 TID : TmpMesh.TriangleIndicesItr())
				{
					int32 OrigMID = MaterialAttrib->GetValue(TID);
					if (AppendMaterialRemap->IsValidIndex(OrigMID))
					{
						int32 NewMID = (*AppendMaterialRemap)[OrigMID];
						MaterialAttrib->SetValue(TID, NewMID);
					}
				}
			}
			if (bApplyTransformToFirstInstance)
			{
				MeshTransforms::ApplyTransform(TmpMesh, XForm, true);
			}
			TargetMesh->EditMesh([&AppendOptions, &XForm, &TmpMesh, RepeatCount](FDynamicMesh3& AppendToMesh)
			{
				AppendOptions.UpdateAttributesForCombineMode(AppendToMesh, TmpMesh);
				FMeshIndexMappings TmpMappings;
				FDynamicMeshEditor Editor(&AppendToMesh);
				for (int32 k = 0; k < RepeatCount; ++k)
				{
					Editor.AppendMesh(&TmpMesh, TmpMappings);
					if (k < RepeatCount)
					{
						MeshTransforms::ApplyTransform(TmpMesh, XForm, true);
						TmpMappings.Reset();
					}
				}
			}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
		}

		return TargetMesh;
	}
}

UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMesh(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* AppendMesh,
	FTransform AppendTransform,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendHelpers::AppendMesh(TargetMesh, AppendMesh, nullptr, AppendTransform, bDeferChangeNotifications, AppendOptions, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMeshWithMaterials(
	UDynamicMesh* TargetMesh,
	const TArray<UMaterialInterface*>& TargetMeshMaterialList,
	UDynamicMesh* AppendMesh,
	const TArray<UMaterialInterface*>& AppendMeshMaterialList,
	TArray<UMaterialInterface*>& ResultMeshMaterialList,
	FTransform AppendTransform,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	bool bCompactAppendedMaterials,
	UGeometryScriptDebug* Debug)
{
	TArray<int32> AppendMaterialRemap;
	UE::Private::AppendHelpers::AppendMaterials(TargetMeshMaterialList, AppendMeshMaterialList, bCompactAppendedMaterials, AppendMaterialRemap, ResultMeshMaterialList);
	return UE::Private::AppendHelpers::AppendMesh(TargetMesh, AppendMesh, &AppendMaterialRemap, AppendTransform, bDeferChangeNotifications, AppendOptions, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMeshTransformed(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* AppendMesh,
	const TArray<FTransform>& AppendTransforms, 
	FTransform ConstantTransform,
	bool bConstantTransformIsRelative,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendHelpers::AppendMeshTransformed(TargetMesh, AppendMesh, nullptr, AppendTransforms, ConstantTransform, bConstantTransformIsRelative, bDeferChangeNotifications, AppendOptions, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMeshTransformedWithMaterials(
	UDynamicMesh* TargetMesh,
	const TArray<UMaterialInterface*>& TargetMeshMaterialList,
	UDynamicMesh* AppendMesh,
	const TArray<UMaterialInterface*>& AppendMeshMaterialList,
	TArray<UMaterialInterface*>& ResultMeshMaterialList,
	const TArray<FTransform>& AppendTransforms,
	FTransform ConstantTransform,
	bool bConstantTransformIsRelative,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	bool bCompactAppendedMaterials,
	UGeometryScriptDebug* Debug)
{
	TArray<int32> AppendMaterialRemap;
	UE::Private::AppendHelpers::AppendMaterials(TargetMeshMaterialList, AppendMeshMaterialList, bCompactAppendedMaterials, AppendMaterialRemap, ResultMeshMaterialList);
	return UE::Private::AppendHelpers::AppendMeshTransformed(TargetMesh, AppendMesh, &AppendMaterialRemap, AppendTransforms, ConstantTransform, bConstantTransformIsRelative, bDeferChangeNotifications, AppendOptions, Debug);
}



UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMeshRepeated(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* AppendMesh,
	FTransform AppendTransform,
	int32 RepeatCount,
	bool bApplyTransformToFirstInstance,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendHelpers::AppendMeshRepeated(TargetMesh, AppendMesh, nullptr, AppendTransform, RepeatCount, bApplyTransformToFirstInstance, bDeferChangeNotifications, AppendOptions, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendMeshRepeatedWithMaterials(
	UDynamicMesh* TargetMesh,
	const TArray<UMaterialInterface*>& TargetMeshMaterialList,
	UDynamicMesh* AppendMesh,
	const TArray<UMaterialInterface*>& AppendMeshMaterialList,
	TArray<UMaterialInterface*>& ResultMeshMaterialList,
	FTransform AppendTransform,
	int RepeatCount,
	bool bApplyTransformToFirstInstance,
	bool bDeferChangeNotifications,
	FGeometryScriptAppendMeshOptions AppendOptions,
	bool bCompactAppendedMaterials,
	UGeometryScriptDebug* Debug)
{
	TArray<int32> AppendMaterialRemap;
	UE::Private::AppendHelpers::AppendMaterials(TargetMeshMaterialList, AppendMeshMaterialList, bCompactAppendedMaterials, AppendMaterialRemap, ResultMeshMaterialList);
	return UE::Private::AppendHelpers::AppendMeshRepeated(TargetMesh, AppendMesh, &AppendMaterialRemap, AppendTransform, RepeatCount, bApplyTransformToFirstInstance, bDeferChangeNotifications, AppendOptions, Debug);
}




UDynamicMesh* UGeometryScriptLibrary_MeshBasicEditFunctions::AppendBuffersToMesh(
	UDynamicMesh* TargetMesh,
	const FGeometryScriptSimpleMeshBuffers& Buffers,
	FGeometryScriptIndexList& NewTriangleIndicesList,
	int MaterialID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	NewTriangleIndicesList.Reset(EGeometryScriptIndexType::Triangle);
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendMeshRepeated_InvalidInput1", "AppendMeshRepeated: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			EditMesh.EnableAttributes();
		}
		if (!EditMesh.HasTriangleGroups())
		{
			EditMesh.EnableTriangleGroups();
		}

		if (!EditMesh.Attributes()->HasMaterialID())
		{
			EditMesh.Attributes()->EnableMaterialID();
		}
		FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();

		TArray<int32> VertexIDMap;
		int32 NumVertices = Buffers.Vertices.Num();
		VertexIDMap.SetNum(NumVertices);
		for (int32 k = 0; k < NumVertices; ++k)
		{
			int32 NewVertexID = EditMesh.AppendVertex((FVector3d)Buffers.Vertices[k]);
			VertexIDMap[k] = NewVertexID;
		}

		auto MapTriangleFunc = [&VertexIDMap](FIntVector Triangle)
		{
			return FIndex3i(VertexIDMap[Triangle.X], VertexIDMap[Triangle.Y], VertexIDMap[Triangle.Z]);
		};

		TArray<int32>& NewTriangleIndices = *NewTriangleIndicesList.List;
		int32 NumTriangles = Buffers.Triangles.Num();
		NewTriangleIndices.Init(INDEX_NONE, NumTriangles);
		bool bHaveGroups = Buffers.TriGroupIDs.Num() == NumTriangles;
		int32 ConstantGroupID = EditMesh.AllocateTriangleGroup();
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			int32 UseGroupID = bHaveGroups ? Buffers.TriGroupIDs[k] : ConstantGroupID;
			const FIntVector& BufferTri = Buffers.Triangles[k];
			if (!VertexIDMap.IsValidIndex(BufferTri.X) || !VertexIDMap.IsValidIndex(BufferTri.Y) || !VertexIDMap.IsValidIndex(BufferTri.Z))
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendBuffersToMesh_Invalid", "AppendBuffersToMesh: Triangle cannot be added because it had invalid indices"));
				continue;
			}
			int32 NewTriangleID = EditMesh.AppendTriangle(MapTriangleFunc(Buffers.Triangles[k]), UseGroupID);
			if (NewTriangleID < 0)
			{
				if (NewTriangleID == FDynamicMesh3::NonManifoldID)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendBuffersToMesh_NonManifold", "AppendBuffersToMesh: Triangle cannot be added because it would create invalid Non-Manifold Mesh Topology"));
				}
				else if (NewTriangleID == FDynamicMesh3::DuplicateTriangleID)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendBuffersToMesh_Duplicate", "AppendBuffersToMesh: Triangle cannot be added because it is a duplicate of an existing Triangle"));
				}
				else
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AppendBuffersToMesh_Unknown", "AppendBuffersToMesh: adding Triangle Failed"));
				}
			}
			else
			{
				MaterialIDs->SetValue(NewTriangleID, MaterialID);
				NewTriangleIndices[k] = NewTriangleID;
			}
		}


		if (Buffers.Normals.Num() == NumVertices)
		{
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			VertexIDMap.SetNum(NumVertices);
			for (int32 k = 0; k < NumVertices; ++k)
			{
				int32 NewElementID = Normals->AppendElement((FVector3f)Buffers.Normals[k]);
				VertexIDMap[k] = NewElementID;
			}
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				if (NewTriangleIndices[k] >= 0)
				{
					Normals->SetTriangle(NewTriangleIndices[k], MapTriangleFunc(Buffers.Triangles[k]));
				}
			}
		}

		const TArray<FVector2D>* AllUVSets[8] = { &Buffers.UV0, &Buffers.UV1, &Buffers.UV2, &Buffers.UV3, &Buffers.UV4, &Buffers.UV5, &Buffers.UV6, &Buffers.UV7 };
		int32 NumUVLayers = 0;
		for (int32 k = 0; k < 8; ++k)
		{
			if (AllUVSets[k]->Num() != NumVertices)
			{
				break;
			}
			NumUVLayers++;
		}
		EditMesh.Attributes()->SetNumUVLayers(NumUVLayers);
		for (int32 li = 0; li < NumUVLayers; ++li)
		{
			FDynamicMeshUVOverlay* UVs = EditMesh.Attributes()->GetUVLayer(li);
			VertexIDMap.SetNum(NumVertices);
			for (int32 k = 0; k < NumVertices; ++k)
			{
				int32 NewElementID = UVs->AppendElement( (FVector2f) (*AllUVSets[li])[k] );
				VertexIDMap[k] = NewElementID;
			}
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				if (NewTriangleIndices[k] >= 0)
				{
					UVs->SetTriangle(NewTriangleIndices[k], MapTriangleFunc(Buffers.Triangles[k]));
				}
			}
		}

		if (Buffers.VertexColors.Num() == NumVertices)
		{
			EditMesh.Attributes()->EnablePrimaryColors();
			FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
			VertexIDMap.SetNum(NumVertices);
			for (int32 k = 0; k < NumVertices; ++k)
			{
				int32 NewElementID = Colors->AppendElement( ToVector4<float>(Buffers.VertexColors[k]) );
				VertexIDMap[k] = NewElementID;
			}
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				if (NewTriangleIndices[k] >= 0)
				{
					Colors->SetTriangle(NewTriangleIndices[k], MapTriangleFunc(Buffers.Triangles[k]));
				}
			}
		}


	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);




	return TargetMesh;
}





#undef LOCTEXT_NAMESPACE

