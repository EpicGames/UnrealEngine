// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshBoneWeightFunctions.h"

#include "Animation/Skeleton.h"
#include "BoneWeights.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "SkinningOps/SkinBindingOp.h"
#include "UDynamicMesh.h"
#include "Operations/TransferBoneWeights.h"
#include "MathUtil.h"
#include "Containers/Queue.h"
#include "DynamicMesh/MeshBones.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBoneWeightFunctions)

using namespace UE::Geometry;
using namespace UE::AnimationCore;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshBoneWeightFunctions"


template<typename ReturnType> 
ReturnType SimpleMeshBoneWeightQuery(
	UDynamicMesh* Mesh, FGeometryScriptBoneWeightProfile Profile, 
	bool& bIsValidBoneWeights, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh, const FDynamicMeshVertexSkinWeightsAttribute& SkinWeights)> QueryFunc)
{
	bIsValidBoneWeights = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (ReadMesh.HasAttributes())
			{
				if ( const FDynamicMeshVertexSkinWeightsAttribute* BoneWeights = ReadMesh.Attributes()->GetSkinWeightsAttribute(Profile.GetProfileName()) )
				{
					bIsValidBoneWeights = true;
					RetVal = QueryFunc(ReadMesh, *BoneWeights);
				}
			}
		});
	}
	return RetVal;
}


template<typename ReturnType> 
ReturnType SimpleMeshBoneWeightEdit(
	UDynamicMesh* Mesh, FGeometryScriptBoneWeightProfile Profile, 
	bool& bIsValidBoneWeights, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(FDynamicMesh3& Mesh, FDynamicMeshVertexSkinWeightsAttribute& SkinWeights)> EditFunc)
{
	bIsValidBoneWeights = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes())
			{
				if ( FDynamicMeshVertexSkinWeightsAttribute* BoneWeights = EditMesh.Attributes()->GetSkinWeightsAttribute(Profile.GetProfileName()) )
				{
					bIsValidBoneWeights = true;
					RetVal = EditFunc(EditMesh, *BoneWeights);
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	return RetVal;
}

/** Validate that the bone hierarchy stored on the mesh is a proper tree with a single root node and no cycles. */
static bool ValidateBoneHierarchy(
	const FDynamicMesh3& InMesh,
	UGeometryScriptDebug* InDebug
	)
{
	if (!InMesh.HasAttributes() || InMesh.Attributes()->GetNumBones() == 0)
	{
		AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ValidateBoneHierarchy_NoBones", "No bone attributes defined on the mesh."));
		return false;
	}

	const int32 NumBones = InMesh.Attributes()->GetNumBones();
	if (NumBones > 1 && !InMesh.Attributes()->GetBoneParentIndices())
	{
		AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ValidateBoneHierarchy_MultpleBonesNoParents", "Multiple bones defined but no parent indices present."));
		return false;
	}

	const FDynamicMeshBoneNameAttribute* NameAttrib = InMesh.Attributes()->GetBoneNames(); 
	const FDynamicMeshBoneParentIndexAttribute* ParentIndexAttrib = InMesh.Attributes()->GetBoneParentIndices(); 

	TSet<FName> BoneNamesSeen;
	
	int32 RootBoneIndex = INDEX_NONE;
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const FName BoneName = NameAttrib->GetValue(BoneIndex);
		
		if (BoneName == NAME_None)
		{
			AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("ValidateBoneHierarchy_UnnamedBone", "Bone at index {0} has no name."), FText::AsNumber(BoneIndex)));
			return false;
		}
		if (BoneNamesSeen.Contains(BoneName))
		{
			AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("ValidateBoneHierarchy_DuplicateBoneNames", "Bone '{0}' defined more than once."), FText::FromName(BoneName)));
			return false;
		}
		BoneNamesSeen.Add(BoneName);

		const int32 ParentBoneIndex = ParentIndexAttrib->GetValue(BoneIndex); 
		if (ParentBoneIndex == INDEX_NONE)
		{
			if (RootBoneIndex != INDEX_NONE)
			{
				AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, 
					FText::Format(LOCTEXT("ValidateBoneHierarchy_MultpleRootBones", "Multiple root bones found ('{0}' and '{1}')."),
						FText::FromName(BoneName), FText::FromName(NameAttrib->GetValue(RootBoneIndex))));
				return false;
			}

			RootBoneIndex = BoneIndex;
		}
		else if (ParentBoneIndex < 0 || ParentBoneIndex >= NumBones)
        {
            AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, 
             	FText::Format(LOCTEXT("ValidateBoneHierarchy_InvalidParentBoneIndex", "Parent bone index {0} for bone '{1}' is invalid ({2} bones defined)."),
             		FText::AsNumber(ParentBoneIndex), FText::FromName(NameAttrib->GetValue(BoneIndex)), FText::AsNumber(NumBones)));
            return false;
        }
	}

	// Once we've verified that all bones are properly named, all the parent indices are valid and there's only one root, let's check for cycles.
	TSet<int32> BoneIndicesVisited;
	
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		BoneIndicesVisited.Reset();

		int32 CurrentBoneIndex = BoneIndex;
		while (CurrentBoneIndex != RootBoneIndex)
		{
			// We store a set of all visited indices rather than just check if a bone cycles back onto itself, since the cycle could be up the hierarchy
			// and not cycle back on the starting bone.
			BoneIndicesVisited.Add(CurrentBoneIndex);
			
			const int32 ParentBoneIndex = ParentIndexAttrib->GetValue(CurrentBoneIndex);
			if (BoneIndicesVisited.Contains(ParentBoneIndex))
			{
				AppendError(InDebug, EGeometryScriptErrorType::InvalidInputs, 
					FText::Format(LOCTEXT("ValidateBoneHierarchy_FoundCycle", "Bone '{0}' does not connect up to the root bone '{1}' but connects into a cycle instead."),
						FText::FromName(NameAttrib->GetValue(CurrentBoneIndex)), FText::FromName(NameAttrib->GetValue(RootBoneIndex))));
				return false;
			}
			CurrentBoneIndex = ParentBoneIndex;
		}
	}

	return true;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::MeshHasBoneWeights(
	UDynamicMesh* TargetMesh,
	bool& bHasBoneWeights,
	FGeometryScriptBoneWeightProfile Profile)
{
	bool bHasBoneWeightProfile = false;
	bool bOK = SimpleMeshBoneWeightQuery<bool>(TargetMesh, Profile, bHasBoneWeightProfile, false,
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshVertexSkinWeightsAttribute& SkinWeights) { return true; });
	bHasBoneWeights = bHasBoneWeightProfile;
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::MeshCreateBoneWeights(
	UDynamicMesh* TargetMesh,
	bool& bProfileExisted,
	bool bReplaceExistingProfile,
	FGeometryScriptBoneWeightProfile Profile)
{
	bProfileExisted = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}

			FDynamicMeshVertexSkinWeightsAttribute *Attribute = EditMesh.Attributes()->GetSkinWeightsAttribute(Profile.GetProfileName());
			bProfileExisted = (Attribute != nullptr);
			if ( Attribute == nullptr || bReplaceExistingProfile)
			{
				if ( bReplaceExistingProfile && bProfileExisted )
				{
					EditMesh.Attributes()->RemoveSkinWeightsAttribute(Profile.GetProfileName());
				}

				Attribute = new FDynamicMeshVertexSkinWeightsAttribute(&EditMesh);
				EditMesh.Attributes()->AttachSkinWeightsAttribute(Profile.GetProfileName(), Attribute);
			}			
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::MeshCopyBoneWeights(
	UDynamicMesh* TargetMesh,
	bool& bProfileExisted,
	FGeometryScriptBoneWeightProfile TargetProfile,
	FGeometryScriptBoneWeightProfile SourceProfile
	)
{
	bProfileExisted = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (!EditMesh.HasAttributes())
			{
				return;
			}
			
			const FDynamicMeshVertexSkinWeightsAttribute* SourceAttribute = EditMesh.Attributes()->GetSkinWeightsAttribute(SourceProfile.GetProfileName());
			FDynamicMeshVertexSkinWeightsAttribute* TargetAttribute = EditMesh.Attributes()->GetSkinWeightsAttribute(TargetProfile.GetProfileName());
			if (!SourceAttribute || !TargetAttribute)
			{
				return;
			}

			TargetAttribute->Copy(*SourceAttribute);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetMaxBoneWeightIndex(
	UDynamicMesh* TargetMesh,
	bool& bHasBoneWeights,
	int& MaxBoneIndex,
	FGeometryScriptBoneWeightProfile Profile)
{
	MaxBoneIndex = -1;
	bool bHasBoneWeightProfile = false;
	bool bOK = SimpleMeshBoneWeightQuery<bool>(TargetMesh, Profile, bHasBoneWeightProfile, false,
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshVertexSkinWeightsAttribute& SkinWeights) 
		{ 
			for (int32 VertexID : Mesh.VertexIndicesItr())
			{
				FBoneWeights BoneWeights;
				SkinWeights.GetValue(VertexID, BoneWeights);
				int32 Num = BoneWeights.Num();
				for (int32 k = 0; k < Num; ++k)
				{
					MaxBoneIndex = FMathd::Max(MaxBoneIndex, BoneWeights[k].GetBoneIndex());
				}
			}
			return true;
		});
	bHasBoneWeights = bHasBoneWeightProfile;
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetVertexBoneWeights(
	UDynamicMesh* TargetMesh,
	int VertexID,
	TArray<FGeometryScriptBoneWeight>& BoneWeightsOut,
	bool& bHasValidBoneWeights,
	FGeometryScriptBoneWeightProfile Profile)
{
	bool bHasBoneWeightProfile = false;
	bHasValidBoneWeights = SimpleMeshBoneWeightQuery<bool>(TargetMesh, Profile, bHasBoneWeightProfile, false,
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshVertexSkinWeightsAttribute& SkinWeights)
	{
		if (Mesh.IsVertex(VertexID))
		{
			FBoneWeights BoneWeights;
			SkinWeights.GetValue(VertexID, BoneWeights);
			int32 Num = BoneWeights.Num();
			BoneWeightsOut.SetNum(Num);
			for (int32 k = 0; k < Num; ++k)
			{
				FGeometryScriptBoneWeight NewBoneWeight;
				NewBoneWeight.BoneIndex = BoneWeights[k].GetBoneIndex();
				NewBoneWeight.Weight = BoneWeights[k].GetWeight();
				BoneWeightsOut.Add(NewBoneWeight);
			}
		}
		return BoneWeightsOut.Num() > 0;
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetLargestVertexBoneWeight(
	UDynamicMesh* TargetMesh,
	int VertexID,
	FGeometryScriptBoneWeight& BoneWeight,
	bool& bHasValidBoneWeights,
	FGeometryScriptBoneWeightProfile Profile)
{
	bHasValidBoneWeights = false;
	bool bHasBoneWeightProfile = false;
	FBoneWeight FoundMax = SimpleMeshBoneWeightQuery<FBoneWeight>(TargetMesh, Profile, bHasBoneWeightProfile, FBoneWeight(),
	[&](const FDynamicMesh3& Mesh, const FDynamicMeshVertexSkinWeightsAttribute& SkinWeights) 
	{ 
		FBoneWeight MaxBoneWeight = FBoneWeight();
		if (Mesh.IsVertex(VertexID))
		{
			bHasValidBoneWeights = true;
			float MaxWeight = 0;
			FBoneWeights BoneWeights;
			SkinWeights.GetValue(VertexID, BoneWeights);
			int32 Num = BoneWeights.Num();
			for (int32 k = 0; k < Num; ++k)
			{
				const FBoneWeight& BoneWeight = BoneWeights[k];
				if (BoneWeight.GetWeight() > MaxWeight)
				{
					MaxWeight = BoneWeight.GetWeight();
					MaxBoneWeight = BoneWeight;
				}
			}
		}
		else
		{
			UE_LOGF(LogGeometry, Warning, "GetLargestMeshBoneWeight: VertexID %d does not exist", VertexID);
		}
		return MaxBoneWeight;
	});
	
	if (bHasValidBoneWeights)
	{
		BoneWeight.BoneIndex = FoundMax.GetBoneIndex();
		BoneWeight.Weight = FoundMax.GetWeight();
	}

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::SetVertexBoneWeights(
	UDynamicMesh* TargetMesh,
	int VertexID,
	const TArray<FGeometryScriptBoneWeight>& BoneWeights,
	bool& bHasValidBoneWeights,
	FGeometryScriptBoneWeightProfile Profile,
	UGeometryScriptDebug* Debug)
{
	bool bHasBoneWeightProfile = false;
	bHasValidBoneWeights = SimpleMeshBoneWeightEdit<bool>(TargetMesh, Profile, bHasBoneWeightProfile, false,
		[&](FDynamicMesh3& Mesh, FDynamicMeshVertexSkinWeightsAttribute& SkinWeights)
	{
		if (Mesh.IsVertex(VertexID))
		{
			int32 Num = BoneWeights.Num();
			TArray<FBoneWeight> NewWeightsList;
			for (int32 k = 0; k < Num; ++k)
			{
				if (BoneWeights[k].BoneIndex < 0)
				{
					AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetVertexBoneWeights_InvalidInput", "SetVertexBoneWeights: Invalid bone index provided; falling back to 0 as bone index."));
				}

				FBoneWeight NewWeight;
				NewWeight.SetBoneIndex(FMath::Max(0, BoneWeights[k].BoneIndex));
				NewWeight.SetWeight(BoneWeights[k].Weight);
				NewWeightsList.Add(NewWeight);
			}
			const FBoneWeights NewBoneWeights = FBoneWeights::Create(NewWeightsList);
			SkinWeights.SetValue(VertexID, NewBoneWeights);
			return true;
		}
		return false;
	});

	return TargetMesh;
}

void UGeometryScriptLibrary_MeshBoneWeightFunctions::BlendBoneWeights(
	const TArray<FGeometryScriptBoneWeight>& BoneWeightsA,
	const TArray<FGeometryScriptBoneWeight>& BoneWeightsB,
	float Alpha, 
	TArray<FGeometryScriptBoneWeight>& Result,
	UGeometryScriptDebug* Debug
	)
{
	TArray<FBoneWeight, TInlineAllocator<UE::AnimationCore::MaxInlineBoneWeightCount>> RawWeightsA;

	for (const FGeometryScriptBoneWeight& BoneWeight: BoneWeightsA)
	{
		
		if (BoneWeight.BoneIndex < 0)
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BlendBoneWeights_InvalidInput", "BlendBoneWeights: Invalid bone index provided; falling back to 0 as bone index."));
		}
		RawWeightsA.Add(FBoneWeight(FMath::Max(0, BoneWeight.BoneIndex), BoneWeight.Weight));
	}
	
	TArray<FBoneWeight, TInlineAllocator<UE::AnimationCore::MaxInlineBoneWeightCount>> RawWeightsB;
	for (const FGeometryScriptBoneWeight& BoneWeight: BoneWeightsB)
	{
		if (BoneWeight.BoneIndex < 0)
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BlendBoneWeights_InvalidInput", "BlendBoneWeights: Invalid bone index provided; falling back to 0 as bone index."));
		}
		RawWeightsB.Add(FBoneWeight(FMath::Max(0, BoneWeight.BoneIndex), BoneWeight.Weight));
	}
	
	FBoneWeights NewWeights = NewWeights.Blend(FBoneWeights::Create(RawWeightsA), FBoneWeights::Create(RawWeightsB), Alpha);
	
	Result.SetNum(NewWeights.Num());
	for (FBoneWeight BoneWeight: NewWeights)
	{
		Result.Emplace(BoneWeight.GetBoneIndex(), BoneWeight.GetWeight());
	}
}	


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::SetAllVertexBoneWeights(
	UDynamicMesh* TargetMesh,
	const TArray<FGeometryScriptBoneWeight>& BoneWeights, 
	FGeometryScriptBoneWeightProfile Profile,
	UGeometryScriptDebug* Debug
	)
{
	bool bHasBoneWeightProfile = false;
	const int32 Num = BoneWeights.Num();
	TArray<FBoneWeight> NewWeightsList;
	for (int32 k = 0; k < Num; ++k)
	{
		if (BoneWeights[k].BoneIndex < 0)
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllVertexBoneWeights_InvalidInput", "SetAllVertexBoneWeights: Invalid bone index provided; falling back to 0 as bone index."));
		}
		
		FBoneWeight NewWeight;
		NewWeight.SetBoneIndex(FMath::Max(0, BoneWeights[k].BoneIndex));
		NewWeight.SetWeight(BoneWeights[k].Weight);
		NewWeightsList.Add(NewWeight);
	}
	const FBoneWeights NewBoneWeights = FBoneWeights::Create(NewWeightsList);
	
	SimpleMeshBoneWeightEdit<bool>(TargetMesh, Profile, bHasBoneWeightProfile, false,
		[&](const FDynamicMesh3& Mesh, FDynamicMeshVertexSkinWeightsAttribute& SkinWeights)
	{
		for (const int32 VertexID : Mesh.VertexIndicesItr())
		{
			SkinWeights.SetValue(VertexID, NewBoneWeights);
		}
		return true;
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::PruneBoneWeights(
	UDynamicMesh* TargetMesh,
	const TArray<FName>& BonesToPrune, 
	FGeometryScriptPruneBoneWeightsOptions Options,
	FGeometryScriptBoneWeightProfile Profile,
	UGeometryScriptDebug* Debug
	)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PruneBoneWeights_InvalidInput", "PruneBoneWeights: TargetMesh is Null"));
		return TargetMesh;
	}
	
	// Nothing to do?
	if (BonesToPrune.IsEmpty())
	{
		return TargetMesh;
	}
	
	// Validate that we're not trying to prune the root bone.
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones() || EditMesh.Attributes()->GetNumBones() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PruneBoneWeights_NoBones", "Target mesh has no bone attribute"));
			return;
		}
		
		if (!ValidateBoneHierarchy(EditMesh, Debug))
		{
			return;
		}

		const TArray<FName>& BoneNames = EditMesh.Attributes()->GetBoneNames()->GetAttribValues();
		const TArray<int32>& BoneParents = EditMesh.Attributes()->GetBoneParentIndices()->GetAttribValues();

		TArray<int32> BoneIndicesToPrune;
		for (FName BoneName: BonesToPrune)
		{
			int32 BoneIndex = BoneNames.IndexOfByKey(BoneName);
			if (BoneIndex > 0)
			{
				BoneIndicesToPrune.Add(BoneIndex);
			}
			else if (!Options.bIgnoredInvalidBones)
			{
				if (BoneIndex == 0)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PruneBoneWeights_RootBoneInvalid", "Pruning the root bone is not allowed"));
					return;
				}
				else
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("PruneBoneWeights_InvalidBone", "Invalid bone '{0}'"), FText::FromName(BoneName)));
					return;
				}
			}
		}

		FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = EditMesh.Attributes()->GetSkinWeightsAttribute(Profile.GetProfileName());
		if (!SkinWeights)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("PruneBoneWeights_UnknownProfile", "Unknown skin weight profile '{0}'"), FText::FromName(Profile.GetProfileName())));
			return;
		}
		
		// Order the bones by descending tree depth.
		auto BoneDepth = [&BoneParents](int32 BoneIndex)
		{
			int32 Depth = 0;
			while (BoneParents[BoneIndex] != INDEX_NONE)
			{
				Depth++;
				BoneIndex = BoneParents[BoneIndex];
			}
			return Depth;
		};
		BoneIndicesToPrune.StableSort([BoneDepth](int32 BoneIndexA, int32 BoneIndexB)
		{
			return BoneDepth(BoneIndexA) > BoneDepth(BoneIndexB);
		});

		// Iteratively prune bones such that we properly propagate weights up the skeleton if multiple bones along the same
		// path are being removed.
		for (const int32 BoneIndex: BoneIndicesToPrune)
		{
			for (const int32 VertexID : EditMesh.VertexIndicesItr())
			{
				FBoneWeights BoneWeights;
				SkinWeights->GetValue(VertexID, BoneWeights);

				const int32 WeightIndex = BoneWeights.FindWeightIndexByBone(BoneIndex);
				if (WeightIndex == INDEX_NONE)
				{
					continue;
				}

				const int32 ParentBoneIndex = BoneParents[BoneIndex];
				checkSlow(ParentBoneIndex != INDEX_NONE);
				
				if (BoneWeights.Num() == 1)
				{
					// Is it the last remaining bone weight? Re-assign this vertex to the parent.
					BoneWeights = FBoneWeights::Create({FBoneWeight(ParentBoneIndex, MaxRawBoneWeight)});
				}
				else if (Options.ReassignmentType == EGeometryScriptPruneBoneWeightsAssignmentType::RenormalizeRemaining)
				{
					// Just remove the weight and renormalize what's remaining.
					BoneWeights.RemoveBoneWeight(BoneIndex);
				}
				else if (Options.ReassignmentType == EGeometryScriptPruneBoneWeightsAssignmentType::ReassignToParent)
				{
					FBoneWeightsSettings SettingsNoNormalize;
					SettingsNoNormalize.SetNormalizeType(EBoneWeightNormalizeType::None);
					
					FBoneWeight BoneWeight = BoneWeights[WeightIndex];

					// Remove the weight but don't normalize yet.
					BoneWeights.RemoveBoneWeight(BoneIndex, SettingsNoNormalize);

					// If the parent weight already exists, add the child weight to it.
					if (const int32 ParentWeightIndex = BoneWeights.FindWeightIndexByBone(ParentBoneIndex);
						ParentWeightIndex != INDEX_NONE)
					{
						FBoneWeight ParentBoneWeight = BoneWeights[ParentWeightIndex];
						BoneWeight.SetRawWeight(BoneWeight.GetRawWeight() + ParentBoneWeight.GetRawWeight());
					}

					// Set the weight to be the combination of the removed weight and the parent and renormalize now.
					BoneWeight.SetBoneIndex(ParentBoneIndex);
					BoneWeights.SetBoneWeight(BoneWeight);
				}

				SkinWeights->SetValue(VertexID, BoneWeights);
			}
		}
	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::ComputeSmoothBoneWeights(
	UDynamicMesh* TargetMesh,
	USkeleton* Skeleton, 
	FGeometryScriptSmoothBoneWeightsOptions Options, 
	FGeometryScriptBoneWeightProfile Profile,
	UGeometryScriptDebug* Debug
	)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeSmoothBoneWeights_InvalidInput", "ComputeSmoothBoneWeights: TargetMesh is Null"));
		return TargetMesh;
	}
	if (Skeleton == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeSmoothBoneWeights_InvalidSkeleton", "ComputeSmoothBoneWeights: Skeleton is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FSkinBindingOp SkinBindingOp;
		SkinBindingOp.OriginalMesh = MakeShared<FDynamicMesh3>(MoveTemp(EditMesh));
		SkinBindingOp.SetTransformHierarchyFromReferenceSkeleton(Skeleton->GetReferenceSkeleton());
		SkinBindingOp.ProfileName = Profile.ProfileName;
		switch(Options.DistanceWeighingType)
		{
		case EGeometryScriptSmoothBoneWeightsType::DirectDistance:
			SkinBindingOp.BindType = ESkinBindingType::DirectDistance;
			break;
		case EGeometryScriptSmoothBoneWeightsType::GeodesicVoxel:
			SkinBindingOp.BindType = ESkinBindingType::GeodesicVoxel;
			break;
		}
		SkinBindingOp.Stiffness = Options.Stiffness;
		SkinBindingOp.MaxInfluences = Options.MaxInfluences;
		SkinBindingOp.VoxelResolution = Options.VoxelResolution;

		SkinBindingOp.CalculateResult(nullptr);

		EditMesh = MoveTemp(*SkinBindingOp.ExtractResult().Release());
	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::TransferBoneWeightsFromMesh(
	UDynamicMesh* SourceMesh,
	UDynamicMesh* TargetMesh,
	FGeometryScriptTransferBoneWeightsOptions Options,
	FGeometryScriptMeshSelection InSelection,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (SourceMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferBoneWeightsFromMesh_InvalidSourceMesh", "TransferBoneWeightsFromMesh: Source Mesh is Null"));
		return TargetMesh;
	}
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferBoneWeightsFromMesh_InvalidTargetMesh", "TransferBoneWeightsFromMesh: Target Mesh is Null"));
		return TargetMesh;
	}

	SourceMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (!ReadMesh.HasAttributes() || !ReadMesh.Attributes()->HasBones())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferBoneWeightsFromMesh_NoBones", "Source Mesh has no bone attribute"));
			return;
		}
		if (ReadMesh.Attributes()->GetNumBones() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferBoneWeightsFromMesh_EmptyBones", "Source Mesh has an empty bone attribute"));
			return;
		}

		FTransferBoneWeights TransferBoneWeights(&ReadMesh, Options.SourceProfile.GetProfileName());
		TransferBoneWeights.TransferMethod = static_cast<FTransferBoneWeights::ETransferBoneWeightsMethod>(Options.TransferMethod);
		TransferBoneWeights.bUseParallel = true;

		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (!InSelection.IsEmpty())
			{
				InSelection.ConvertToMeshIndexArray(EditMesh, TransferBoneWeights.TargetVerticesSubset, EGeometryScriptIndexType::Vertex);
			}
			
			if (!EditMesh.HasAttributes())
			{
				EditMesh.EnableAttributes();
			}
			
			if (EditMesh.Attributes()->HasBones())
			{
				// If the TargetMesh has bone attributes, but we want to use the SourceMesh bone attributes, then we copy.
				// Otherwise, nothing to do, and we use the target mesh bone attributes.
				if (Options.OutputTargetMeshBones == EOutputTargetMeshBones::SourceBones)
				{
					EditMesh.Attributes()->CopyBoneAttributes(*ReadMesh.Attributes());
				}
			}
			else
			{
				// If the TargetMesh has no bone attributes, then we must use the SourceMesh bone attributes. Otherwise, throw an error.
				if (Options.OutputTargetMeshBones == EOutputTargetMeshBones::SourceBones)
				{
					EditMesh.Attributes()->CopyBoneAttributes(*ReadMesh.Attributes());
				}
				else 
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferBoneWeightsFromMesh_NoTargetMeshBones", "TransferBoneWeightsFromMesh: TargetMesh has no bone attributes but the OutputTargetMeshBones option is set to TargetBones"));
				}
			}
			
			if (Options.TransferMethod == ETransferBoneWeightsMethod::InpaintWeights)
			{
				TransferBoneWeights.NormalThreshold = FMathd::DegToRad * Options.NormalThreshold;
				TransferBoneWeights.SearchRadius = Options.RadiusPercentage * EditMesh.GetBounds().DiagonalLength();
				TransferBoneWeights.NumSmoothingIterations = Options.NumSmoothingIterations;
				TransferBoneWeights.SmoothingStrength = Options.SmoothingStrength;
				TransferBoneWeights.LayeredMeshSupport = Options.LayeredMeshSupport;
				TransferBoneWeights.ForceInpaintWeightMapName = Options.InpaintMask;
			}

			if (TransferBoneWeights.Validate() != EOperationValidationResult::Ok)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("TransferBoneWeightsFromMesh_ValidationFailed", "TransferBoneWeightsFromMesh: Invalid parameters were set for the transfer weight operator"));
				return;
			}
			if (!TransferBoneWeights.TransferWeightsToMesh(EditMesh, Options.TargetProfile.GetProfileName()))
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("TransferBoneWeightsFromMesh_TransferFailed", "TransferBoneWeightsFromMesh: Failed to transfer the weights"));
			}
			
		}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	});

	return TargetMesh;
}


static bool
GetBoneCopyHierarchyAndReindexMeshIfNeeded(
	TConstArrayView<FName> SourceMeshBoneNames,
	TConstArrayView<int32> SourceMeshBoneParentIndices,
	FDynamicMesh3& TargetMesh,
	TMap<FName, FName>& OutBoneHierarchy,
	FGeometryScriptCopyBonesFromMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	// We can only perform this operation if there are source bones present. 
	check(!SourceMeshBoneNames.IsEmpty());
	
	OutBoneHierarchy.Reset();

	// Check across all skin weight attributes which bones are bound to the mesh.
	if (Options.BonesToCopyFromSource != EBonesToCopyFromSource::AllBones)
	{
		// This should have been verified by the caller.
		check(SourceMeshBoneNames.Num() == SourceMeshBoneParentIndices.Num());
		
		if (!TargetMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("GetBoneCopyHierarchy_NoTargetBonesDefined", "Target mesh has no bone names defined which is needed when not copying all bones"));
			return false;
		}

		// Construct the actual bone hierarchy as a map of bone -> parent from the bone name and bone parent index lists.
		FName RootBone;
		
		for (int32 Index = 0; Index < SourceMeshBoneNames.Num(); Index++)
		{
			const int32 BoneParentIndex = SourceMeshBoneParentIndices[Index];
			if (BoneParentIndex == INDEX_NONE || (BoneParentIndex >= 0 && BoneParentIndex < SourceMeshBoneNames.Num()))
			{
				OutBoneHierarchy.Add(SourceMeshBoneNames[Index], BoneParentIndex != INDEX_NONE ? SourceMeshBoneNames[BoneParentIndex] : NAME_None);

				if (BoneParentIndex == INDEX_NONE)
				{
					if (!RootBone.IsNone())
					{
						AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("GetBoneCopyHierarchy_MultipleRootBonesFound", "Found multiple root bones on source mesh"));
						return false;
					}
					RootBone = SourceMeshBoneNames[Index];
				}
			}
		}

		// Get the name of all bones that contribute to skin binding on all skin weight profiles.
		TSet<int32> BoundBoneIndices;
		for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : TargetMesh.Attributes()->GetSkinWeightsAttributes())
		{
			if (const FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = TargetMesh.Attributes()->GetSkinWeightsAttribute(AttribPair.Key))
			{
				BoundBoneIndices.Append(ToAttrib->GetBoundBoneIndices());
			}
		}

		const FDynamicMeshBoneNameAttribute* BoneNameTargetAttrib = TargetMesh.Attributes()->GetBoneNames();
		TSet<FName> BoundBones;
		for (int32 BoneIndex: BoundBoneIndices)
		{
			if (BoneIndex < 0 || BoneIndex >= BoneNameTargetAttrib->Num())
			{
				AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("GetBoneCopyHierarchy_InvalidBoneWeightIndex", "Invalid bone index found on mesh"));
				return false;
			}

			const FName BoneName = BoneNameTargetAttrib->GetValue(BoneIndex);
			if (BoneName.IsNone())
			{
				const FText Error = FText::Format(LOCTEXT("GetBoneCopyHierarchy_NoBoneName", "Target bone at index {0} has no name."), FText::AsNumber(BoneIndex));
				AppendError(Debug, EGeometryScriptErrorType::OperationFailed, Error);
				return false;
			}
			if (!OutBoneHierarchy.Contains(BoneName))
			{
				const FText Error = FText::Format(LOCTEXT("GetBoneCopyHierarchy_BoneNotFound", "Target bone '{0}' not found on source mesh."), FText::FromName(BoneName));
				AppendError(Debug, EGeometryScriptErrorType::OperationFailed, Error);
				return false;
			}
			BoundBones.Add(BoneName);
		}

		// Go from each bound bone, and add parent bones to root, optionally skipping over unbound bones, if using OnlyBoundAndRoot.
		TSet<FName> UsedBones;
		UsedBones.Add(RootBone);
		
		for (FName BoneName: BoundBones)
		{
			UsedBones.Add(BoneName);

			// Traverse up to the root bone. If OnlyBoundAndRoot is set, then each bone's parent is set to either a bone further up
			// in the hierarchy that's actually bound, or the root.
			FName ParentBone = OutBoneHierarchy[BoneName];
			while (ParentBone != RootBone)
			{
				if (Options.BonesToCopyFromSource == EBonesToCopyFromSource::OnlyBoundAndParents)
				{
					UsedBones.Add(ParentBone);
				}
				else if (Options.BonesToCopyFromSource == EBonesToCopyFromSource::OnlyBoundAndRoot && BoundBones.Contains(ParentBone))
				{
					// We found another bound bone, make this our new parent.
					break;
				}
				
				ParentBone = OutBoneHierarchy[ParentBone];
			}

			if (Options.BonesToCopyFromSource == EBonesToCopyFromSource::OnlyBoundAndRoot)
			{
				OutBoneHierarchy[BoneName] = ParentBone;
			}
		}

		// Leave only used bones in the hierarchy.
		OutBoneHierarchy = OutBoneHierarchy.FilterByPredicate([&UsedBones](const TPair<FName, FName>& Item)
		{
			return UsedBones.Contains(Item.Key);
		});
	}

	if (Options.ReindexWeights)
	{
		if (TargetMesh.Attributes()->HasBones())
		{
			TArray<FName> ToBones(SourceMeshBoneNames);
			if (!OutBoneHierarchy.IsEmpty())
			{
				// Remove all bones that are not in the hierarchy.
				ToBones.RemoveAll([&OutBoneHierarchy](FName BoneName)
				{
					return !OutBoneHierarchy.Contains(BoneName);
				});
			}
			
			for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : TargetMesh.Attributes()->GetSkinWeightsAttributes())
			{
				if (FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = TargetMesh.Attributes()->GetSkinWeightsAttribute(AttribPair.Key))
				{
					if (!ToAttrib->ReindexBoneIndicesToSkeleton(TargetMesh.Attributes()->GetBoneNames()->GetAttribValues(), ToBones))
					{
						const FText Error = FText::Format(LOCTEXT("GetBoneCopyHierarchy_FailedToReindexWeights", "Failed to reindex bone weights for {0} weights profile"), FText::FromName(AttribPair.Key));
						AppendError(Debug, EGeometryScriptErrorType::OperationFailed, Error);
						return false;
					}
				}
			}
		}
		else
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneCopyHierarchy_TargetMeshHasNoBones", "Bone weight re-indexing was requested but the target mesh has no skeleton data"));
		}
	}

	return true;
}
	

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::CopyBonesFromMesh(
	UDynamicMesh* SourceMesh, 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptCopyBonesFromMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (SourceMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromMesh_InvalidSourceMesh", "CopyBonesFromMesh: SourceMesh is Null"));
		return TargetMesh;
	}

	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromMesh_InvalidTargetMesh", "CopyBonesFromMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	SourceMesh->EditMesh([&](const FDynamicMesh3& ReadMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (!ReadMesh.HasAttributes() || !ReadMesh.Attributes()->HasBones())
			{
				AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromMesh_SourceMeshHasNoBones", "SourceMesh has no bone attributes"));
				return;
			}

			if (!ReadMesh.Attributes()->CheckBoneValidity(EValidityCheckFailMode::ReturnOnly))
			{
				AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromMesh_InvalidSourceMeshBones", "SourceMesh has invalid bone attributes"));
				return;
			}

			if (!EditMesh.HasAttributes())
			{
				EditMesh.EnableAttributes();
			}

			TConstArrayView<FName> SourceMeshBoneNames = ReadMesh.Attributes()->GetBoneNames()->GetAttribValues();
			TConstArrayView<int32> SourceMeshParentBoneIndices;
			
			// Check across all skin weight attributes which bones are bound to the mesh.
			if (Options.BonesToCopyFromSource != EBonesToCopyFromSource::AllBones)
			{
				// Ensure that all required attributes on the source are defined, that all names and parent indices are valid, and that the bone
				// hierarchy is consistent.
				if (!ValidateBoneHierarchy(ReadMesh, Debug))
				{
					return;
				}
		
				// Target bone name attribute's existence has already been verified above with a call to HasBones, the source has been verified through
				// ValidateBoneHierarchy.
				SourceMeshParentBoneIndices = ReadMesh.Attributes()->GetBoneParentIndices()->GetAttribValues();
			}

			TMap<FName, FName> BoneHierarchy;
			if (GetBoneCopyHierarchyAndReindexMeshIfNeeded(SourceMeshBoneNames, SourceMeshParentBoneIndices, EditMesh, BoneHierarchy, Options, Debug))
			{
				// If the bone hierarchy wasn't set up, then do a full copy.
				if (BoneHierarchy.IsEmpty())
				{
					EditMesh.Attributes()->CopyBoneAttributes(*ReadMesh.Attributes());
				}
				else
				{
					// Copy the bone attributes but only copy the ones in the remapping map and update the parent index as well.
					EditMesh.Attributes()->CopyBoneAttributesWithRemapping(*ReadMesh.Attributes(), BoneHierarchy);
				}
			}

		}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::CopyBonesFromSkeleton(
	USkeleton* SourceSkeleton,
	UDynamicMesh* TargetMesh,
	FGeometryScriptCopyBonesFromMeshOptions Options,
	UGeometryScriptDebug* Debug
	)
{
	if (SourceSkeleton == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromSkeleton_InvalidSourceSkeleton", "CopyBonesFromMesh: SourceSkeleton is Null"));
		return TargetMesh;
	}

	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyBonesFromSkeleton_InvalidTargetMesh", "CopyBonesFromMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			EditMesh.EnableAttributes();
		}

		TMap<FName, FName> BoneHierarchy;

		const FReferenceSkeleton& RefSkeleton = SourceSkeleton->GetReferenceSkeleton();
		const int32 NumSourceBones = RefSkeleton.GetRawBoneNum();
		TArray<FName> SourceSkeletonBoneNames;
		TArray<int32> SourceSkeletonParentBoneIndices;
		SourceSkeletonBoneNames.Reserve(NumSourceBones);
		SourceSkeletonParentBoneIndices.Reserve(NumSourceBones);
		
		for (const FMeshBoneInfo& BoneInfo: SourceSkeleton->GetReferenceSkeleton().GetRawRefBoneInfo())
		{
			SourceSkeletonBoneNames.Add(BoneInfo.Name);
			SourceSkeletonParentBoneIndices.Add(BoneInfo.ParentIndex);
		}
		
		if (GetBoneCopyHierarchyAndReindexMeshIfNeeded(SourceSkeletonBoneNames, SourceSkeletonParentBoneIndices, EditMesh, BoneHierarchy, Options, Debug))
		{
			EditMesh.Attributes()->EnableBones(BoneHierarchy.IsEmpty() ? NumSourceBones : BoneHierarchy.Num());

			FDynamicMeshBoneNameAttribute* BoneNameAttrib = EditMesh.Attributes()->GetBoneNames();
			FDynamicMeshBoneParentIndexAttribute* BoneParentIndexAttrib = EditMesh.Attributes()->GetBoneParentIndices();
			FDynamicMeshBonePoseAttribute* BonePoses = EditMesh.Attributes()->GetBonePoses();
			
			const TArray<FTransform>& SourceSkeletonBonePoses = RefSkeleton.GetRawRefBonePose();

			TMap<FName, int32> NameToIndexMap;

			int32 NumTargetBones = 0;
			for (int32 BoneIndex = 0; BoneIndex < NumSourceBones; BoneIndex++)
			{
				const FName BoneName = SourceSkeletonBoneNames[BoneIndex];
				if (BoneHierarchy.IsEmpty() || BoneHierarchy.Contains(BoneName))
				{
					NameToIndexMap.Add(BoneName, NumTargetBones);
					BoneNameAttrib->SetValue(NumTargetBones, BoneName);
					BonePoses->SetValue(NumTargetBones, SourceSkeletonBonePoses[BoneIndex]);
					NumTargetBones++;
				}
			}

			// Now that we have all bone names and their new indices, remap the parent indices to them.
			// The target list should already contain all parents listed in the BoneHierarchy map.
			if (BoneHierarchy.IsEmpty())
			{
				for (int32 BoneIndex = 0; BoneIndex < NumSourceBones; BoneIndex++)
				{
					BoneParentIndexAttrib->SetValue(BoneIndex, SourceSkeletonParentBoneIndices[BoneIndex]);
				}
			}
			else
			{
				// Add a marker for the root bone, so we don't need to special case it below.
				NameToIndexMap.Add(NAME_None, INDEX_NONE);
				
				for (const TPair<FName, FName>& BoneItem: BoneHierarchy)
				{
					const int32 BoneIndex = NameToIndexMap[BoneItem.Key];
					const int32 BoneParentIndex = NameToIndexMap[BoneItem.Value];

					BoneParentIndexAttrib->SetValue(BoneIndex, BoneParentIndex);
				}
			}
		}

	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::DiscardBonesFromMesh(
	UDynamicMesh* TargetMesh, 
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DiscardBonesFromMesh_InvalidTargetMesh", "DiscardBonesFromMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes())
		{
			EditMesh.Attributes()->DisableBones();
		}
		
	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetBoneIndex(
	UDynamicMesh* TargetMesh,
	FName BoneName,
	bool& bIsValidBoneName,
	int& BoneIndex,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneIndex_InvalidTargetMesh", "GetBoneIndex: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{	
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneIndex_TargetMeshHasNoBones", "GetBoneIndex: TargetMesh has no bone attributes"));
			return;
		}

		// INDEX_NONE if BoneName doesn't exist in the bone names attribute
		BoneIndex = EditMesh.Attributes()->GetBoneNames()->GetAttribValues().Find(BoneName);
		
		bIsValidBoneName = BoneIndex != INDEX_NONE;

	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetRootBoneName(
	UDynamicMesh* TargetMesh,
	FName& BoneName,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetRootBoneName_InvalidTargetMesh", "GetRootBoneName: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{	
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetRootBoneName_TargetMeshHasNoBones", "GetRootBoneName: TargetMesh has no bone attributes"));
			return;
		}

		if (EditMesh.Attributes()->GetBoneNames()->GetAttribValues().IsEmpty())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetRootBoneName_TargetMeshHasEmptySkeleton", "GetRootBoneName: TargetMesh has bone attributes set, but they are empty (doesn't contain a single bone)"));
			return;
		}

		// Root bone's parent will be INDEX_NONE
		const int32 BoneIdx = EditMesh.Attributes()->GetBoneParentIndices()->GetAttribValues().Find(INDEX_NONE);

		if (BoneIdx == INDEX_NONE)
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetRootBoneName_TargetMeshHasNoRootBone", "GetRootBoneName: TargetMesh has no root bone"));
			return;
		}

		BoneName = EditMesh.Attributes()->GetBoneNames()->GetAttribValues()[BoneIdx];
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetBoneChildren(
	UDynamicMesh* TargetMesh,
	FName BoneName,
	bool bRecursive,
	bool& bIsValidBoneName,
	TArray<FGeometryScriptBoneInfo>& ChildrenInfo,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneChildren_InvalidTargetMesh", "GetBoneChildren: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{	
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneChildren_TargetMeshHasNoBones", "GetBoneChildren: TargetMesh has no bone attributes"));
			return;
		}
		
		const int32 NumBones = EditMesh.Attributes()->GetNumBones();
		const TArray<FName>& NamesAttrib = EditMesh.Attributes()->GetBoneNames()->GetAttribValues();
		const TArray<int32>& ParentsAttrib = EditMesh.Attributes()->GetBoneParentIndices()->GetAttribValues();
		const TArray<FTransform>& TransformsAttrib = EditMesh.Attributes()->GetBonePoses()->GetAttribValues();
		const TArray<FVector4f>& ColorsAttrib = EditMesh.Attributes()->GetBoneColors()->GetAttribValues();

		// INDEX_NONE if BoneName doesn't exist in the bone names attribute
		const int32 BoneIndex = NamesAttrib.Find(BoneName);
		
		bIsValidBoneName = BoneIndex != INDEX_NONE;

		if (bIsValidBoneName)
		{	
			ChildrenInfo.Reset();

			TArray<int32> ChildrenIndices;

			FMeshBones::GetBoneChildren(EditMesh, BoneIndex, ChildrenIndices, bRecursive);
			
			// Get all information about the children
			ChildrenInfo.SetNum(ChildrenIndices.Num());
			for (int32 Idx = 0; Idx < ChildrenIndices.Num(); ++Idx)
			{
				const int32 ChildIdx = ChildrenIndices[Idx];
				FGeometryScriptBoneInfo& Entry = ChildrenInfo[Idx];
				Entry.Index = ChildIdx;
				Entry.Name = NamesAttrib[ChildIdx];
				Entry.ParentIndex = ParentsAttrib[ChildIdx];
				Entry.LocalTransform = TransformsAttrib[ChildIdx];
				Entry.WorldTransform = Entry.LocalTransform;
				Entry.Color = ColorsAttrib[ChildIdx];

				int32 CurParentIndex = ParentsAttrib[ChildIdx];
				while (CurParentIndex != INDEX_NONE) // until we didn't reach the root bone
				{
					Entry.WorldTransform = Entry.WorldTransform * TransformsAttrib[CurParentIndex];
					CurParentIndex = ParentsAttrib[CurParentIndex];
				}
			}
		}
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetBoneInfo(
		UDynamicMesh* TargetMesh,
		FName BoneName,
		bool& bIsValidBoneName,
		FGeometryScriptBoneInfo& BoneInfo,
		UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneInfo_InvalidTargetMesh", "GetBoneInfo: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{	
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetBoneInfo_TargetMeshHasNoBones", "GetBoneInfo: TargetMesh has no bone attributes"));
			return;
		}
		
		
		// INDEX_NONE if BoneName doesn't exist in the bone names attribute
		int32 BoneIndex = EditMesh.Attributes()->GetBoneNames()->GetAttribValues().Find(BoneName);

		bIsValidBoneName = BoneIndex != INDEX_NONE;

		if (bIsValidBoneName)
		{	
			const TArray<FName>& NamesAttrib = EditMesh.Attributes()->GetBoneNames()->GetAttribValues();
			const TArray<int32>& ParentsAttrib = EditMesh.Attributes()->GetBoneParentIndices()->GetAttribValues();
			const TArray<FTransform>& TransformsAttrib = EditMesh.Attributes()->GetBonePoses()->GetAttribValues();
			const TArray<FVector4f>& ColorsAttrib = EditMesh.Attributes()->GetBoneColors()->GetAttribValues();

			BoneInfo.Index = BoneIndex;
			BoneInfo.Name = NamesAttrib[BoneIndex];
			BoneInfo.ParentIndex = ParentsAttrib[BoneIndex];
			BoneInfo.LocalTransform = TransformsAttrib[BoneIndex];
			BoneInfo.WorldTransform = BoneInfo.LocalTransform;
			BoneInfo.Color = ColorsAttrib[BoneIndex];

			int32 CurParentIndex = ParentsAttrib[BoneIndex];
			while (CurParentIndex != INDEX_NONE) // until we didn't reach the root bone
			{
				BoneInfo.WorldTransform = BoneInfo.WorldTransform * TransformsAttrib[CurParentIndex];
				CurParentIndex = ParentsAttrib[CurParentIndex];
			}
		}
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshBoneWeightFunctions::GetAllBonesInfo(
		UDynamicMesh* TargetMesh,
		TArray<FGeometryScriptBoneInfo>& BonesInfo,
		UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetAllBonesInfo_InvalidTargetMesh", "GetAllBonesInfo: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{	
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->HasBones())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetAllBonesInfo_TargetMeshHasNoBones", "GetAllBonesInfo: TargetMesh has no bone attributes"));
			return;
		}
		
		const int32 NumBones = EditMesh.Attributes()->GetNumBones();

		BonesInfo.SetNum(NumBones);

		const TArray<FName>& NamesAttrib = EditMesh.Attributes()->GetBoneNames()->GetAttribValues();
		const TArray<int32>& ParentsAttrib = EditMesh.Attributes()->GetBoneParentIndices()->GetAttribValues();
		const TArray<FTransform>& TransformsAttrib = EditMesh.Attributes()->GetBonePoses()->GetAttribValues();
		const TArray<FVector4f>& ColorsAttrib = EditMesh.Attributes()->GetBoneColors()->GetAttribValues();

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			FGeometryScriptBoneInfo& Entry = BonesInfo[BoneIndex];
			
			Entry.Index = BoneIndex;
			Entry.Name = NamesAttrib[BoneIndex];
			Entry.ParentIndex = ParentsAttrib[BoneIndex];
			Entry.LocalTransform = TransformsAttrib[BoneIndex];
			Entry.WorldTransform = Entry.LocalTransform;
			Entry.Color = ColorsAttrib[BoneIndex];

			int32 CurParentIndex = ParentsAttrib[BoneIndex];
			while (CurParentIndex != INDEX_NONE) // until we didn't reach the root bone
			{
				Entry.WorldTransform = Entry.WorldTransform * TransformsAttrib[CurParentIndex];
				CurParentIndex = ParentsAttrib[CurParentIndex];
			}
		}
	});

	return TargetMesh;
}

#undef LOCTEXT_NAMESPACE
