// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshBooleanFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "UDynamicMesh.h"

#include "Operations/MeshBoolean.h"
#include "Operations/MeshIsoCurves.h"
#include "Operations/MeshPlaneCut.h"
#include "Operations/MeshMirror.h"
#include "Operations/MeshSelfUnion.h"
#include "MeshSimplification.h"
#include "MeshBoundaryLoops.h"
#include "Operations/MinimalHoleFiller.h"
#include "ConstrainedDelaunay2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBooleanFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshBooleanFunctions"





UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshBoolean(
	UDynamicMesh* TargetMesh,
	FTransform TargetTransform,
	UDynamicMesh* ToolMesh,
	FTransform ToolTransform,
	EGeometryScriptBooleanOperation Operation,
	FGeometryScriptMeshBooleanOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshBoolean_InvalidInput1", "ApplyMeshBoolean: TargetMesh is Null"));
		return TargetMesh;
	}
	if (ToolMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshBoolean_InvalidInput2", "ApplyMeshBoolean: ToolMesh is Null"));
		return TargetMesh;
	}


	FMeshBoolean::EBooleanOp ApplyOperation = FMeshBoolean::EBooleanOp::Union;
	switch (Operation)
	{
	case EGeometryScriptBooleanOperation::Intersection: 
		ApplyOperation = FMeshBoolean::EBooleanOp::Intersect;
		break;
	case EGeometryScriptBooleanOperation::Subtract: 
		ApplyOperation = FMeshBoolean::EBooleanOp::Difference;
		break;	
	case EGeometryScriptBooleanOperation::TrimInside:
		ApplyOperation = FMeshBoolean::EBooleanOp::TrimInside;
		break;
	case EGeometryScriptBooleanOperation::TrimOutside:
		ApplyOperation = FMeshBoolean::EBooleanOp::TrimOutside;
		break;
	case EGeometryScriptBooleanOperation::NewPolyGroupInside:
		ApplyOperation = FMeshBoolean::EBooleanOp::NewGroupInside;
		break;
	case EGeometryScriptBooleanOperation::NewPolyGroupOutside:
		ApplyOperation = FMeshBoolean::EBooleanOp::NewGroupOutside;
		break;
	}

	FDynamicMesh3 NewResultMesh;
	bool bSuccess = false;
	TArray<int> NewBoundaryEdges;

	// TODO: can actually compute in place (ie without making a copy) if we pass the first mesh as the ResultMesh argument...
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh1)
	{
		ToolMesh->ProcessMesh([&](const FDynamicMesh3& Mesh2)
		{
			FMeshBoolean MeshBoolean(
				&Mesh1, (FTransformSRT3d)TargetTransform,
				&Mesh2, (FTransformSRT3d)ToolTransform,
				&NewResultMesh, ApplyOperation);
			MeshBoolean.bPutResultInInputSpace = true;
			MeshBoolean.bSimplifyAlongNewEdges = Options.bSimplifyOutput;
			bSuccess = MeshBoolean.Compute();
			NewBoundaryEdges = MoveTemp(MeshBoolean.CreatedBoundaryEdges);
		});
	});

	// Note: ignore bSuccess, as it comes back false even if we only had small errors...

	bool bFailDueToEmptyResult = (!Options.bAllowEmptyResult && NewResultMesh.TriangleCount() == 0);
	if (bFailDueToEmptyResult)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BooleanUnion_Failed", "BooleanUnion: Boolean operation failed due to an empty result; enable Allow Empty Result if empty results should be accepted."));
		return TargetMesh;
	}

	if (Options.OutputTransformSpace == EGeometryScriptBooleanOutputSpace::TargetTransformSpace)
	{
		// Put the result in the local space of the target mesh
		MeshTransforms::ApplyTransformInverse(NewResultMesh, (FTransformSRT3d)TargetTransform, true);
	}
	else if (Options.OutputTransformSpace == EGeometryScriptBooleanOutputSpace::ToolTransformSpace)
	{
		// Put the result in the local space of the tool mesh
		MeshTransforms::ApplyTransformInverse(NewResultMesh, (FTransformSRT3d)ToolTransform, true);
	}
	// Note EGeometryScriptBooleanOutputSpace::SharedTransformSpace does not need a transform

	if (NewBoundaryEdges.Num() > 0 && Options.bFillHoles)
	{
		FMeshBoundaryLoops OpenBoundary(&NewResultMesh, false);
		TSet<int> ConsiderEdges(NewBoundaryEdges);
		OpenBoundary.EdgeFilterFunc = [&ConsiderEdges](int EID)
		{
			return ConsiderEdges.Contains(EID);
		};
		OpenBoundary.Compute();

		for (FEdgeLoop& Loop : OpenBoundary.Loops)
		{
			FMinimalHoleFiller Filler(&NewResultMesh, Loop);
			Filler.Fill();
		}
	}

	TargetMesh->SetMesh(MoveTemp(NewResultMesh));

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshSelfUnion(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelfUnionOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshSelfUnion_InvalidInput", "ApplyMeshSelfUnion: TargetMesh is Null"));
		return TargetMesh;
	}

	// we emit multiple change events below...not clear how to avoid this though as
	// the hole-fill edit may not occur

	bool bSuccess = false;
	TArray<int> NewBoundaryEdges;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FMeshSelfUnion Union(&EditMesh);
		Union.WindingThreshold = FMath::Clamp(Options.WindingThreshold, 0.0f, 1.0f);
		Union.bTrimFlaps = Options.bTrimFlaps;
		Union.bSimplifyAlongNewEdges = Options.bSimplifyOutput;
		Union.SimplificationAngleTolerance = Options.SimplifyPlanarTolerance;
		bSuccess = Union.Compute();
		NewBoundaryEdges = MoveTemp(Union.CreatedBoundaryEdges);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	// currently ignoring bSuccess as it returns false in many cases
	//if (bSuccess == false)
	//{
	//	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BooleanUnion_Failed", "BooleanUnion: Boolean operation failed"));
	//	return TargetMesh;
	//}

	if (NewBoundaryEdges.Num() > 0 && Options.bFillHoles)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FMeshBoundaryLoops OpenBoundary(&EditMesh, false);
			TSet<int> ConsiderEdges(NewBoundaryEdges);
			OpenBoundary.EdgeFilterFunc = [&ConsiderEdges](int EID)
			{
				return ConsiderEdges.Contains(EID);
			};
			OpenBoundary.Compute();

			for (FEdgeLoop& Loop : OpenBoundary.Loops)
			{
				FMinimalHoleFiller Filler(&EditMesh, Loop);
				Filler.Fill();
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}

	return TargetMesh;
}







UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshPlaneCut(
	UDynamicMesh* TargetMesh,
	FTransform CutFrame,
	FGeometryScriptMeshPlaneCutOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshPlaneCut_InvalidInput", "ApplyMeshPlaneCut: TargetMesh is Null"));
		return TargetMesh;
	}

	FFrame3d UseCutFrame(CutFrame);
	FVector3d CutPlaneOrigin = UseCutFrame.Origin;
	FVector3d UseNormal = UseCutFrame.Z();
	if (Options.bFlipCutSide)
	{
		UseNormal = -UseNormal;
	}

	bool bSuccess = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FMeshPlaneCut Cut(&EditMesh, CutPlaneOrigin, UseNormal);
		Cut.UVScaleFactor = 1.0 / Options.UVWorldDimension;
		bSuccess = Cut.Cut();

		if (Options.bFillHoles)
		{
			Cut.HoleFill(ConstrainedDelaunayTriangulate<double>, Options.bFillSpans, -1, Options.HoleFillMaterialID);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshPlaneSlice(
	UDynamicMesh* TargetMesh,
	FTransform CutFrame,
	FGeometryScriptMeshPlaneSliceOptions Options,
	UGeometryScriptDebug* Debug)
{
	const FName ObjectIndexAttribute = "ObjectIndexAttribute";

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshPlaneSlice_InvalidInput", "ApplyMeshPlaneSlice: TargetMesh is Null"));
		return TargetMesh;
	}

	FFrame3d UseCutFrame(CutFrame);
	FVector3d CutPlaneOrigin = UseCutFrame.Origin;
	FVector3d UseNormal = UseCutFrame.Z();

	bool bSuccess = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		TDynamicMeshScalarTriangleAttribute<int>* SubObjectAttrib = new TDynamicMeshScalarTriangleAttribute<int>(&EditMesh);
		SubObjectAttrib->Initialize(0);
		EditMesh.Attributes()->AttachAttribute(ObjectIndexAttribute, SubObjectAttrib);

		int MaxSubObjectID = -1;
		for (int TID : EditMesh.TriangleIndicesItr())
		{
			MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectAttrib->GetValue(TID));
		}

		FMeshPlaneCut Cut(&EditMesh, CutPlaneOrigin, UseNormal);
		Cut.UVScaleFactor = 1.0 / Options.UVWorldDimension;

		Cut.PlaneOrigin -= Cut.PlaneNormal * Options.GapWidth;
		Cut.CutWithoutDelete(true, 0, SubObjectAttrib, MaxSubObjectID + 1, true, false);

		int SecondCutMaxID = MaxSubObjectID;
		for (int TID : EditMesh.TriangleIndicesItr())
		{
			SecondCutMaxID = FMath::Max(SecondCutMaxID, SubObjectAttrib->GetValue(TID));
		}

		Cut.PlaneOrigin += Cut.PlaneNormal * (2.0 * Options.GapWidth);
		Cut.CutWithoutDelete(true, 0, SubObjectAttrib, SecondCutMaxID + 1, false, true);

		for (int TID : EditMesh.TriangleIndicesItr())
		{
			int SubObjectID = SubObjectAttrib->GetValue(TID);
			if (SubObjectID > MaxSubObjectID && SubObjectID <= SecondCutMaxID)
			{
				EditMesh.RemoveTriangle(TID);
			}
		}

		if (Options.bFillHoles)
		{
			Cut.HoleFill(ConstrainedDelaunayTriangulate<double>, Options.bFillSpans, -1, Options.HoleFillMaterialID);
		}

		EditMesh.Attributes()->RemoveAttribute(ObjectIndexAttribute);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshMirror(
	UDynamicMesh* TargetMesh,
	FTransform CutFrame,
	FGeometryScriptMeshMirrorOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshMirror_InvalidInput", "ApplyMeshMirror: TargetMesh is Null"));
		return TargetMesh;
	}

	const double PlaneTolerance = FMathf::ZeroTolerance * 10.0;

	FFrame3d UseCutFrame(CutFrame);
	FVector3d MirrorPlaneOrigin = UseCutFrame.Origin;
	FVector3d UseNormal = UseCutFrame.Z();
	if (Options.bApplyPlaneCut && Options.bFlipCutSide)
	{
		UseNormal = -UseNormal;
	}

	bool bSuccess = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (Options.bApplyPlaneCut)
		{
			FMeshPlaneCut Cutter(&EditMesh, MirrorPlaneOrigin, UseNormal);
			Cutter.PlaneTolerance = PlaneTolerance;
			Cutter.Cut();
		}

		FMeshMirror Mirrorer(&EditMesh, MirrorPlaneOrigin, UseNormal);
		Mirrorer.bWeldAlongPlane = Options.bWeldAlongPlane;
		Mirrorer.bAllowBowtieVertexCreation = false;
		Mirrorer.PlaneTolerance = PlaneTolerance;

		Mirrorer.MirrorAndAppend(nullptr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshIsoCurves(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshIsoCurveOptions Options,
	const FGeometryScriptScalarList& PerVertexValues,
	float IsoValue,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshIsoCurves_InvalidInput", "ApplyMeshIsoCurves: TargetMesh is Null"));
		return TargetMesh;
	}
	if (!PerVertexValues.List.IsValid())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshIsoCurves_InvalidInput2", "ApplyMeshIsoCurves: Per Vertex Values list is Null"));
	}

	TargetMesh->EditMesh([&Options, &PerVertexValues, IsoValue, Debug](FDynamicMesh3& EditMesh)
	{
		if (PerVertexValues.List->Num() != EditMesh.MaxVertexID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshIsoCurves_ListWrongSize", "ApplyMeshIsoCurves: Per Vertex Values list has wrong length -- must match vertices"));
		}
		FMeshIsoCurves IsoCurves;
		IsoCurves.Settings.SnapToExistingVertexTol = Options.SnapToExistingVertexTolerance;
		IsoCurves.Settings.CurveIsoValueSnapTolerance = Options.CurveIsoValueSnapTolerance;

		IsoCurves.Cut(EditMesh, [&PerVertexValues](int32 VID) -> float
			{
				return (float)(*PerVertexValues.List)[VID];
			}, 
			IsoValue
		);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
