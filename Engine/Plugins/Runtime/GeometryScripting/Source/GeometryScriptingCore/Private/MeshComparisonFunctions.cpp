// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshComparisonFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "MeshQueries.h"
#include "UDynamicMesh.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshComparisonFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshComparisonFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshComparisonFunctions::IsSameMeshAs(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* OtherMesh,
	FGeometryScriptIsSameMeshOptions Options,
	bool &bIsSameMesh,
	FGeometryScriptMeshDifferenceInfo& DifferenceInfo,
	UGeometryScriptDebug* Debug)
{
	bIsSameMesh = false;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsSameMeshAs_InvalidInput", "IsSameMeshAs: TargetMesh is Null"));
		return TargetMesh;
	}
	if (OtherMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsSameMeshAs_InvalidInput2", "IsSameMeshAs: OtherMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		OtherMesh->ProcessMesh([&](const FDynamicMesh3& OtherMesh)
		{
			FDynamicMesh3::FSameAsOptions CompareOptions;
			FDynamicMesh3::FMeshDifferenceInfo Difference;
			CompareOptions.bCheckConnectivity = Options.bCheckConnectivity;
			CompareOptions.bCheckEdgeIDs = Options.bCheckEdgeIDs;
			CompareOptions.bCheckNormals = Options.bCheckNormals;
			CompareOptions.bCheckColors = Options.bCheckColors;
			CompareOptions.bCheckUVs = Options.bCheckUVs;
			CompareOptions.bCheckGroups = Options.bCheckGroups;
			CompareOptions.bCheckAttributes = Options.bCheckAttributes;
			CompareOptions.EpsilonFloat = Options.Epsilon;
			CompareOptions.EpsilonDouble = static_cast<double>(Options.Epsilon);
			bIsSameMesh = ReadMesh.IsSameAs(OtherMesh, CompareOptions, Difference);
			auto ConvertIDType = [](FDynamicMesh3::FMeshDifferenceInfo::EIDType IDType)
			{
				switch (IDType)
				{
				case FDynamicMesh3::FMeshDifferenceInfo::EIDType::Vertex:
					return EGeometryScriptIndexType::Vertex;
				case FDynamicMesh3::FMeshDifferenceInfo::EIDType::Triangle:
					return EGeometryScriptIndexType::Triangle;
				case FDynamicMesh3::FMeshDifferenceInfo::EIDType::Edge:
					return EGeometryScriptIndexType::Edge;
				}
				return EGeometryScriptIndexType::Any;
			};
			auto ConvertReason = [](FDynamicMesh3::FMeshDifferenceInfo::EReason Reason) -> EGeometryScriptMeshDifferenceReason
			{
				switch (Reason)
				{
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::VertexCount:
					return EGeometryScriptMeshDifferenceReason::VertexCount;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::TriangleCount:
					return EGeometryScriptMeshDifferenceReason::TriangleCount;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::EdgeCount:
					return EGeometryScriptMeshDifferenceReason::EdgeCount;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Vertex:
					return EGeometryScriptMeshDifferenceReason::Vertex;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Triangle:
					return EGeometryScriptMeshDifferenceReason::Triangle;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Edge:
					return EGeometryScriptMeshDifferenceReason::Edge;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Connectivity:
					return EGeometryScriptMeshDifferenceReason::Connectivity;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Normal:
					return EGeometryScriptMeshDifferenceReason::Normal;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Color:
					return EGeometryScriptMeshDifferenceReason::Color;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::UV:
					return EGeometryScriptMeshDifferenceReason::UV;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Group:
					return EGeometryScriptMeshDifferenceReason::Group;
				case FDynamicMesh3::FMeshDifferenceInfo::EReason::Attribute:
					return EGeometryScriptMeshDifferenceReason::Attribute;
				}
				return EGeometryScriptMeshDifferenceReason::Unknown;
			};
			if (!bIsSameMesh)
			{
				DifferenceInfo.ElementIDType = ConvertIDType(Difference.IDType);
				DifferenceInfo.Reason = ConvertReason(Difference.Reason);
				DifferenceInfo.Detail = MoveTemp(Difference.Detail);
				DifferenceInfo.TargetMeshElementID = Difference.ID;
				DifferenceInfo.OtherMeshElementID = Difference.OtherID;
			}
		});
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshComparisonFunctions::IsSameMeshAs(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* OtherMesh,
	FGeometryScriptIsSameMeshOptions Options,
	bool& bIsSameMesh,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptMeshDifferenceInfo DifferenceInfo_Unused;
	return IsSameMeshAs(TargetMesh, OtherMesh, Options, bIsSameMesh, DifferenceInfo_Unused, Debug);

}


UDynamicMesh* UGeometryScriptLibrary_MeshComparisonFunctions::IsIntersectingMesh(
	UDynamicMesh* TargetMesh,
	FTransform TargetTransformIn,
	UDynamicMesh* OtherMesh,
	FTransform OtherTransformIn,
	bool &bIsIntersecting,
	UGeometryScriptDebug* Debug)
{
	bIsIntersecting = false;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsIntersectingMesh_InvalidInput", "IsIntersectingMesh: TargetMesh is Null"));
		return TargetMesh;
	}
	if (OtherMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsIntersectingMesh_InvalidInput2", "IsIntersectingMesh: OtherMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh1) 
	{
		OtherMesh->ProcessMesh([&](const FDynamicMesh3& Mesh2)
		{
			FDynamicMeshAABBTree3 Spatials[2];
			ParallelFor(2, [&](int32 k) { Spatials[k].SetMesh((k == 0) ? &Mesh1 : &Mesh2, true); }, EParallelForFlags::Unbalanced);

			bool bIsIdentity1 = TargetTransformIn.Equals(FTransform::Identity, 0);
			FTransformSRT3d Transform1(TargetTransformIn);
			bool bIsIdentity2 = OtherTransformIn.Equals(FTransform::Identity, 0);
			FTransformSRT3d Transform2(OtherTransformIn);
			if (bIsIdentity1 && bIsIdentity2)
			{
				bIsIntersecting = Spatials[0].TestIntersection(Spatials[1]);
			} 
			else if (bIsIdentity1 || bIsIdentity2)
			{
				FIndex2i Indices = (bIsIdentity1) ? FIndex2i(0,1) : FIndex2i(1,0);
				FTransformSRT3d UseTransform = (bIsIdentity2) ? Transform1 : Transform2;
				bIsIntersecting = Spatials[Indices.A].TestIntersection(Spatials[Indices.B],
					[&](const FVector3d& Pos) { return UseTransform.TransformPosition(Pos); });
			}
			else
			{
				bIsIntersecting = Spatials[0].TestIntersection(Spatials[1],
					[&](const FVector3d& Pos) { return Transform1.InverseTransformPosition(Transform2.TransformPosition(Pos)); });
			}
		});
	});

	return TargetMesh;
}






UDynamicMesh* UGeometryScriptLibrary_MeshComparisonFunctions::MeasureDistancesBetweenMeshes(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* OtherMesh,
	FGeometryScriptMeasureMeshDistanceOptions Options,
	double& MaxDistance,
	double& MinDistance,
	double& AverageDistance,
	double& RootMeanSqrDeviation,
	UGeometryScriptDebug* Debug)
{
	MaxDistance = TNumericLimits<float>::Max();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MeasureDistancesBetweenMeshes_InvalidInput", "MeasureDistancesBetweenMeshes: TargetMesh is Null"));
		return TargetMesh;
	}
	if (OtherMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MeasureDistancesBetweenMeshes_InvalidInput2", "MeasureDistancesBetweenMeshes: OtherMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh1) 
	{
		OtherMesh->ProcessMesh([&](const FDynamicMesh3& Mesh2)
		{
			if (Options.bSymmetric)
			{
				FDynamicMeshAABBTree3 Spatials[2];
				ParallelFor(2, [&](int32 k) { Spatials[k].SetMesh((k == 0) ? &Mesh1 : &Mesh2, true); }, EParallelForFlags::Unbalanced);
				TMeshQueries<FDynamicMesh3>::MeshDistanceStatistics(
					Mesh1, Spatials[1], 
					&Mesh2, &Spatials[0], true,
					MaxDistance, MinDistance, AverageDistance, RootMeanSqrDeviation);
			}
			else
			{
				FDynamicMeshAABBTree3 Spatial2(&Mesh2, true);
				TMeshQueries<FDynamicMesh3>::MeshDistanceStatistics(
					Mesh1, Spatial2, 
					(const FDynamicMesh3*)nullptr, (const FDynamicMeshAABBTree3*)nullptr, false,
					MaxDistance, MinDistance, AverageDistance, RootMeanSqrDeviation);
			}
		});
	});

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
