// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshDecompositionFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshQueries.h"
#include "Selections/MeshConnectedComponents.h"
#include "VertexConnectedComponents.h"
#include "Polygroups/PolygroupSet.h"
#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"
#include "Util/BufferUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDecompositionFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshDecompositionFunctions"


static void BuildNewDynamicMeshes(UDynamicMesh* TargetMesh, UDynamicMeshPool* MeshPool, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes, TArray<UDynamicMesh*>& ComponentMeshes)
{
	ComponentMeshes.SetNum(0);

	int32 NumSplitMeshes = SplitMeshes.Num();
	if (NumSplitMeshes == 0)
	{
		UDynamicMesh* ComponentMesh = (MeshPool != nullptr) ?
			MeshPool->RequestMesh() : NewObject<UDynamicMesh>();
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
		{
			ComponentMesh->SetMesh(EditMesh);
		});
		ComponentMeshes.Add(ComponentMesh);
	}
	else
	{
		for (int32 mi = 0; mi < NumSplitMeshes; ++mi)
		{
			UDynamicMesh* ComponentMesh = (MeshPool != nullptr) ?
				MeshPool->RequestMesh() : NewObject<UDynamicMesh>();
			ComponentMesh->SetMesh(MoveTemp(*SplitMeshes[mi]));
			ComponentMeshes.Add(ComponentMesh);
		}
	}
}




UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::SplitMeshByComponents(  
	UDynamicMesh* TargetMesh, 
	TArray<UDynamicMesh*>& ComponentMeshes,
	UDynamicMeshPool* MeshPool,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshByComponents_InvalidInput", "SplitMeshByComponents: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		FMeshConnectedComponents Components(&EditMesh);
		Components.FindConnectedTriangles();
		int32 NumComponents = Components.Num();
		if (NumComponents <= 1)
		{
			return;		// for single-component case, BuildNewDynamicMeshes() will just copy TargetMesh
		}
		
		TArray<int32> TriSubmeshIndices;
		TriSubmeshIndices.SetNum(EditMesh.MaxTriangleID());
		for (int32 ci = 0; ci < NumComponents; ++ci)
		{
			for (int32 tid : Components.GetComponent(ci).Indices)
			{
				TriSubmeshIndices[tid] = ci;
			}
		}

		FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, false, [&](int32 tid) { return TriSubmeshIndices[tid]; });
	});

	BuildNewDynamicMeshes(TargetMesh, MeshPool, SplitMeshes, ComponentMeshes);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::SplitMeshByVertexOverlap(  
	UDynamicMesh* TargetMesh, 
	TArray<UDynamicMesh*>& ComponentMeshes,
	UDynamicMeshPool* MeshPool,
	double ConnectVerticesThreshold,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshByVertexOverlap_InvalidInput", "SplitMeshByVertexOverlap: TargetMesh is Null"));
		return TargetMesh;
	}
	
	double UseThreshold = FMath::Max(ConnectVerticesThreshold, UE_DOUBLE_KINDA_SMALL_NUMBER);

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;

	TargetMesh->ProcessMesh([&SplitMeshes, UseThreshold](const FDynamicMesh3& EditMesh) -> void
	{
		FVertexConnectedComponents Components(EditMesh.MaxVertexID());
		Components.ConnectTriangles(EditMesh);
		Components.ConnectCloseVertices(EditMesh, UseThreshold, 2);
		FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, false, [&Components, &EditMesh](int32 TID) -> int32
		{
			return Components.GetComponent(EditMesh.GetTriangle(TID).A);
		});
	});

	BuildNewDynamicMeshes(TargetMesh, MeshPool, SplitMeshes, ComponentMeshes);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::SplitMeshByMaterialIDs(  
	UDynamicMesh* TargetMesh, 
	TArray<UDynamicMesh*>& ComponentMeshes,
	TArray<int>& ComponentMaterialIDs,
	UDynamicMeshPool* MeshPool,
	UGeometryScriptDebug* Debug)
{
	ComponentMeshes.Reset();
	ComponentMaterialIDs.Reset();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshByComponents_InvalidInput", "SplitMeshByComponents: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		const FDynamicMeshMaterialAttribute* MaterialIDs =
			(EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID()) ? EditMesh.Attributes()->GetMaterialID() : nullptr;
		if (MaterialIDs == nullptr)
		{
			ComponentMaterialIDs.Add(0);
			return;
		}

		FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, false, [&MaterialIDs](int32 TID)
		{
			return MaterialIDs->GetValue(TID);
		}, INDEX_NONE, &ComponentMaterialIDs, true);

	});

	BuildNewDynamicMeshes(TargetMesh, MeshPool, SplitMeshes, ComponentMeshes);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::SplitMeshByPolygroups(  
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	TArray<UDynamicMesh*>& ComponentMeshes,
	TArray<int>& ComponentPolygroups,
	UDynamicMeshPool* MeshPool,
	UGeometryScriptDebug* Debug)
{
	ComponentMeshes.Reset();
	ComponentPolygroups.Reset();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshByPolygroups_InvalidInput", "SplitMeshByPolygroups: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		TUniquePtr<FPolygroupSet> SplitGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshByPolygroups_MissingGroups", "SplitMeshByPolygroups: Target Polygroup Layer does not exist"));
			return;
		}
		SplitGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);

		FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, false, [&SplitGroups](int32 TID)
		{
			return SplitGroups->GetGroup(TID);
		}, INDEX_NONE, &ComponentPolygroups, true);
	});

	BuildNewDynamicMeshes(TargetMesh, MeshPool, SplitMeshes, ComponentMeshes);

	return TargetMesh;
}

namespace UE::MeshDecompositionFunctionsLocals
{
	template<typename ValueType>
	TArray<ValueType> ValuePerMeshArrayHelper(TArray<UDynamicMesh*>& Meshes, TFunctionRef<ValueType(const FDynamicMesh3&)> MeshValueFn, ValueType DefaultValue = 0)
	{
		TArray<ValueType> Values;
		Values.SetNumUninitialized(Meshes.Num());
		for (int32 Idx = 0; Idx < Meshes.Num(); ++Idx)
		{
			ValueType MeshValue = DefaultValue;
			if (Meshes[Idx])
			{
				Meshes[Idx]->ProcessMesh([&MeshValue, &MeshValueFn](const FDynamicMesh3& Mesh) -> void
				{
					MeshValue = (ValueType)MeshValueFn(Mesh);
				});
			}
			Values[Idx] = MeshValue;
		}
		return Values;
	}
}


void UGeometryScriptLibrary_MeshDecompositionFunctions::
SortMeshesByVolume(
	UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
	bool bStableSort,
	EArraySortOrder SortOrder)
{
	using namespace UE::MeshDecompositionFunctionsLocals;

	// compute mesh volumes
	TArray<float> MeshVolumes = ValuePerMeshArrayHelper<float>(Meshes, [](const FDynamicMesh3& Mesh) -> float
		{
			return (float)TMeshQueries<FDynamicMesh3>::GetVolumeNonWatertight(Mesh);
		}, 0.0f);
	
	// sort by volumes
	BufferUtil::SortByValuesArray<UDynamicMesh*, float>(Meshes, MeshVolumes, bStableSort, SortOrder == EArraySortOrder::Ascending);
}

void UGeometryScriptLibrary_MeshDecompositionFunctions::
SortMeshesByArea(
	UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
	bool bStableSort,
	EArraySortOrder SortOrder)
{
	using namespace UE::MeshDecompositionFunctionsLocals;

	// compute mesh areas
	TArray<float> MeshAreas = ValuePerMeshArrayHelper<float>(Meshes, [](const FDynamicMesh3& Mesh) -> float
		{
			return (float)TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh).Y;
		}, 0.0f);

	// sort by areas
	BufferUtil::SortByValuesArray<UDynamicMesh*, float>(Meshes, MeshAreas, bStableSort, SortOrder == EArraySortOrder::Ascending);
}

void UGeometryScriptLibrary_MeshDecompositionFunctions::
SortMeshesByBoundsVolume(
	UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
	bool bStableSort,
	EArraySortOrder SortOrder)
{
	using namespace UE::MeshDecompositionFunctionsLocals;

	// compute mesh bounds volumes
	TArray<float> MeshBoundsVolumes = ValuePerMeshArrayHelper<float>(Meshes, [](const FDynamicMesh3& Mesh) -> float
		{
			return (float)Mesh.GetBounds().Volume();
		}, 0.0f);

	// sort by bounds volumes
	BufferUtil::SortByValuesArray<UDynamicMesh*, float>(Meshes, MeshBoundsVolumes, bStableSort, SortOrder == EArraySortOrder::Ascending);
}

void UGeometryScriptLibrary_MeshDecompositionFunctions::
SortMeshesByCustomValues(
	UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
	const TArray<float>& ValuesToSortBy,
	bool bStableSort,
	EArraySortOrder SortOrder)
{
	if (Meshes.Num() != ValuesToSortBy.Num())
	{
		UE::Geometry::MakeScriptError(EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SortMeshesByCustomValues_InvalidInput", "SortMeshesByCustomValues: Meshes and Values array must have same number of elements"));
		return;
	}
	BufferUtil::SortByValuesArray<UDynamicMesh*, float>(Meshes, ValuesToSortBy, bStableSort, SortOrder == EArraySortOrder::Ascending);
}


UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::GetSubMeshFromMesh(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* StoreToSubmesh, 
	FGeometryScriptIndexList TriangleList,
	UDynamicMesh*& StoreToSubmeshOut, 
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSubMeshFromMesh_InvalidInput", "GetSubMeshFromMesh: TargetMesh is Null"));
		return TargetMesh;
	}
	if (StoreToSubmesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSubMeshFromMesh_InvalidInput2", "GetSubMeshFromMesh: Submesh is Null"));
		return TargetMesh;
	}
	if (TriangleList.List.IsValid() == false || TriangleList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSubMeshFromMesh_InvalidList", "GetSubMeshFromMesh: TriangleList is empty"));
		return TargetMesh;
	}
	if (TriangleList.IsCompatibleWith(EGeometryScriptIndexType::Triangle) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetSubMeshFromMesh_InvalidList2", "GetSubMeshFromMesh: TriangleList has incompatible index type"));
		return TargetMesh;
	}

	FDynamicMesh3 Submesh;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes())
		{
			Submesh.EnableAttributes();
			Submesh.Attributes()->EnableMatchingAttributes(*EditMesh.Attributes());
		}

		FMeshIndexMappings Mappings;
		FDynamicMeshEditResult EditResult;
		FDynamicMeshEditor Editor(&Submesh);
		Editor.AppendTriangles(&EditMesh, *TriangleList.List, Mappings, EditResult);
	});

	StoreToSubmesh->SetMesh(MoveTemp(Submesh));
	StoreToSubmeshOut = StoreToSubmesh;

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::CopyMeshSelectionToMesh(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* StoreToSubmesh, 
	FGeometryScriptMeshSelection Selection,
	UDynamicMesh*& StoreToSubmeshOut, 
	bool bAppendToExisting,
	bool bPreserveGroupIDs,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshSelectionToMesh_InvalidInput", "CopyMeshSelectionToMesh: TargetMesh is Null"));
		return TargetMesh;
	}
	if (StoreToSubmesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshSelectionToMesh_InvalidInput2", "CopyMeshSelectionToMesh: StoreToSubmesh is Null"));
		return TargetMesh;
	}
	if (Selection.IsEmpty())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshSelectionToMesh_InvalidList", "CopyMeshSelectionToMesh: Selection is empty"));
		return TargetMesh;
	}

	FDynamicMesh3 Submesh;
	if (bAppendToExisting)
	{
		StoreToSubmesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			Submesh = ReadMesh;
		});
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		TArray<int32> Triangles;
		Selection.ConvertToMeshIndexArray(SourceMesh, Triangles, EGeometryScriptIndexType::Triangle);

		if (bAppendToExisting == false)
		{
			Submesh.Clear();
			Submesh.EnableMatchingAttributes(SourceMesh, false);
		}

		FMeshIndexMappings Mappings;
		FDynamicMeshEditResult EditResult;
		FDynamicMeshEditor Editor(&Submesh);
		Editor.AppendTriangles(&SourceMesh, Triangles, Mappings, EditResult, bPreserveGroupIDs);

		if (bPreserveGroupIDs)
		{
			for (int32 tid : EditResult.NewTriangles)
			{
				int32 GroupID = Submesh.GetTriangleGroup(tid);
				int32 OldGroupID = Mappings.GetGroupMap().GetFrom(GroupID);
				Submesh.SetTriangleGroup(tid, OldGroupID);
			}
		}

	});

	StoreToSubmesh->SetMesh(MoveTemp(Submesh));
	StoreToSubmeshOut = StoreToSubmesh;

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshDecompositionFunctions::CopyMeshToMesh(
	UDynamicMesh* CopyFromMesh,
	UDynamicMesh* CopyToMesh, 
	UDynamicMesh*& CopyToMeshOut, 
	UGeometryScriptDebug* Debug)
{
	if (CopyFromMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMesh_InvalidFirstInput", "CopyMeshToMesh: CopyFromMesh is Null"));
		return CopyFromMesh;
	}
	if (CopyToMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMesh_InvalidSecondInput", "CopyMeshToMesh: CopyToMesh is Null"));
		return CopyFromMesh;
	}

	FDynamicMesh3 MeshCopy;
	CopyFromMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		MeshCopy = EditMesh;
	});

	CopyToMesh->SetMesh(MoveTemp(MeshCopy));
	CopyToMeshOut = CopyToMesh;

	return CopyFromMesh;
}




#undef LOCTEXT_NAMESPACE
