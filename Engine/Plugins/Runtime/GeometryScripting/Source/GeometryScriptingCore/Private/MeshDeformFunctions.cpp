// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshDeformFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "EngineDefines.h"
#include "SpaceDeformerOps/BendMeshOp.h"
#include "SpaceDeformerOps/TwistMeshOp.h"
#include "SpaceDeformerOps/FlareMeshOp.h"
#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"

#include "AssetUtils/Texture2DUtil.h"
#include "Spatial/SampledScalarField2.h"

#include "Tessellation/AdaptiveDisplacement.h"
#include "Tessellation/AdaptiveTessellator.h"
#include "MeshConstraints.h"
#include "MeshConstraintsUtil.h"
#include "Image/DisplacementMap.h"
#include "Math/Bounds.h"
#include "Containers/BitArray.h"

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDeformFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshDeformFunctions"


namespace UE::MeshDeformFunctionsLocals
{
	void ComputePerlinNoiseRandomOffsets(int RandomSeed, FVector FrequencyShift, FVector3d (&OutOffsets)[3])
	{
		FRandomStream Random(RandomSeed);
		for (int k = 0; k < 3; ++k)
		{
			const float RandomOffset = 10000.0f * Random.GetFraction();
			OutOffsets[k] = FVector3d(RandomOffset, RandomOffset, RandomOffset);
			OutOffsets[k] += FrequencyShift;
		}

	}
}



UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyBendWarpToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptBendWarpOptions Options,
	FTransform BendOrientation,
	float BendAngle,
	float BendExtent,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyBendWarpToMesh_InvalidInput", "ApplyBendWarpToMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	FFrame3d WarpFrame = FFrame3d(BendOrientation);

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		// todo extract bend warp into standalone math code

		TSharedPtr<FDynamicMesh3> TmpMeshPtr = MakeShared<FDynamicMesh3>();
		*TmpMeshPtr = MoveTemp(EditMesh);

		FBendMeshOp BendOp;
		BendOp.OriginalMesh = TmpMeshPtr;
		BendOp.GizmoFrame = WarpFrame;
		BendOp.LowerBoundsInterval = (Options.bSymmetricExtents) ? -BendExtent : -Options.LowerExtent;
		BendOp.UpperBoundsInterval = BendExtent;
		BendOp.BendDegrees = BendAngle;
		BendOp.bLockBottom = !Options.bBidirectional;

		BendOp.CalculateResult(nullptr);

		TUniquePtr<FDynamicMesh3> NewResultMesh = BendOp.ExtractResult();
		FDynamicMesh3* NewResultMeshPtr = NewResultMesh.Release();
		EditMesh = MoveTemp(*NewResultMeshPtr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyTwistWarpToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptTwistWarpOptions Options,
	FTransform TwistOrientation,
	float TwistAngle,
	float TwistExtent,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyTwistWarpToMesh_InvalidInput", "ApplyTwistWarpToMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	FFrame3d WarpFrame = FFrame3d(TwistOrientation);

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		// todo extract Twist warp into standalone math code
		TSharedPtr<FDynamicMesh3> TmpMeshPtr = MakeShared<FDynamicMesh3>();
		*TmpMeshPtr = MoveTemp(EditMesh);

		FTwistMeshOp TwistOp;
		TwistOp.OriginalMesh = TmpMeshPtr;
		TwistOp.GizmoFrame = WarpFrame;
		TwistOp.LowerBoundsInterval = (Options.bSymmetricExtents) ? -TwistExtent : -Options.LowerExtent;
		TwistOp.UpperBoundsInterval = TwistExtent;
		TwistOp.TwistDegrees = TwistAngle;
		TwistOp.bLockBottom = !Options.bBidirectional;

		TwistOp.CalculateResult(nullptr);

		TUniquePtr<FDynamicMesh3> NewResultMesh = TwistOp.ExtractResult();
		FDynamicMesh3* NewResultMeshPtr = NewResultMesh.Release();
		EditMesh = MoveTemp(*NewResultMeshPtr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyFlareWarpToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptFlareWarpOptions Options,
	FTransform FlareOrientation,
	float FlarePercentX,
	float FlarePercentY,
	float FlareExtent,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyFlareWarpToMesh_InvalidInput", "ApplyFlareWarpToMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	FFrame3d WarpFrame = FFrame3d(FlareOrientation);

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		// todo extract Flare warp into standalone math code
		TSharedPtr<FDynamicMesh3> TmpMeshPtr = MakeShared<FDynamicMesh3>();
		*TmpMeshPtr = MoveTemp(EditMesh);

		FFlareMeshOp FlareOp;
		FlareOp.OriginalMesh = TmpMeshPtr;
		FlareOp.GizmoFrame = WarpFrame;
		FlareOp.LowerBoundsInterval = (Options.bSymmetricExtents) ? -FlareExtent : -Options.LowerExtent;
		FlareOp.UpperBoundsInterval = FlareExtent;
		FlareOp.FlarePercentX = FlarePercentX;
		FlareOp.FlarePercentY = FlarePercentY;
		if (Options.FlareType == EGeometryScriptFlareType::SinMode)
		{
			FlareOp.FlareType = FFlareMeshOp::EFlareType::SinFlare;
		}
		else if (Options.FlareType == EGeometryScriptFlareType::SinSquaredMode)
		{
			FlareOp.FlareType = FFlareMeshOp::EFlareType::SinSqrFlare;
		}
		else
		{
			FlareOp.FlareType = FFlareMeshOp::EFlareType::LinearFlare;
		}

		FlareOp.CalculateResult(nullptr);

		TUniquePtr<FDynamicMesh3> NewResultMesh = FlareOp.ExtractResult();
		FDynamicMesh3* NewResultMeshPtr = NewResultMesh.Release();
		EditMesh = MoveTemp(*NewResultMeshPtr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyMathWarpToMesh(
	UDynamicMesh* TargetMesh,
	FTransform WarpOrientation,
	EGeometryScriptMathWarpType WarpType,
	FGeometryScriptMathWarpOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyFlareWarpToMesh_InvalidInput", "ApplyFlareWarpToMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	FTransformSRT3d WarpTransform(WarpOrientation);
	double UseShift = (double)Options.FrequencyShift;
	double UseFrequency = (double)Options.Frequency;
	double UseMagnitude = (double)Options.Magnitude;

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		ParallelFor(EditMesh.MaxVertexID(), [&](int32 vid)
		{
			if (EditMesh.IsVertex(vid))
			{
				FVector3d Pos = EditMesh.GetVertex(vid);
				FVector3d LocalPos = WarpTransform.InverseTransformPosition(Pos);

				FVector3d Displacement = FVector3d::Zero();
				switch (WarpType)
				{
				case EGeometryScriptMathWarpType::SinWave1D:
					{
						LocalPos.Z += UseMagnitude * FMathd::Sin(UseFrequency * (LocalPos.X + UseShift));
					}
					break;
				case EGeometryScriptMathWarpType::SinWave2D:
					{
						double Radius = FVector2d(LocalPos.X, LocalPos.Y).Length();
						LocalPos.Z += UseMagnitude * FMathd::Sin(UseFrequency * (Radius + UseShift));
					}
					break;
				}

				Pos = WarpTransform.TransformPosition(LocalPos);
				EditMesh.SetVertex(vid, Pos);
			}
		});

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDynamicMesh*
UGeometryScriptLibrary_MeshDeformFunctions::ApplyPerlinNoiseToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptPerlinNoiseOptions Options,
	UGeometryScriptDebug* Debug)
{
	// The compatibility version squares the Frequency to replicate the previous incorrect behavior
	// and is otherwise the same as v2
	FGeometryScriptPerlinNoiseOptions CompatOptions = Options;
	CompatOptions.BaseLayer.Frequency *= CompatOptions.BaseLayer.Frequency;
	return ApplyPerlinNoiseToMesh2(TargetMesh, Selection, CompatOptions, Debug);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyPerlinNoiseToMesh2(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptPerlinNoiseOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPerlinNoiseToMesh_InvalidInput", "ApplyPerlinNoiseToMesh: TargetMesh is Null"));
		return TargetMesh;
	}
	if (Selection.IsEmpty() && Options.EmptyBehavior != EGeometryScriptEmptySelectionBehavior::FullMeshSelection )
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FVector3d Offsets[3];
		UE::MeshDeformFunctionsLocals::ComputePerlinNoiseRandomOffsets(Options.BaseLayer.RandomSeed, Options.BaseLayer.FrequencyShift, Offsets);

		FMeshNormals Normals(&EditMesh);
		bool bAlignWithNormal = Options.bApplyAlongNormal;
		if (bAlignWithNormal)
		{
			if (Options.NormalSource == EGeometryScriptPerVertexNormalSource::Computed)
			{
				Normals.ComputeVertexNormals();
			}
			else // EGeometryScriptPerVertexNormalSource::AverageFromOverlay
			{
				if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->PrimaryNormals())
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPerlinNoiseToMesh_MissingOverlayNormals", "ApplyPerlinNoiseToMesh: TargetMesh did not have overlay normals computed"));
					return;
				}
				Normals.GetVertexNormalsFromOverlayNormals();
			}
		}

		auto GetDisplacedPosition = [&EditMesh, &Options, &Offsets, &Normals, bAlignWithNormal](int32 VertexID)
		{
			FVector3d Pos = EditMesh.GetVertex(VertexID);
			if (bAlignWithNormal)
			{
				FVector NoisePos = (FVector)((double)Options.BaseLayer.Frequency * (Pos + Offsets[0]));
				float Displacement = Options.BaseLayer.Magnitude * FMath::PerlinNoise3D(NoisePos);
				Pos += Displacement * Normals[VertexID];
			}
			else
			{
				FVector3d Displacement;
				for (int32 k = 0; k < 3; ++k)
				{
					FVector NoisePos = (FVector)((double)Options.BaseLayer.Frequency * (Pos + Offsets[k]));
					Displacement[k] = Options.BaseLayer.Magnitude * FMath::PerlinNoise3D(NoisePos);
				}
				Pos += Displacement;
			}
			return Pos;
		};

		if (Selection.IsEmpty())
		{
			ParallelFor(EditMesh.MaxVertexID(), [&](int32 VertexID)
			{
				if (EditMesh.IsVertex(VertexID))
				{
					EditMesh.SetVertex(VertexID, GetDisplacedPosition(VertexID));
				}
			});
		}
		else
		{
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID)
			{
				EditMesh.SetVertex(VertexID, GetDisplacedPosition(VertexID));
			});
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


FGeometryScriptScalarList
UGeometryScriptLibrary_MeshDeformFunctions::ComputePerlinNoise(FGeometryScriptVectorList Positions, FGeometryScriptPerlinNoiseOptions Options)
{
	FGeometryScriptScalarList NoiseValues;
	NoiseValues.Reset();

	if (!Positions.List)
	{
		return NoiseValues;
	}

	FVector3d Offsets[3];
	UE::MeshDeformFunctionsLocals::ComputePerlinNoiseRandomOffsets(Options.BaseLayer.RandomSeed, Options.BaseLayer.FrequencyShift, Offsets);

	TArray<double>& NoiseArray = *NoiseValues.List;
	TArray<FVector>& PosArray = *Positions.List;
	NoiseArray.SetNumUninitialized(PosArray.Num());
	ParallelFor(PosArray.Num(), [&PosArray, &NoiseArray, &Options, &Offsets](int32 Idx)
		{
			FVector NoisePos = (FVector)((double)Options.BaseLayer.Frequency * (PosArray[Idx] + Offsets[0]));
			NoiseArray[Idx] = Options.BaseLayer.Magnitude * FMath::PerlinNoise3D(NoisePos);
		}
	);

	return NoiseValues;
}




UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyIterativeSmoothingToMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptIterativeMeshSmoothingOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyIterativeSmoothingToMesh_InvalidInput", "ApplyIterativeSmoothingToMesh: TargetMesh is Null"));
		return TargetMesh;
	}
	if (Selection.IsEmpty() && Options.EmptyBehavior != EGeometryScriptEmptySelectionBehavior::FullMeshSelection )
	{
		return TargetMesh;
	}

	double ClampAlpha = FMathd::Clamp((double)Options.Alpha, 0.0, 1.0);
	int ClampIters = FMath::Clamp(Options.NumIterations, 0, 100);
	if (ClampIters == 0 || ClampAlpha == 0.0)
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		int32 MaxVertices = EditMesh.MaxVertexID();
		TArray<FVector3d> SmoothPositions;
		SmoothPositions.SetNum(MaxVertices);

		if (Selection.GetNumSelected() > 0)
		{
			TArray<int32> Vertices;
			Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
			int32 NumVertices = Vertices.Num();
			for (int32 k = 0; k < ClampIters; ++k)
			{
				ParallelFor(NumVertices, [&](int32 i)
				{
					int32 VertexID = Vertices[i];
					FVector3d Centroid;
					EditMesh.GetVtxOneRingCentroid(VertexID, Centroid);
					SmoothPositions[VertexID] = Lerp(EditMesh.GetVertex(VertexID), Centroid, ClampAlpha);
				});
				for (int32 VertexID : Vertices)
				{
					EditMesh.SetVertex(VertexID, SmoothPositions[VertexID]);
				}
			}
		}
		else
		{
			for (int32 k = 0; k < ClampIters; ++k)
			{
				ParallelFor(MaxVertices, [&](int32 VertexID)
				{
					if (EditMesh.IsVertex(VertexID))
					{
						FVector3d Centroid;
						EditMesh.GetVtxOneRingCentroid(VertexID, Centroid);
						SmoothPositions[VertexID] = Lerp(EditMesh.GetVertex(VertexID), Centroid, ClampAlpha);
					}
				});
				for (int32 VertexID = 0; VertexID < MaxVertices; ++VertexID)
				{
					if (EditMesh.IsVertex(VertexID))
					{
						EditMesh.SetVertex(VertexID, SmoothPositions[VertexID]);
					}
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

namespace UGeometryScriptLibrary_MeshDeformFunctionsInternals
{

using namespace UE;
using namespace UE::Geometry;

void ApplyAdaptiveDisplacement(
	FDynamicMesh3& Mesh, 
	const FGeometryScriptMeshSelection& Selection,
	const FGeometryScriptDisplaceFromTextureOptions& Options,
	const FGeometryScriptAdaptiveTessellationOptions& TessellationOptions,
	const FSampledScalarField2f& DisplaceField,
	const int32 UVLayer )
{
	float SampleRate = TessellationOptions.ErrorTolerance;

	if (TessellationOptions.ErrorMode == EGeometryScriptTessellationErrorMode::Relative)
	{
		FBounds3d Bounds;
		for( const int VID : Mesh.VertexIndicesItr() )
		{
			Bounds += Mesh.GetVertex(VID);
		}

		double SurfaceArea = 0.0;
		for( const int TID : Mesh.TriangleIndicesItr() )
		{
			SurfaceArea += Mesh.GetTriArea(TID);
		}
		SampleRate = TessellationOptions.ErrorTolerance * 0.01 * FMath::Sqrt( FMath::Min( 2.0 * SurfaceArea, Bounds.GetSurfaceArea() ) );
	}

	FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(UVLayer);
	check(UVOverlay); 
	
	if (!Mesh.HasVertexNormals())
	{
		Mesh.EnableVertexNormals(FVector3f(0.f));
		FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
		
		for (int TID : Mesh.TriangleIndicesItr())
		{
			if (Normals->IsSetTriangle(TID) == false)
			{
				continue;
			}

			FIndex3i Tri = Mesh.GetTriangle(TID);
			FIndex3i NormalTri = Normals->GetTriangle(TID);
			
			for (int j = 0; j < 3; ++j)
			{
				int VID = Tri[j];
				FVector3f Normal = Normals->GetElement(NormalTri[j]);
				Mesh.SetVertexNormal(VID, Normal + Mesh.GetVertexNormal(VID));
			}
		}

		for (int VID : Mesh.VertexIndicesItr())
		{
			FVector3f Normal = Mesh.GetVertexNormal(VID);
			Normalize(Normal);
			Mesh.SetVertexNormal(VID, Normal);
		}
	}

	//
	TArrayView<const uint8> RawSourceData( reinterpret_cast<const uint8*>(DisplaceField.GridValues.GridValues().GetData()), 
		DisplaceField.GridValues.GridValues().Num() * sizeof(float) );

	FDisplacementMap DisplacementMap(RawSourceData, TSF_R32F, DisplaceField.Width(), DisplaceField.Height(), 
		Options.Magnitude, Options.Center, TextureAddress::TA_Clamp, TextureAddress::TA_Clamp);

	FMeshConstraints MeshConstraints;

	// preserve boundaries, one to one
	const Geometry::EEdgeRefineFlags MeshBoundaryConstraint = Geometry::EEdgeRefineFlags::FullyConstrained;

	// poly-groups 
	const Geometry::EEdgeRefineFlags GroupBoundaryConstraint = Geometry::EEdgeRefineFlags::NoConstraint;

	// 
	const Geometry::EEdgeRefineFlags MaterialBoundaryConstraint = Geometry::EEdgeRefineFlags::NoFlip;

	// impose constraints attribute seams
	Geometry::FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(MeshConstraints, Mesh,
		MeshBoundaryConstraint, GroupBoundaryConstraint, MaterialBoundaryConstraint, 
		true  /* allow seam splits */, 
		false /* don't allow seam smoothing, not needed */,
	    false /* don't allow seams to collapse, not needed */ );

	TBitArray<> ActiveTriangles;
	if (!Selection.IsEmpty())
	{
		ActiveTriangles.Init(false, Mesh.MaxTriangleID());
		Selection.ProcessByTriangleID(Mesh, [&ActiveTriangles](int32 TID) {
			ActiveTriangles[TID] = true;
		});

		FEdgeConstraint NoFlipConstraint(EEdgeRefineFlags::FullyConstrained);
		for (int EID : Mesh.EdgeIndicesItr())
		{
			const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);
			if (Edge.Tri[1] != IndexConstants::InvalidID && (ActiveTriangles[Edge.Tri[0]] != ActiveTriangles[Edge.Tri[1]]))
			{
				MeshConstraints.SetOrUpdateEdgeConstraint(EID, NoFlipConstraint);
			}
		}
	}
	
	const float TargetError = SampleRate * SampleRate;

	FAdaptiveTessellatorOptions AdaptiveTessOptions;
	AdaptiveTessOptions.TargetError      = TargetError;
	AdaptiveTessOptions.SampleRate       = SampleRate;
	AdaptiveTessOptions.bCrackFree       = false; // not required, the dynamic mesh policy operations don't introduce cracks
	AdaptiveTessOptions.MaxTriangles     = 5'000'000u;
	AdaptiveTessOptions.RefinementMethod = EAdaptiveTessellationRefinementMethod::Custom;
	AdaptiveTessOptions.CosineThreshold = FMath::Clamp(TessellationOptions.Coplanarity, 0.f, 1.f);

	ApplyAdaptiveDisplacement( Mesh, UVOverlay, DisplacementMap, AdaptiveTessOptions, TessellationOptions.FeatureSensitivity, MeshConstraints, ActiveTriangles, 
		TessellationOptions.bMeshOptimization );
}

} // namespace UGeometryScriptLibrary_MeshDeformFunctionsInternals

UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyDisplaceFromTextureMap(
	UDynamicMesh* TargetMesh,
	UTexture2D* TextureAsset,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptDisplaceFromTextureOptions Options,
	FGeometryScriptAdaptiveTessellationOptions TessellationOptions,
	int32 UVLayer,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyDisplaceFromTextureMap_InvalidInput", "ApplyDisplaceFromTextureMap: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TextureAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyDisplaceFromTextureMap_InvalidInput2", "ApplyDisplaceFromTextureMap: Texture is Null"));
		return TargetMesh;
	}

	if (Selection.IsEmpty() && Options.EmptyBehavior != EGeometryScriptEmptySelectionBehavior::FullMeshSelection)
	{
		return TargetMesh;
	}

	TImageBuilder<FVector4f> ImageData;
	if (UE::AssetUtils::ReadTexture(TextureAsset, ImageData, false) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyDisplaceFromTextureMap_TexReadFailed", "ApplyDisplaceFromTextureMap: Error reading source texture data. If using this function at Runtime, The Compression Settings type on the UTexture2D Asset must be set to VectorDisplacementmap (RGBA8)."));
		return TargetMesh;
	}
	int UseImageChannel = FMath::Clamp(Options.ImageChannel, 0, 3);

	int64 TextureWidth  = ImageData.GetDimensions().GetWidth();
	int64 TextureHeight = ImageData.GetDimensions().GetHeight();
	if (TextureWidth == 0 || TextureHeight == 0)
	{
		return TargetMesh;
	}
	
	FSampledScalarField2f DisplaceField;
	DisplaceField.Resize(TextureWidth, TextureHeight, 0.0f);
	DisplaceField.CellDimensions.X = 1.0f / (float)TextureWidth;
	DisplaceField.CellDimensions.Y = 1.0f / (float)TextureHeight;
	for (int64 y = 0; y < TextureHeight; ++y)
	{
		for (int64 x = 0; x < TextureWidth; ++x)
		{
			DisplaceField.GridValues[y*TextureWidth + x] = ImageData.GetPixel(y*TextureWidth + x)[UseImageChannel];
		}
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() == false || !(UVLayer < EditMesh.Attributes()->NumUVLayers()))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyDisplaceFromTextureMap_InvalidUVLayer", "ApplyDisplaceFromTextureMap: TargetMesh is missing requested UV Set"));
			return;
		}

		if (TessellationOptions.bEnableTessellation)
		{
			UGeometryScriptLibrary_MeshDeformFunctionsInternals::ApplyAdaptiveDisplacement(EditMesh, Selection, Options, TessellationOptions, DisplaceField, UVLayer);
			return;
		}

		FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVLayer);
		FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
		FVector2f UseUVScale = (FVector2f)Options.UVScale;
		FVector2f UseUVOffset = (FVector2f)Options.UVOffset;

		// We set things up such that DisplaceField goes from 0 to 1 in the U direction,
		// but the V direction may be shorter or longer if the texture is not square
		// (it will be 1/AspectRatio)
		float VHeight = DisplaceField.Height() * DisplaceField.CellDimensions.Y;

		TArray<FVector3d> DisplacedPositions;
		DisplacedPositions.Init(FVector3d::Zero(), EditMesh.MaxVertexID());
		TArray<int> Counts;
		Counts.Init(0, EditMesh.MaxVertexID());

		for (int tid : EditMesh.TriangleIndicesItr())
		{
			if (UVOverlay->IsSetTriangle(tid) == false || Normals->IsSetTriangle(tid) == false)
			{
				continue;
			}

			FIndex3i Tri = EditMesh.GetTriangle(tid);
			FIndex3i UVTri = UVOverlay->GetTriangle(tid);
			FIndex3i NormalTri = Normals->GetTriangle(tid);
			for (int j = 0; j < 3; ++j)
			{
				int vid = Tri[j];
				FVector2f UV = UVOverlay->GetElement(UVTri[j]);

				// Adjust UV value and tile it. 
				UV = UV * UseUVScale + UseUVOffset;
				UV = UV - FVector2f(FMath::Floor(UV.X), FMath::Floor(UV.Y));
				UV.Y *= VHeight;

				double Offset = DisplaceField.BilinearSampleClamped(UV);
				Offset -= Options.Center;

				FVector3f Normal = Normals->GetElement(NormalTri[j]);

				double Intensity = Options.Magnitude;
				DisplacedPositions[vid] += EditMesh.GetVertex(Tri[j]) + (FVector)(Offset * Intensity * Normal);
				Counts[vid]++;
			}
		}

		// This is not necessarily the most efficient strategy, as we have computed the full-mesh displacement and
		// then are potentially discarding most of that work. However if we filter by triangles then we might
		// get different results when averaging per-triangle normals.
		if (Selection.IsEmpty() == false)
		{
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID)
			{
				if (Counts[VertexID] != 0)
				{
					EditMesh.SetVertex(VertexID, DisplacedPositions[VertexID] / (double)Counts[VertexID]);
				}
			});
		}
		else
		{
			for (int32 vid : EditMesh.VertexIndicesItr())
			{
				if (Counts[vid] != 0)
				{
					EditMesh.SetVertex(vid, DisplacedPositions[vid] / (double)Counts[vid]);
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshDeformFunctions::ApplyDisplaceFromPerVertexVectors(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	const FGeometryScriptVectorList& VectorList, 
	float Magnitude,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyDisplaceFromPerVertexVectors_InvalidInput", "ApplyDisplaceFromPerVertexVectors: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (VectorList.List.IsValid() == false || VectorList.List->Num() < EditMesh.MaxVertexID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyDisplaceFromPerVertexVectors_InvalidVectorLength", "ApplyDisplaceFromPerVertexVectors: VectorList Length is less than TargetMesh MaxVertexID"));
			return;
		}

		const TArray<FVector>& Vectors = *VectorList.List;

		auto UpdateVertex = [&EditMesh, &Magnitude, &Vectors](int32 VertexID)
		{
			FVector Position = EditMesh.GetVertex(VertexID);
			FVector Vector = Magnitude * Vectors[VertexID];
			if (Vector.Length() < UE_FLOAT_HUGE_DISTANCE)		// this is a bit arbitrary but should avoid disasters
			{
				EditMesh.SetVertex(VertexID, Position + Vector);
			}
		};

		if (Selection.IsEmpty())
		{
			for (int32 vid : EditMesh.VertexIndicesItr())
			{
				UpdateVertex(vid);
			}
		}
		else
		{
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID) { UpdateVertex(VertexID); });
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
