// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshGeodesicFunctions.h"
#include "UDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Operations/GeodesicPath.h"
#include "Operations/MeshGeodesicSurfaceTracer.h"
#include "Math/UnrealMathUtility.h"
#include "Parameterization/MeshDijkstra.h"



#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshGeodesicFunctions)


using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshGeodesicFunctions"



UDynamicMesh* UGeometryScriptLibrary_MeshGeodesicFunctions::GetShortestVertexPath( UDynamicMesh* TargetMesh,
                                                                                   int32 StartVID,
																			       int32 EndVID,
																			       FGeometryScriptIndexList& VertexIDList,
																			       bool& bFoundErrors,
																			       UGeometryScriptDebug* Debug)
{
	bFoundErrors = true;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestVertexPath_InvalidInput", "GetShortestVertexPath: TargetMesh is Null"));
		return TargetMesh;
	}

	if (StartVID == EndVID)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestVertexPath_DuplicateInputs", "GetShortestVertexPath: Start and End vertex are the same"));
		return TargetMesh;
	}

	auto MeshHasVertex = [TargetMesh](int32 VID)
	{ 
		bool Result = false;
		TargetMesh->ProcessMesh([VID, &Result](const FDynamicMesh3& Mesh)
			{
				Result =  Mesh.IsVertex(VID);
			}); 
		return Result;
	};


	const bool bMeshHasStartVID = MeshHasVertex(StartVID);
	const bool bMeshHasEndVID   = MeshHasVertex(EndVID);

	if (!bMeshHasStartVID)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestVertexPath_InvalidStart", "GetShortestVertexPath: Start vertex not part of mesh"));
		return TargetMesh;
	}

	if (!bMeshHasEndVID)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestVertexPath_InvalidEnd", "GetShortestVertexPath: End vertex not part of mesh"));
		return TargetMesh;
	}
	
	
	const TArray<int32> DijkstraVertexPath = [&]
	{
		TArray<int32> VertexPath;
		TargetMesh->ProcessMesh([StartVID, EndVID, &VertexPath](const FDynamicMesh3& ReadMesh)
		{
			typedef TMeshDijkstra<FDynamicMesh3>  FMeshDijkstra;
			FMeshDijkstra MeshDijkstra(&ReadMesh);

			TArray<FMeshDijkstra::FSeedPoint> SeedPoints;
			FMeshDijkstra::FSeedPoint& Seed = SeedPoints.AddDefaulted_GetRef();
			Seed.PointID = EndVID;

			// compute graph distances
			MeshDijkstra.ComputeToTargetPoint(SeedPoints, StartVID);

			// Find path from StartVID to the seed point (EndVID) 
			MeshDijkstra.FindPathToNearestSeed(StartVID, VertexPath);
		});
		return MoveTemp(VertexPath);
	}();

	if (DijkstraVertexPath.Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestVertexPath_Failed", "GetShortestVertexPath: Failed to find connecting path"));
		return TargetMesh;
	}


	// convert to correct output type
	{
		VertexIDList.Reset(EGeometryScriptIndexType::Vertex);
		VertexIDList.List->Append(DijkstraVertexPath);
	}

	bFoundErrors = false;
	return TargetMesh;

}
 
UDynamicMesh* UGeometryScriptLibrary_MeshGeodesicFunctions::GetShortestSurfacePath( UDynamicMesh* TargetMesh,
																					int32 StartTID,
																					FVector StartTriCoords,
																					int32 EndTID,
																					FVector EndTriCoords,
																					FGeometryScriptPolyPath& ShortestPath,
																					bool& bFoundErrors,
																					UGeometryScriptDebug* Debug)
{
	bFoundErrors = true;

	ShortestPath.Reset();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_InvalidInput", "GetShortestSurfacePath: TargetMesh is Null"));
		return TargetMesh;
	}



	auto MeshHasTriangle = [TargetMesh](const int32& TID)->bool
	{ 
		bool Result = false;
		TargetMesh->ProcessMesh([TID, &Result](const FDynamicMesh3& Mesh)
			{
				Result =  Mesh.IsTriangle(TID);
			});
		return Result;	
	};



	
	const bool bHasStartTID = MeshHasTriangle(StartTID);
	const bool bHasEndTID   = MeshHasTriangle(EndTID);

	if (!bHasStartTID)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_InvalidStartTriangle", "GetShortestSurfacePath: Start Point triangle not part of mesh"));
		return TargetMesh;
	}

	if (!bHasEndTID)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_InvalidEndTriangle", "GetShortestSurfacePath: End Point triangle not part of mesh"));
		return TargetMesh;
	}

	auto MakeValidBaryCentric = [](const FVector& Vec)->FVector
	{
		// any negative value indicates invalid BC
		if (Vec.X < 0 || Vec.Y < 0 || Vec.Z < 0)
		{
			return FVector(1./3., 1./3., 1./3.);
		}

		const double Sum = Vec.X + Vec.Y + Vec.Z;
		constexpr double Tol = 0.05;
		if (FMath::Abs(Sum - 1.0)  > Tol) // allow for a small amount of error before we declare invalid BC 
		{
			return FVector(1./3., 1./3., 1./3.);
		}
		else
		{
			return (1./Sum) * Vec;
		}
	};

	const FVector StartBC = MakeValidBaryCentric(StartTriCoords);
	const FVector EndBC   = MakeValidBaryCentric(EndTriCoords);

	
	// trivial case when both start and end points are on the same triangle.
	if (StartTID == EndTID)
	{
		
		if (StartBC == EndBC)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_DuplicateInputs", "GetShortestSurfacePath: Start and End Point are the same"));
			return TargetMesh;
		}

		auto GetTriBaryPoint = [TargetMesh](const int32 TID, const FVector& BaryCoords)
		{

			FVector PositionOfBaryPoint = FVector::Zero();
			TargetMesh->ProcessMesh([TID, &BaryCoords, &PositionOfBaryPoint](const FDynamicMesh3& Mesh)
				{
					PositionOfBaryPoint = Mesh.GetTriBaryPoint(TID, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
				});

			return PositionOfBaryPoint;
		};

		const FVector StartPosition = GetTriBaryPoint(StartTID, StartBC);
		const FVector EndPosition   = GetTriBaryPoint(EndTID, EndBC);
		
		ShortestPath.bClosedLoop = false;
		ShortestPath.Path->Add(StartPosition);
		ShortestPath.Path->Add(EndPosition);

		bFoundErrors = false;
		return TargetMesh;
	}



	// compute the geodesic using the intrinsic mesh.
	{

		// Currently the intrinsic mesh geodesic only connects vertices on of the dynamic mesh.
		// @todo see if this limitation can be removed..
		FDynamicMesh3 TmpMesh;
		TargetMesh->ProcessMesh([&TmpMesh](const FDynamicMesh3& SrcMesh){TmpMesh = SrcMesh;});

		int32 StartVID = FDynamicMesh3::InvalidID;
		int32 EndVID = FDynamicMesh3::InvalidID;
		
		// add start and end vertices to the tmp mesh
		{
			DynamicMeshInfo::FPokeTriangleInfo PokeInfo;
			if (TmpMesh.PokeTriangle(StartTID, StartBC, PokeInfo) == EMeshResult::Ok)
			{
				  StartVID = PokeInfo.NewVertex;
			}
			if (TmpMesh.PokeTriangle(EndTID, EndBC, PokeInfo) == EMeshResult::Ok)
			{
				EndVID = PokeInfo.NewVertex;
			}

			// somehow we were unable to create the start or end point.  perhaps the barycoords weren't valid...
			if (StartVID == FDynamicMesh3::InvalidID || EndVID == FDynamicMesh3::InvalidID)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_PokeFailure", "GetShortestSurfacePath: failed"));
				return TargetMesh;
			}
		}
		

		// create the initial path that will be deformed into the shortest path.
		const TArray<FEdgePath::FDirectedSegment> DirectedSegments = [&TmpMesh, StartVID, EndVID]
		{
			// find the initial path by using Dijkstra
			const TArray<int32> DijkstraVertexPath = [&]
			{
				typedef TMeshDijkstra<FDynamicMesh3>  FMeshDijkstra;
				FMeshDijkstra MeshDijkstra(&TmpMesh);

				TArray<FMeshDijkstra::FSeedPoint> SeedPoints;
				FMeshDijkstra::FSeedPoint& Seed = SeedPoints.AddDefaulted_GetRef();
				Seed.PointID = EndVID;

				// compute distances from EndVID, up to StartVID
				MeshDijkstra.ComputeToTargetPoint(SeedPoints, StartVID);

				// make a vertex path from Seed (EndVID) to StartVID
				TArray<int32> VertexPath;
				MeshDijkstra.FindPathToNearestSeed(StartVID, VertexPath);
				return MoveTemp(VertexPath);
			}();
														
			// convert the initial path into a sequence of directed edges.
			TArray<FEdgePath::FDirectedSegment> SegmentPath;
			SegmentPath.Empty();
			if (DijkstraVertexPath.Num() == 0)
			{
				return MoveTemp(SegmentPath);
			}

			SegmentPath.Reserve(DijkstraVertexPath.Num() - 1);

			for (int32 i = 1; i < DijkstraVertexPath.Num(); ++i)
			{
				int32 VID = DijkstraVertexPath[i];
				int32 PVID = DijkstraVertexPath[i - 1];

				int32 EID = TmpMesh.FindEdge(VID, PVID);
				if (EID != FDynamicMesh3::InvalidID)
				{
					const FIndex2i EdgeV = TmpMesh.GetEdgeV(EID);

					FEdgePath::FDirectedSegment& Segment = SegmentPath.AddDefaulted_GetRef();
					Segment.EID = EID;
					Segment.HeadIndex = (EdgeV.B == VID) ? 1 : 0;
				}
			}

			return MoveTemp(SegmentPath);
		}();

		// failed to find any path connecting the start and end point.  Most likely they are not in the same connected component.
		if ( DirectedSegments.Num() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetShortestSurfacePath_DijkstraFailure", "GetShortestSurfacePath: failed to find a path connecting the points"));
			return TargetMesh;
		}

		// populate the deformable path
		FDeformableEdgePath DeformableEdgePath(TmpMesh, DirectedSegments);

		// minimize the deformable path
		FDeformableEdgePath::FEdgePathDeformationInfo ResultInfo;
		DeformableEdgePath.Minimize(ResultInfo);

		// convert minimized deformable path to a PolyPath
		{
			ShortestPath.bClosedLoop = false;
			TArray<FVector>& GeodesicPath = *ShortestPath.Path;
			GeodesicPath.Empty();

			constexpr double CoalesceThreshold = 0.01; // essentially a welding threshold for adjacent path points.

			TArray<FDeformableEdgePath::FSurfacePoint> SurfacePathPoints = DeformableEdgePath.AsSurfacePoints(CoalesceThreshold);
			
			GeodesicPath.Reserve(SurfacePathPoints.Num());
			
			const FDynamicMesh3* SurfaceMesh = DeformableEdgePath.GetIntrinsicMesh().GetNormalCoordinates().SurfaceMesh;
			for (const FDeformableEdgePath::FSurfacePoint& SurfacePoint : SurfacePathPoints)
			{
				bool bValid;
				const FVector3d Pos = AsR3Position(SurfacePoint, *SurfaceMesh, bValid);
				GeodesicPath.Add(Pos);
			}
		}

		bFoundErrors = false;
		return TargetMesh;
	}

}

UDynamicMesh* UGeometryScriptLibrary_MeshGeodesicFunctions::CreateSurfacePath( UDynamicMesh* TargetMesh,
																			   FVector StartDirection,
																			   int32 StartTriangleID,
																			   FVector StartBaryCoords,
																			   float PathLength,
																			   FGeometryScriptPolyPath& SurfacePath,
																			   bool& bFoundErrors,
																			   UGeometryScriptDebug* Debug)
{
	typedef UE::Geometry::FMeshGeodesicSurfaceTracer  FSurfaceTracer;

	bFoundErrors = true;

	SurfacePath.Reset();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateSurfacePath_InvalidInput", "CreateSurfacePath: TargetMesh is Null"));
		return TargetMesh;
	}

	auto MakeValidBaryCentric = [](const FVector& Vec)->FVector
	{
		FVector ResultVec = Vec;
		
		// correct for error in BC calculation that might have returned a very small number in place of zero.
		constexpr double BCTol = 1.e-2;
		for (int i = 0; i < 3; ++i)
		{
			if (ResultVec[i] < 0 && ResultVec[i] >= -BCTol)
			{
				ResultVec[i] = 0.;
			}
		}



		// any negative value indicates invalid BC
		if (ResultVec.X < 0 || ResultVec.Y < 0 || ResultVec.Z < 0)
		{
			ResultVec = FVector(1./3., 1./3., 1./3.);
		}

		const double Sum = Vec.X + Vec.Y + Vec.Z;
		constexpr double Tol = 0.05;
		if (FMath::Abs(Sum - 1.0)  > Tol) // allow for a small amount of error before we declare invalid BC 
		{
			ResultVec = FVector(1./3., 1./3., 1./3.);
		}
		else
		{
			ResultVec *= (1./Sum);
		}

		return ResultVec;
	};

	
	const FVector Direction = StartDirection.GetSafeNormal(1.e-5 /* Tolerance */, FVector::UpVector);

	const FVector StartBC = MakeValidBaryCentric(StartBaryCoords);

	
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{

			// do the surface trace.
			FSurfaceTracer SurfaceTracer(ReadMesh);
			SurfaceTracer.TraceMeshFromBaryPoint(StartTriangleID, StartBC, Direction, PathLength);
			const TArray<FSurfaceTracer::FTraceResult>& Trace = SurfaceTracer.GetTraceResults();
			const int32 NumPoints = Trace.Num();
			
			// convert results to poly path
			SurfacePath.bClosedLoop = false;
			SurfacePath.Path->Reserve(NumPoints);
			for (int32 i = 0; i < NumPoints; ++i)
			{
				const FSurfaceTracer::FTraceResult& TracePoint = Trace[i];

				if (TracePoint.TriID == IndexConstants::InvalidID) break;
				
				if (ReadMesh.IsTriangle(TracePoint.TriID))
				{
					FVector Pos = ReadMesh.GetTriBaryPoint(TracePoint.TriID, TracePoint.Barycentric[0], TracePoint.Barycentric[1], TracePoint.Barycentric[2]);
					SurfacePath.Path->Add(Pos);
				}
				
			}
		});

	bFoundErrors = false;
	return TargetMesh;

}

#undef LOCTEXT_NAMESPACE