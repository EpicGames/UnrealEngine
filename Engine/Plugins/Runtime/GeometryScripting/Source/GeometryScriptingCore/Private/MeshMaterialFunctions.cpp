// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshMaterialFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshAttributeUtil.h"
#include "UDynamicMesh.h"
#include "Polygroups/PolygroupSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMaterialFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshMaterialFunctions"


template<typename ReturnType> 
ReturnType SimpleMeshMaterialQuery(UDynamicMesh* Mesh, bool& bHasMaterials, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialIDs)> QueryFunc)
{
	bHasMaterials = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->HasMaterialID() )
			{
				const FDynamicMeshMaterialAttribute* MaterialIDs = ReadMesh.Attributes()->GetMaterialID();
				if (MaterialIDs != nullptr)
				{
					bHasMaterials = true;
					RetVal = QueryFunc(ReadMesh, *MaterialIDs);
				}
			}
		});
	}
	return RetVal;
}


void SimpleMeshMaterialEdit(UDynamicMesh* Mesh, bool bEnableIfMissing, bool& bHasMaterialIDs,
	TFunctionRef<void(FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs)> EditFunc)
{
	bHasMaterialIDs = false;
	if (Mesh)
	{
		Mesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes() == false)
			{
				if (bEnableIfMissing)
				{
					EditMesh.EnableAttributes();
				}
				else
				{
					return;
				}
			}
			if (EditMesh.Attributes()->HasMaterialID() == false)
			{
				if (bEnableIfMissing)
				{
					EditMesh.Attributes()->EnableMaterialID();
				}
				else
				{
					return;
				}
			}
			FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
			if (ensure(MaterialIDs != nullptr))
			{
				bHasMaterialIDs = true;
				EditFunc(EditMesh, *MaterialIDs);
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
}




UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::EnableMaterialIDs(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("EnableMaterialIDs_InvalidInput", "EnableMaterialIDs: TargetMesh is Null"));
		return TargetMesh;
	}

	// this will enable the material IDs even though the lambda doesn't do anything
	bool bHasMaterialIDs;
	SimpleMeshMaterialEdit(TargetMesh, true, bHasMaterialIDs, [](FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs) {});

	return TargetMesh;

}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::ClearMaterialIDs(
	UDynamicMesh* TargetMesh,
	int ClearValue,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ClearMaterialIDs_InvalidInput", "ClearMaterialIDs: TargetMesh is Null"));
		return TargetMesh;
	}

	ClearValue = FMath::Max(0, ClearValue);

	bool bHasMaterialIDs;
	SimpleMeshMaterialEdit(TargetMesh, true, bHasMaterialIDs, 
		[ClearValue](FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs) 
	{
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			MaterialIDs.SetValue(TriangleID, ClearValue);
		}
	});

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::RemapMaterialIDs(
	UDynamicMesh* TargetMesh,
	int FromMaterialID,
	int ToMaterialID,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapMaterialIDs_InvalidInput", "RemapMaterialIDs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasMaterialIDs;
	SimpleMeshMaterialEdit(TargetMesh, true, bHasMaterialIDs, 
		[&](FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs) 
	{
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			int32 CurID = MaterialIDs.GetValue(TriangleID);
			if (CurID == FromMaterialID)
			{
				MaterialIDs.SetValue(TriangleID, ToMaterialID);
			}
		}
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::RemapToNewMaterialIDsByMaterial(
	UDynamicMesh* TargetMesh,
	const TArray<UMaterialInterface*>& FromMaterialList,
	const TArray<UMaterialInterface*>& ToMaterialList,
	int MissingMaterialID,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapToNewMaterialIDsByMaterial_InvalidInput", "RemapToNewMaterialIDsByMaterial: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<int32> ToMaterialID;
	ToMaterialID.SetNum(FromMaterialList.Num());
	for (int32 k = 0; k < FromMaterialList.Num(); ++k)
	{
		int32 FoundIdx = ToMaterialList.IndexOfByKey(FromMaterialList[k]);
		if (FoundIdx == INDEX_NONE)
		{
			if (MissingMaterialID >= 0)
			{
				ToMaterialID[k] = MissingMaterialID;
			}
			else
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapToNewMaterialIDsByMaterial_MaterialMissing", "RemapToNewMaterialIDsByMaterial: Material in FromMaterialList not found in ToMaterialList, skipping"));
				ToMaterialID[k] = k;
			}
		}
		else
		{
			ToMaterialID[k] = FoundIdx;
		}
	}

	bool bHasMaterialIDs;
	SimpleMeshMaterialEdit(TargetMesh, true, bHasMaterialIDs, 
		[&](FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs) 
	{
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			int32 CurID = MaterialIDs.GetValue(TriangleID);
			if (!ToMaterialID.IsValidIndex(CurID))
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapToNewMaterialIDsByMaterial_InvalidMaterial", "RemapToNewMaterialIDsByMaterial: Invalid material ID in mesh was not a valid index into FromMaterialList, skipping"));
				continue;
			}
			int32 NewID = ToMaterialID[CurID];
			MaterialIDs.SetValue(TriangleID, NewID);
		}
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::RemapAndCombineMaterials(
	UDynamicMesh* TargetMesh,
	const TArray<UMaterialInterface*>& TargetMeshMaterials,
	const TArray<UMaterialInterface*>& RequiredMaterials,
	TArray<UMaterialInterface*>& CombinedMaterials,
	int RemapInvalidMaterialID,
	bool bCompactDuplicateMaterials,
	UGeometryScriptDebug* Debug
)
{
	CombinedMaterials = RequiredMaterials;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapAndCombineMaterials_InvalidInput", "RemapAndCombineMaterials: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<int32> ToMaterialID;
	if (bCompactDuplicateMaterials)
	{
		ToMaterialID.Init(-1, TargetMeshMaterials.Num());
		for (int32 ToMaterialIdx = 0; ToMaterialIdx < TargetMeshMaterials.Num(); ++ToMaterialIdx)
		{
			int32 IdxInCombined = -1;
			if (!CombinedMaterials.Find(TargetMeshMaterials[ToMaterialIdx], IdxInCombined))
			{
				IdxInCombined = CombinedMaterials.Add(TargetMeshMaterials[ToMaterialIdx]);
			}
			ToMaterialID[ToMaterialIdx] = IdxInCombined;
		}
	}
	else
	{
		CombinedMaterials.Append(TargetMeshMaterials);
	}

	bool bHasMaterialIDs;
	SimpleMeshMaterialEdit(TargetMesh, true, bHasMaterialIDs,
		[&](FDynamicMesh3& Mesh, FDynamicMeshMaterialAttribute& MaterialIDs)
		{
			for (int32 TriangleID : Mesh.TriangleIndicesItr())
			{
				int32 CurID = MaterialIDs.GetValue(TriangleID);
				int32 NewID = RemapInvalidMaterialID;
				if (TargetMeshMaterials.IsValidIndex(CurID))
				{
					if (!bCompactDuplicateMaterials)
					{
						NewID = CurID + RequiredMaterials.Num();
					}
					else
					{
						NewID = ToMaterialID[CurID];
					}
				}
				if (NewID < 0)
				{
					UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemapAndCombineMaterials_InvalidMaterial", "RemapAndCombineMaterials: Invalid material ID in mesh was not a valid index into TargetMeshMaterials, skipping"));
				}
				else
				{
					MaterialIDs.SetValue(TriangleID, NewID);
				}
			}
		});

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::GetMaterialIDsOfTriangles( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptIndexList TriangleIDList,
	FGeometryScriptIndexList& MaterialIDList,
    UGeometryScriptDebug* Debug)
{
	MaterialIDList.Reset(EGeometryScriptIndexType::MaterialID);
	
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_InvalidMesh", "GetMaterialIDsOfTriangles: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TriangleIDList.List.IsValid() == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_InvalidList", "GetMaterialIDsOfTriangles: TriangleIDList is Null"));
		return TargetMesh;
	}
	if (TriangleIDList.IsCompatibleWith(EGeometryScriptIndexType::Triangle) == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_InvalidList2", "GetMaterialIDsOfTriangles: TriangleIDList has incompatible index type"));
		return TargetMesh;
	}
	if (TriangleIDList.List->Num() == 0 && TargetMesh->IsEmpty() == false)
	{
		AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_EmptyList", "GetMaterialIDsOfTriangles: TriangleIDList is empty"));
	}
	
	bool bHasMaterials = false;
	bool bAllValidTriangles = true;
	SimpleMeshMaterialQuery<int32>(TargetMesh, bHasMaterials, 0, 
		[&MaterialIDList, &bHasMaterials, &bAllValidTriangles, &TriangleIDList] 
		(const FDynamicMesh3& EditMesh, const FDynamicMeshMaterialAttribute& MaterialIDs) 
	{
		TArray<int>& MaterialIDArray = *MaterialIDList.List;
		MaterialIDArray.SetNum(TriangleIDList.List->Num());
		for (int32 i = 0; i < TriangleIDList.List->Num(); i++)
		{
			int32 TriangleID = (*TriangleIDList.List)[i];
			if (EditMesh.IsTriangle(TriangleID) == false)
			{
				bAllValidTriangles = false;
				MaterialIDArray[i] = -1;
			}
			else
			{
				MaterialIDArray[i] = MaterialIDs.GetValue(TriangleID);
			}
		}
		return 0;
	});
	
	if (bHasMaterials == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_MissingMaterials", "GetMaterialIDsOfTriangles: MaterialID Attribute is not enabled"));
		return TargetMesh;
	}
	
	if (bAllValidTriangles == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMaterialIDsOfTriangles_InvalidTriangles", "GetMaterialIDsOfTriangles: TriangleIDList has invalid triangles"));
		return TargetMesh;
	}
	
	return TargetMesh;
}


int UGeometryScriptLibrary_MeshMaterialFunctions::GetMaxMaterialID( UDynamicMesh* TargetMesh, bool& bHasMaterialIDs )
{
	return SimpleMeshMaterialQuery<int32>(TargetMesh, bHasMaterialIDs, 0, [&](const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialIDs) {
		int32 MaxID = 0;
		for (int TriangleID : Mesh.TriangleIndicesItr())
		{
			MaxID = FMath::Max(MaxID, MaterialIDs.GetValue(TriangleID));
		}
		return MaxID;
	});
}



int32 UGeometryScriptLibrary_MeshMaterialFunctions::GetTriangleMaterialID( 
	UDynamicMesh* TargetMesh, 
	int TriangleID, 
	bool& bIsValidTriangle)
{
	bool bHasMaterials = false;
	return SimpleMeshMaterialQuery<int32>(TargetMesh, bHasMaterials, 0, [&](const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialIDs) {
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		return (bIsValidTriangle) ? MaterialIDs.GetValue(TriangleID) : 0;
	});
}


UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::GetAllTriangleMaterialIDs(UDynamicMesh* TargetMesh, FGeometryScriptIndexList& MaterialIDList, bool& bHasMaterialIDs)
{
	MaterialIDList.Reset(EGeometryScriptIndexType::MaterialID);
	TArray<int32>& MaterialIDs = *MaterialIDList.List;
	bHasMaterialIDs = false;
	SimpleMeshMaterialQuery<int32>(TargetMesh, bHasMaterialIDs, 0, [&](const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialIDAttrib) {
		int32 MaxTriangleID = Mesh.MaxTriangleID();
		for (int32 TriangleID = 0; TriangleID < Mesh.MaxTriangleID(); ++TriangleID)
		{
			int32 MaterialID = Mesh.IsTriangle(TriangleID) ? MaterialIDAttrib.GetValue(TriangleID) : -1;
			MaterialIDs.Add(MaterialID);
		}
		return 0;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::GetTrianglesByMaterialID( 
		UDynamicMesh* TargetMesh, 
		int MaterialID,
		FGeometryScriptIndexList& TriangleIDList,
		UGeometryScriptDebug* Debug)
{
	TriangleIDList.Reset(EGeometryScriptIndexType::Triangle);

	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetTrianglesByMaterialID_InvalidInput", "GetTrianglesByMaterialID: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasMaterialIDs = false;
	bool bAllValidTriangles = true;
	SimpleMeshMaterialQuery<int32>(TargetMesh, bHasMaterialIDs, 0, [&](const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialIDAttrib) {
		for (int32 TriangleID = 0; TriangleID < Mesh.MaxTriangleID(); ++TriangleID)
		{
			if (Mesh.IsTriangle(TriangleID) == false)
			{
				bAllValidTriangles = false;
				continue;
			}

			if (MaterialIDAttrib.GetValue(TriangleID) == MaterialID)
			{
				TriangleIDList.List->Add(TriangleID);
			}
		}
		return 0;
	});

	if (bHasMaterialIDs == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetTrianglesByMaterialID_MissingMaterialID", "GetTrianglesByMaterialID: MaterialID Attribute is not enabled"));
		return TargetMesh;
	}

	if (bAllValidTriangles == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetTrianglesByMaterialID_InvalidTriangles", "GetTrianglesByMaterialID: TriangleIDList has invalid triangles"));
		return TargetMesh;
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::SetTriangleMaterialID( 
	UDynamicMesh* TargetMesh, 
	int TriangleID, 
	int MaterialID,
	bool& bIsValidTriangle,
	bool bDeferChangeNotifications)
{
	bIsValidTriangle = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsTriangle(TriangleID) && EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID() )
			{
				FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
				if (MaterialIDs != nullptr)
				{
					bIsValidTriangle = true;
					MaterialIDs->SetValue(TriangleID, MaterialID);
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::SetAllTriangleMaterialIDs(
	UDynamicMesh* TargetMesh,
	FGeometryScriptIndexList TriangleMaterialIDList,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllTriangleMaterialIDs_InvalidMesh", "SetAllTriangleMaterialIDs: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TriangleMaterialIDList.List.IsValid() == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllTriangleMaterialIDs_InvalidList", "SetAllTriangleMaterialIDs: TriangleMaterialIDList is Null"));
		return TargetMesh;
	}
	if (TriangleMaterialIDList.IsCompatibleWith(EGeometryScriptIndexType::MaterialID) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllTriangleMaterialIDs_InvalidList2", "SetAllTriangleMaterialIDs: TriangleMaterialIDList has incompatible index type"));
		return TargetMesh;
	}
	if (TriangleMaterialIDList.List->Num() == 0 && TargetMesh->IsEmpty() == false)
	{
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllTriangleMaterialIDs_EmptyList", "SetAllTriangleMaterialIDs: TriangleMaterialIDList is empty"));
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		const TArray<int32>& TriangleMaterialIDs = *TriangleMaterialIDList.List;
		if (TriangleMaterialIDs.Num() < EditMesh.MaxTriangleID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetAllTriangleMaterialIDs_IncorrectCount", "SetAllTriangleMaterialIDs: size of provided TriangleMaterialIDList is smaller than MaxTriangleID of Mesh"));
		}
		else
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}
			if (EditMesh.Attributes()->HasMaterialID() == false)
			{
				EditMesh.Attributes()->EnableMaterialID();
			}
			FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				MaterialIDs->SetValue(TriangleID, TriangleMaterialIDs[TriangleID]);
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::SetMaterialIDOnTriangles( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptIndexList TriangleIDList,
	int MaterialID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{	
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDOnTriangles_InvalidMesh", "SetMaterialIDOnTriangles: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TriangleIDList.List.IsValid() == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDOnTriangles_InvalidList", "SetMaterialIDOnTriangles: TriangleIDList is Null"));
		return TargetMesh;
	}
	if (TriangleIDList.IsCompatibleWith(EGeometryScriptIndexType::Triangle) == false)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDOnTriangles_InvalidList2", "SetMaterialIDOnTriangles: TriangleIDList has incompatible index type"));
		return TargetMesh;
	}
	if (TriangleIDList.List->Num() == 0 && TargetMesh->IsEmpty() == false)
	{
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDOnTriangles_EmptyList", "SetMaterialIDOnTriangles: TriangleIDList is empty"));
	}

	bool bHasInvalidTriangles = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}
			if (EditMesh.Attributes()->HasMaterialID() == false)
			{
				EditMesh.Attributes()->EnableMaterialID();
			}
			FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
			for (int32 TriangleID : *TriangleIDList.List)
			{
				if (EditMesh.IsTriangle(TriangleID) == false)
				{
					bHasInvalidTriangles = true;
					return;
				}
				MaterialIDs->SetValue(TriangleID, MaterialID);
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	if (bHasInvalidTriangles == true)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDOnTriangles_InvalidTriangles", "SetMaterialIDOnTriangles: TriangleIDList has invalid triangles"));
		return TargetMesh;
	}
	
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::SetMaterialIDForMeshSelection( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptMeshSelection Selection,
	int MaterialID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{	
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMaterialIDForMeshSelection_InvalidMesh", "SetMaterialIDForMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		if (EditMesh.Attributes()->HasMaterialID() == false)
		{
			EditMesh.Attributes()->EnableMaterialID();
		}
		FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
		Selection.ProcessByTriangleID(EditMesh,
			[&](int32 TriangleID) { MaterialIDs->SetValue(TriangleID, MaterialID); } );
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::SetPolygroupMaterialID( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	int PolygroupID, 
	int MaterialID,
	bool& bIsValidPolygroupID,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	bIsValidPolygroupID = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh) == false)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetPolygroupMaterialID_MissingGroups", "SetPolygroupMaterialID: Specified Polygroup Layer does not exist"));
				return;
			}
			FDynamicMeshMaterialAttribute* MaterialIDs = (EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID()) ? EditMesh.Attributes()->GetMaterialID() : nullptr;
			if (MaterialIDs == nullptr)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetPolygroupMaterialID_NoMaterialID", "SetPolygroupMaterialID: MaterialID Attribute is not enabled"));
				return;
			}

			FPolygroupSet Groups(&EditMesh, InputGroupLayer);
			for (int32 tid : EditMesh.TriangleIndicesItr())
			{
				if (Groups.GetGroup(tid) == PolygroupID)
				{
					MaterialIDs->SetValue(tid, MaterialID);
					bIsValidPolygroupID = true;
				}
			}

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}




UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::DeleteTrianglesByMaterialID(
	UDynamicMesh* TargetMesh,
	int MaterialID,
	int& NumDeleted,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	NumDeleted = 0;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FDynamicMeshMaterialAttribute* MaterialIDs = (EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID()) ? EditMesh.Attributes()->GetMaterialID() : nullptr;
			if (MaterialIDs == nullptr)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DeleteTrianglesByMaterialID_NoMaterialID", "DeleteTrianglesByMaterialID: MaterialID Attribute is not enabled"));
				return;
			}

			TArray<int32> TriangleList;
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				if (MaterialIDs->GetValue(TriangleID) == MaterialID)
				{
					TriangleList.Add(TriangleID);
				}
			}

			for (int32 TriangleID : TriangleList)
			{
				EMeshResult Result = EditMesh.RemoveTriangle(TriangleID);
				if (Result == EMeshResult::Ok)
				{
					NumDeleted++;
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshMaterialFunctions::CompactMaterialIDs(
	UDynamicMesh* TargetMesh,
	TArray<UMaterialInterface*> SourceMaterialList,
	TArray<UMaterialInterface*>& CompactedMaterialList,
	bool bRemoveDuplicateMaterials,
	UGeometryScriptDebug* Debug)
{
	CompactedMaterialList = SourceMaterialList;

	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FDynamicMeshMaterialAttribute* MaterialIDs = (EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID()) ? EditMesh.Attributes()->GetMaterialID() : nullptr;
			if (MaterialIDs == nullptr)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CompactMaterialIDs_NoMaterialID", "CompactMaterialIDs: MaterialID Attribute is not enabled"));
				return;
			}

			// Find which material IDs are used
			FInterval1i OldValueRange;
			TArray<bool> MaterialUsed;
			MaterialUsed.SetNumZeroed(SourceMaterialList.Num());
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				int32 MID = MaterialIDs->GetValue(TID);
				if (MID < 0)
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CompactMaterialIDs_InvalidMaterialID", "CompactMaterialIDs: Invalid MaterialIDs found, unsafe to Compact"));
					return;
				}
				else if (MID > MaterialUsed.Num()) // Note we allow material IDs beyond the SourceMaterialList array size
				{
					MaterialUsed.AddZeroed(MID + 1 - MaterialUsed.Num());
				}
				MaterialUsed[MID] = true;
				OldValueRange.Contain(MID);
			}
			check(OldValueRange.Min >= 0 && OldValueRange.Max < MaterialUsed.Num());

			CompactedMaterialList.Reset();

			// Build an order-preserving mapping from original material IDs to their compacted ID
			TArray<int32> ToCompactIdx;
			ToCompactIdx.Init(-1, SourceMaterialList.Num());
			TMap<UMaterialInterface*, int32> UniqueMaterials;
			for (int32 SourceIdx = 0; SourceIdx < MaterialUsed.Num(); ++SourceIdx)
			{
				if (!MaterialUsed[SourceIdx])
				{
					continue;
				}
				UMaterialInterface* Mat = SourceIdx < SourceMaterialList.Num() ? SourceMaterialList[SourceIdx] : nullptr;
				int32 ToIdx = -1;
				if (bRemoveDuplicateMaterials)
				{
					int32* FoundIdx = UniqueMaterials.Find(Mat);
					if (!FoundIdx)
					{
						ToIdx = CompactedMaterialList.Add(Mat);
						UniqueMaterials.Add(Mat, ToIdx);
					}
					else
					{
						ToIdx = *FoundIdx;
					}
				}
				else
				{
					ToIdx = CompactedMaterialList.Add(Mat);
				}
				ToCompactIdx[SourceIdx] = ToIdx;
			}

			// Apply the remapping
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				int32 OrigMID = MaterialIDs->GetValue(TID);
				if (ToCompactIdx.IsValidIndex(OrigMID))
				{
					MaterialIDs->SetValue(TID, ToCompactIdx[OrigMID]);
				}
			}

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	return TargetMesh;
}

#undef LOCTEXT_NAMESPACE
