// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshModelingFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"
#include "Util/ProgressCancel.h"
#include "Operations/JoinMeshLoops.h"
#include "MeshBoundaryLoops.h"
#include "DynamicMeshEditor.h"
#include "GroupTopology.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Operations/OffsetMeshRegion.h"
#include "Operations/InsetMeshRegion.h"
#include "Operations/MeshBevel.h"
#include "UDynamicMesh.h"
#include "GeometryScript/MeshSelectionFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshModelingFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshModelingFunctions"




class FMeshOffset
{
public:

	FDynamicMesh3* Mesh;

	int32 Steps = 1;
	double SmoothAlpha = 0.1;
	bool bReprojectSmooth = false;
	double BoundaryAlpha = 0.2;		// should not be > 0.9
	bool bFixedBoundary = false;

	FMeshNormals* PerVertexNormals = nullptr;

	FDynamicMesh3 ResultMesh;

	double OffsetDistance = 1.0;
	double GetOffsetDistance(int32 VertexID)
	{
		return OffsetDistance;
	}

protected:
	TArray<FVector3d> PositionBuffer;
	FMeshNormals* BaseNormals = nullptr;

	TArray<bool> bIsBoundary;
	TArray<int32> BoundaryVerts;

public:

	FMeshOffset(FDynamicMesh3* MeshIn)
	{
		Mesh = MeshIn;
	}

	virtual ~FMeshOffset() {}

	virtual void Apply(FProgressCancel* Progress)
	{
		BaseNormals = PerVertexNormals;
		FMeshNormals LocalNormals;
		if (PerVertexNormals == nullptr || PerVertexNormals->GetNormals().Num() != Mesh->VertexCount())
		{
			LocalNormals.SetMesh(Mesh);
			LocalNormals.ComputeVertexNormals();
			BaseNormals = &LocalNormals;
		}

		ResultMesh = *Mesh;
		PositionBuffer.SetNum(ResultMesh.MaxVertexID());

		// probably could be more efficient...
		bIsBoundary.Init(false, ResultMesh.MaxVertexID());
		BoundaryVerts.Reset();
		for (int32 vid : ResultMesh.VertexIndicesItr())
		{
			if ((bIsBoundary[vid] = ResultMesh.IsBoundaryVertex(vid)))
			{
				BoundaryVerts.Add(vid);
			}
		}

		if (Steps > 1 && SmoothAlpha > 0)
		{
			Offset_Smoothed(Progress);
		}
		else
		{
			Offset(Progress);
		}

		for (int32 vid : ResultMesh.VertexIndicesItr())
		{
			ResultMesh.SetVertex(vid, PositionBuffer[vid]);
		}
	}

protected:
	void Offset(FProgressCancel* Progress)
	{
		int32 UseSteps = FMath::Max(1, Steps);
		if (UseSteps == 1)
		{
			for (int32 vid : ResultMesh.VertexIndicesItr())
			{
				double Dist = GetOffsetDistance(vid);

				FVector3d OffsetPosition = ResultMesh.GetVertex(vid);
				if (bFixedBoundary == false || bIsBoundary[vid] == false)
				{
					OffsetPosition += Dist * BaseNormals->GetNormals()[vid];
				}
				PositionBuffer[vid] = OffsetPosition;
			}
		}
		else
		{
			FMeshNormals StepNormals(&ResultMesh);
			for (int32 k = 0; k < UseSteps; ++k)
			{
				FMeshNormals& UseNormals = (k == 0) ? *BaseNormals : StepNormals;

				// do offset step
				int32 NumVertices = ResultMesh.MaxVertexID();
				for (int32 vid : ResultMesh.VertexIndicesItr())
				{
					double Dist = GetOffsetDistance(vid);
					double OffsetPerStep = Dist / (double)UseSteps;

					FVector3d OffsetPosition = ResultMesh.GetVertex(vid);
					if (bFixedBoundary == false || bIsBoundary[vid] == false)
					{
						OffsetPosition += OffsetPerStep * UseNormals.GetNormals()[vid];
					}
					PositionBuffer[vid] = OffsetPosition;
				}

				// bake positions and re-calc normals
				for (int32 vid : ResultMesh.VertexIndicesItr())
				{
					ResultMesh.SetVertex(vid, PositionBuffer[vid]);
				}
				StepNormals.ComputeVertexNormals();
			}

			for (int32 vid : ResultMesh.VertexIndicesItr())
			{
				PositionBuffer[vid] = ResultMesh.GetVertex(vid);
			}
		}
	}


	void Offset_Smoothed(FProgressCancel* Progress)
	{
		int32 UseSteps = FMath::Max(1, Steps);
		TArray<FVector3d> SmoothedBuffer = PositionBuffer;

		bool bHasSmoothing = (SmoothAlpha > 0);
		FDynamicMesh3 ProjectMesh;
		FDynamicMeshAABBTree3 Spatial;
		if (bReprojectSmooth && bHasSmoothing)
		{
			ProjectMesh.Copy(ResultMesh, false, false, false, false);
			Spatial.SetMesh(&ProjectMesh, true);
		}

		FMeshNormals StepNormals(&ResultMesh);
		for (int32 k = 0; k < UseSteps; ++k)
		{
			FMeshNormals& UseNormals = (k == 0) ? *BaseNormals : StepNormals;

			// do offset step
			int32 NumVertices = ResultMesh.MaxVertexID();
			for (int32 vid : ResultMesh.VertexIndicesItr())
			{
				double Dist = GetOffsetDistance(vid);
				double OffsetPerStep = Dist / (double)UseSteps;

				FVector3d OffsetPosition = ResultMesh.GetVertex(vid);
				if (bFixedBoundary == false || bIsBoundary[vid] == false)
				{
					OffsetPosition += OffsetPerStep * UseNormals.GetNormals()[vid];
				}
				PositionBuffer[vid] = OffsetPosition;
			}

			if (bReprojectSmooth && bHasSmoothing)
			{
				for (int32 vid : ResultMesh.VertexIndicesItr())
				{
					ProjectMesh.SetVertex(vid, PositionBuffer[vid]);
				}
				Spatial.Build();
			}

			// do smooth step
			if (bHasSmoothing)
			{
				ParallelFor(NumVertices, [&](int32 vid)
				{
					if (ResultMesh.IsReferencedVertex(vid) == false || bIsBoundary[vid])
					{
						SmoothedBuffer[vid] = PositionBuffer[vid];
						return;
					}
					FVector3d Centroid = FMeshWeights::UniformCentroid(ResultMesh, vid, [&](int32 nbrvid) { return PositionBuffer[nbrvid]; });
					SmoothedBuffer[vid] = UE::Geometry::Lerp(PositionBuffer[vid], Centroid, SmoothAlpha);
					if (bReprojectSmooth)
					{
						SmoothedBuffer[vid] = Spatial.FindNearestPoint(SmoothedBuffer[vid]);
					}
				} /*, EParallelForFlags::ForceSingleThread*/);

				if (bFixedBoundary == false)
				{
					ParallelFor(BoundaryVerts.Num(), [&](int32 idx)
					{
						int32 vid = BoundaryVerts[idx];
						FVector3d Centroid = FMeshWeights::FilteredUniformCentroid(ResultMesh, vid,
							[&](int32 nbrvid) { return PositionBuffer[nbrvid]; },
							[&](int32 nbrvid) { return bIsBoundary[nbrvid]; });
						SmoothedBuffer[vid] = UE::Geometry::Lerp(PositionBuffer[vid], Centroid, BoundaryAlpha);
					});
				}

				for (int32 vid : ResultMesh.VertexIndicesItr())
				{
					PositionBuffer[vid] = SmoothedBuffer[vid];
				}
			}

			// bake positions and re-calc normals
			for (int32 vid : ResultMesh.VertexIndicesItr())
			{
				ResultMesh.SetVertex(vid, PositionBuffer[vid]);
			}
			StepNormals.ComputeVertexNormals();
		}

		for (int32 vid : ResultMesh.VertexIndicesItr())
		{
			PositionBuffer[vid] = ResultMesh.GetVertex(vid);
		}
	}

};



namespace UELocal
{
	static void ApplyPolygroupModeToNewTriangles(
		FDynamicMesh3& EditMesh,
		TArray<int32>& NewTriangles,
		FGeometryScriptMeshEditPolygroupOptions GroupOptions)
	{
		if (GroupOptions.GroupMode == EGeometryScriptMeshEditPolygroupMode::PreserveExisting)
		{
			return;
		}
		else if (GroupOptions.GroupMode == EGeometryScriptMeshEditPolygroupMode::SetConstant)
		{
			for (int32 tid : NewTriangles)
			{
				EditMesh.SetTriangleGroup(tid, GroupOptions.ConstantGroup);
			}
		}
		else  // EGeometryScriptMeshEditPolygroupMode::AutoGenerateNew
		{
			TMap<int32, int32> NewGroups;
			for (int32 tid : NewTriangles)
			{
				int32 GroupID = EditMesh.GetTriangleGroup(tid);
				int32* Found = NewGroups.Find(GroupID);
				if (Found == nullptr)
				{
					int32 NewGroupID = EditMesh.AllocateTriangleGroup();
					NewGroups.Add(GroupID, NewGroupID);
					EditMesh.SetTriangleGroup(tid, NewGroupID);
				}
				else
				{
					EditMesh.SetTriangleGroup(tid, *Found);
				}
			}
		}
	}

}





UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshDisconnectFaces(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	bool bAllowBowtiesInOutput,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDisconnectFaces_InvalidInput", "ApplyMeshDisconnectFaces: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TArray<int32> Triangles;
		Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		if (Triangles.Num() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDisconnectFaces_NoTriangles", "ApplyMeshDisconnectFaces: Disconnect Area contains 0 triangles"));
		}
		else
		{
			FDynamicMeshEditor Editor(&EditMesh);
			Editor.DisconnectTriangles(Triangles, !bAllowBowtiesInOutput);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshDisconnectFacesAlongEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDisconnectEdges_InvalidInput", "ApplyMeshDisconnectEdges: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		TArray<int32> Edges;
		Selection.ConvertToMeshIndexArray(EditMesh, Edges, EGeometryScriptIndexType::Edge);
		if (Edges.Num() == 0)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDisconnectFaces_NoEdges", "ApplyMeshDisconnectEdges: Selection contains no edges"));
		}
		else
		{
			FDynamicMeshEditor Editor(&EditMesh);
			TSet<int32> EdgeSet(Edges);
			Editor.DisconnectTrianglesAlongEdges(EdgeSet);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshDuplicateFaces(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptMeshSelection& NewSelection,
	FGeometryScriptMeshEditPolygroupOptions GroupOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDuplicateFaces_InvalidInput", "ApplyMeshDuplicateFaces: TargetMesh is Null"));
		return TargetMesh;
	}

	TArray<int32> NewTriangles;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TArray<int32> Triangles;
		Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		if (Triangles.Num() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshDuplicateFaces_NoTriangles", "ApplyMeshDuplicateFaces: Disconnect Area contains 0 triangles"));
		}
		else
		{
			FDynamicMeshEditor Editor(&EditMesh);
			FMeshIndexMappings Mappings; 
			FDynamicMeshEditResult EditResult;
			Editor.AppendTriangles(&EditMesh, Triangles, Mappings, EditResult);
			NewTriangles = MoveTemp(EditResult.NewTriangles);

			UELocal::ApplyPolygroupModeToNewTriangles(EditMesh, NewTriangles, GroupOptions);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	UGeometryScriptLibrary_MeshSelectionFunctions::ConvertIndexArrayToMeshSelection(
		TargetMesh, NewTriangles, EGeometryScriptMeshSelectionType::Triangles, NewSelection);

	return TargetMesh;
}







UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshOffset(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshOffsetOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshOffset_InvalidInput", "ApplyMeshOffset: TargetMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMeshOffset Offset(&EditMesh);
		Offset.bFixedBoundary = Options.bFixedBoundary;
		Offset.OffsetDistance = Options.OffsetDistance;
		Offset.Steps = FMath::Clamp(Options.SolveSteps, 1, 1000);
		Offset.SmoothAlpha = FMath::Clamp((double)Options.SmoothAlpha, 0, 1.0);
		Offset.bReprojectSmooth = Options.bReprojectDuringSmoothing;
		Offset.BoundaryAlpha = FMath::Clamp((double)Options.BoundaryAlpha, 0, 0.9);

		Offset.Apply(nullptr);
		EditMesh = MoveTemp(Offset.ResultMesh);

		if (EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals() != nullptr)
		{
			FMeshNormals Normals(&EditMesh);
			FDynamicMeshNormalOverlay* NormalOverlay = EditMesh.Attributes()->PrimaryNormals();
			Normals.RecomputeOverlayNormals(NormalOverlay);
			Normals.CopyToOverlay(NormalOverlay);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshShell(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshOffsetOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshShell_InvalidInput", "ApplyMeshShell: TargetMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		UE::Geometry::FMeshBoundaryLoops BoundaryLoops(&EditMesh, true);
		
		double OffsetSign = (Options.OffsetDistance >= 0) ? 1 : -1;

		FDynamicMesh3 InnerMesh;
		InnerMesh.Copy(EditMesh);
		bool bIsPositiveOffset = (OffsetSign >= 0);
		if (bIsPositiveOffset)
		{
			InnerMesh.ReverseOrientation();
		}

		FMeshOffset Offset(&EditMesh);
		Offset.bFixedBoundary = Options.bFixedBoundary;
		Offset.OffsetDistance = Options.OffsetDistance;
		Offset.Steps = FMath::Clamp(Options.SolveSteps, 1, 1000);
		Offset.SmoothAlpha = FMath::Clamp((double)Options.SmoothAlpha, 0, 1.0);
		Offset.bReprojectSmooth = Options.bReprojectDuringSmoothing;
		Offset.BoundaryAlpha = FMath::Clamp((double)Options.BoundaryAlpha, 0, 0.9);

		Offset.Apply(nullptr);
		EditMesh = MoveTemp(Offset.ResultMesh);

		if (!bIsPositiveOffset)
		{
			EditMesh.ReverseOrientation();
		}

		UE::Geometry::FDynamicMeshEditor Editor(&EditMesh);
		UE::Geometry::FMeshIndexMappings MeshMap;
		Editor.AppendMesh(&InnerMesh, MeshMap);

		bool bWeldStitch = Options.bFixedBoundary;

		// join the boundary loops by weld or stitch
		for (UE::Geometry::FEdgeLoop& BaseLoop : BoundaryLoops.Loops)
		{
			int32 LoopCount = BaseLoop.GetVertexCount();
			TArray<int32> OffsetLoop;
			OffsetLoop.SetNum(LoopCount);
			for (int k = 0; k < LoopCount; ++k)
			{
				OffsetLoop[k] = MeshMap.GetNewVertex(BaseLoop.Vertices[k]);
			}

			TArray<int32>& LoopA = (bIsPositiveOffset) ? BaseLoop.Vertices : OffsetLoop;
			TArray<int32>& LoopB = (bIsPositiveOffset) ? OffsetLoop : BaseLoop.Vertices;

			if (bWeldStitch)
			{
				bool bStitchSuccess = Editor.WeldVertexLoops(LoopA, LoopB);
			}
			else
			{
				FJoinMeshLoops Join(&EditMesh, LoopA, LoopB);
				Join.Apply();
			}
		}

		if (EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals() != nullptr)
		{
			FMeshNormals Normals(&EditMesh);
			FDynamicMeshNormalOverlay* NormalOverlay = EditMesh.Attributes()->PrimaryNormals();
			Normals.RecomputeOverlayNormals(NormalOverlay);
			Normals.CopyToOverlay(NormalOverlay);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshExtrude_Compatibility_5p0(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshExtrudeOptions Options,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptMeshLinearExtrudeOptions ConvertOptions;
	ConvertOptions.Distance = Options.ExtrudeDistance;
	ConvertOptions.Direction = Options.ExtrudeDirection;
	ConvertOptions.UVScale = Options.UVScale;
	ConvertOptions.bSolidsToShells = Options.bSolidsToShells;

	return ApplyMeshLinearExtrudeFaces(TargetMesh, ConvertOptions, FGeometryScriptMeshSelection(), Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshLinearExtrudeFaces(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshLinearExtrudeOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshLinearExtrudeFaces_InvalidInput", "ApplyMeshLinearExtrudeFaces: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		auto ApplyExtrudeArea = [&EditMesh, &Options](TArray<int32>&& Triangles)
		{
			FOffsetMeshRegion Extruder(&EditMesh);
			Extruder.Triangles = MoveTemp(Triangles);
			if (Extruder.Triangles.Num() > 0)
			{
				FVector3d UseDirection = Options.Direction;
				if (Options.DirectionMode == EGeometryScriptLinearExtrudeDirection::AverageFaceNormal)
				{
					UseDirection = FVector3d::Zero();
					for (int32 tid : Extruder.Triangles)
					{
						FVector3d Normal, Centroid; double Area;
						EditMesh.GetTriInfo(tid, Normal, Area, Centroid);
						UseDirection += Area * Normal;
					}
					if (Normalize(UseDirection) <= 0)
					{
						UseDirection = Options.Direction;
					}
				}
				UseDirection *= Options.Distance;

				Extruder.OffsetPositionFunc = [UseDirection](const FVector3d& Position, const FVector3d& VertexVector, int VertexID)
				{
					return Position + UseDirection;
				};
				Extruder.bIsPositiveOffset = (Options.Distance > 0);
				Extruder.UVScaleFactor = Options.UVScale;
				Extruder.bOffsetFullComponentsAsSolids = Options.bSolidsToShells;
				Extruder.Apply();
				for (FOffsetMeshRegion::FOffsetInfo& OffsetInfo : Extruder.OffsetRegions)
				{
					UELocal::ApplyPolygroupModeToNewTriangles(EditMesh, OffsetInfo.OffsetTids, Options.GroupOptions);
				}
			}
		};

		TArray<int32> Triangles;
		if (Selection.GetNumSelected() == 0)
		{
			for (int32 tid : EditMesh.TriangleIndicesItr())
			{
				Triangles.Add(tid);
			}
		}
		else
		{
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		}

		if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerPolygroup)
		{
			TMap<int32, TArray<int32>> PolygroupTris;
			for (int32 tid : Triangles)
			{
				int32 gid = EditMesh.GetTriangleGroup(tid);
				TArray<int32>* Found = PolygroupTris.Find(gid);
				if (Found != nullptr)
				{
					Found->Add(tid);
				}
				else
				{
					TArray<int32> Temp;
					Temp.Add(tid);
					PolygroupTris.Add(gid, Temp);
				}
			}

			for (TPair<int32, TArray<int32>> Pair : PolygroupTris)
			{
				ApplyExtrudeArea(MoveTemp(Pair.Value));
			}
		}
		else if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerTriangle)
		{
			for (int32 tid : Triangles)
			{
				TArray<int32> Temp;
				Temp.Add(tid);
				ApplyExtrudeArea(MoveTemp(Temp));
			}
		}
		else
		{
			ApplyExtrudeArea(MoveTemp(Triangles));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshOffsetFaces(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshOffsetFacesOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshOffsetFaces_InvalidInput", "ApplyMeshOffsetFaces: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		auto ApplyOffsetArea = [&EditMesh, &Options](TArray<int32>&& Triangles)
		{
			FOffsetMeshRegion Extruder(&EditMesh);
			Extruder.Triangles = MoveTemp(Triangles);
			if (Extruder.Triangles.Num() > 0)
			{
				Extruder.OffsetPositionFunc = [Options](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID) {
					return Pos + Options.Distance * VertexVector;
				};

				if (Options.OffsetType == EGeometryScriptOffsetFacesType::VertexNormal)
				{
					Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::VertexNormal;
				}
				else if (Options.OffsetType == EGeometryScriptOffsetFacesType::FaceNormal)
				{
					Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage;
				}
				else
				{
					Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted;
				}

				Extruder.DefaultOffsetDistance = Options.Distance;
				Extruder.bIsPositiveOffset = (Options.Distance > 0);
				Extruder.UVScaleFactor = Options.UVScale;
				Extruder.bOffsetFullComponentsAsSolids = Options.bSolidsToShells;
				Extruder.Apply();
				for (FOffsetMeshRegion::FOffsetInfo& OffsetInfo : Extruder.OffsetRegions)
				{
					UELocal::ApplyPolygroupModeToNewTriangles(EditMesh, OffsetInfo.OffsetTids, Options.GroupOptions);
				}
			}
		};

		TArray<int32> Triangles;
		if (Selection.GetNumSelected() == 0)
		{
			for (int32 tid : EditMesh.TriangleIndicesItr())
			{
				Triangles.Add(tid);
			}
		}
		else
		{
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		}

		if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerPolygroup)
		{
			TMap<int32, TArray<int32>> PolygroupTris;
			for (int32 tid : Triangles)
			{
				int32 gid = EditMesh.GetTriangleGroup(tid);
				TArray<int32>* Found = PolygroupTris.Find(gid);
				if (Found != nullptr)
				{
					Found->Add(tid);
				}
				else
				{
					TArray<int32> Temp;
					Temp.Add(tid);
					PolygroupTris.Add(gid, Temp);
				}
			}

			for (TPair<int32, TArray<int32>> Pair : PolygroupTris)
			{
				ApplyOffsetArea(MoveTemp(Pair.Value));
			}
		}
		else if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerTriangle)
		{
			for (int32 tid : Triangles)
			{
				TArray<int32> Temp;
				Temp.Add(tid);
				ApplyOffsetArea(MoveTemp(Temp));
			}
		}
		else
		{
			ApplyOffsetArea(MoveTemp(Triangles));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshInsetOutsetFaces(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshInsetOutsetFacesOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshInsetOutsetFaces_InvalidInput", "ApplyMeshInsetOutsetFaces: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		auto ApplyInsetOutsetArea = [&EditMesh, &Options](TArray<int32>&& Triangles)
		{
			FInsetMeshRegion InsetOutset(&EditMesh);
			InsetOutset.Triangles = MoveTemp(Triangles);
			if (InsetOutset.Triangles.Num() > 0)
			{
				InsetOutset.InsetDistance = Options.Distance;
				InsetOutset.bReproject = (Options.Distance < 0) ? false : Options.bReproject;
				InsetOutset.Softness = Options.Softness;
				InsetOutset.bSolveRegionInteriors = !Options.bBoundaryOnly;
				InsetOutset.AreaCorrection = Options.AreaScale;
				InsetOutset.Apply();

				for (FInsetMeshRegion::FInsetInfo& InsetInfo : InsetOutset.InsetRegions)
				{
					UELocal::ApplyPolygroupModeToNewTriangles(EditMesh, InsetInfo.InitialTriangles, Options.GroupOptions);
				}
			}
		};

		TArray<int32> Triangles;
		if (Selection.GetNumSelected() == 0)
		{
			for (int32 tid : EditMesh.TriangleIndicesItr())
			{
				Triangles.Add(tid);
			}
		}
		else
		{
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		}

		if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerPolygroup)
		{
			TMap<int32, TArray<int32>> PolygroupTris;
			for (int32 tid : Triangles)
			{
				int32 gid = EditMesh.GetTriangleGroup(tid);
				TArray<int32>* Found = PolygroupTris.Find(gid);
				if (Found != nullptr)
				{
					Found->Add(tid);
				}
				else
				{
					TArray<int32> Temp;
					Temp.Add(tid);
					PolygroupTris.Add(gid, Temp);
				}
			}

			for (TPair<int32, TArray<int32>> Pair : PolygroupTris)
			{
				ApplyInsetOutsetArea(MoveTemp(Pair.Value));
			}
		}
		else if (Options.AreaMode == EGeometryScriptPolyOperationArea::PerTriangle)
		{
			for (int32 tid : Triangles)
			{
				TArray<int32> Temp;
				Temp.Add(tid);
				ApplyInsetOutsetArea(MoveTemp(Temp));
			}
		}
		else
		{
			ApplyInsetOutsetArea(MoveTemp(Triangles));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshBevelEdgeSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptMeshBevelSelectionOptions BevelOptions,
	UGeometryScriptDebug* Debug)
{
	return ApplyMeshBevelSelection(
		TargetMesh,
		Selection,
		EGeometryScriptMeshBevelSelectionMode::SelectedEdges,
		BevelOptions,
		Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshBevelSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	EGeometryScriptMeshBevelSelectionMode BevelMode,
	FGeometryScriptMeshBevelSelectionOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshBevelSelection_InvalidInput", "ApplyMeshBevelSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FMeshBevel Bevel;
		Bevel.InsetDistance = Options.BevelDistance;
		Bevel.MaterialIDMode = (Options.bInferMaterialID) ? FMeshBevel::EMaterialIDMode::InferMaterialID : FMeshBevel::EMaterialIDMode::ConstantMaterialID;
		Bevel.SetConstantMaterialID = Options.SetMaterialID;
		Bevel.NumSubdivisions = FMath::Clamp(Options.Subdivisions, 0, 9999);
		Bevel.RoundWeight = FMath::Clamp(Options.RoundWeight, -10.0, 10.0);

		TUniquePtr<FGroupTopology> GroupTopology;

		if (BevelMode == EGeometryScriptMeshBevelSelectionMode::SelectedEdges)
		{
			TArray<int32> Edges;
			Selection.ConvertToMeshIndexArray(EditMesh, Edges, EGeometryScriptIndexType::Edge);
			// Note: Using group junctions as corner vertices tends to help the bevel match the result of a similar polygroup bevel in the UI
			// (though is often very similar to the no-corner bevel; necessary corners where more than 2 selected edges meet are automatically detected)
			if (EditMesh.HasTriangleGroups())
			{
				Bevel.InitializeFromTriangleEdges(EditMesh, Edges, [&EditMesh](int32 VID)
					{
						return EditMesh.IsGroupJunctionVertex(VID);
					});
			}
			else
			{
				Bevel.InitializeFromTriangleEdges(EditMesh, Edges, [](int32) {return false;});
			}
			
		}
		else if (BevelMode == EGeometryScriptMeshBevelSelectionMode::TriangleArea)
		{
			TArray<int32> Triangles;
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
			Bevel.InitializeFromTriangleSet(EditMesh, Triangles);
		}
		else
		{
			int32 NumRequiredAdjacentGroups = (BevelMode == EGeometryScriptMeshBevelSelectionMode::SharedPolygroupEdges) ? 2 : 1;

			TArray<int32> GroupIDs;
			Selection.ConvertToMeshIndexArray(EditMesh, GroupIDs, EGeometryScriptIndexType::PolygroupID);
			GroupTopology = MakeUnique<FGroupTopology>(&EditMesh, true);
			TArray<int32> GroupEdgeIDs;
			int32 NumEdges = GroupTopology->Edges.Num();
			for ( int32 GroupEdgeID = 0; GroupEdgeID < NumEdges; ++GroupEdgeID )
			{
				FIndex2i EdgeGroupIDs = GroupTopology->Edges[GroupEdgeID].Groups;
				int32 NumAdjacentGroups = (GroupIDs.Contains(EdgeGroupIDs.A) ? 1 : 0) + (GroupIDs.Contains(EdgeGroupIDs.B) ? 1 : 0);
				if (NumAdjacentGroups >= NumRequiredAdjacentGroups )
				{
					GroupEdgeIDs.Add(GroupEdgeID);
				}
			}
			Bevel.InitializeFromGroupTopologyEdges(EditMesh, *GroupTopology, GroupEdgeIDs);
		}

		Bevel.Apply(EditMesh, nullptr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}






UDynamicMesh* UGeometryScriptLibrary_MeshModelingFunctions::ApplyMeshPolygroupBevel(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshBevelOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshPolygroupBevel_InvalidInput", "ApplyMeshPolygroupBevel: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		UE::Geometry::FGroupTopology Topology(&EditMesh, true);

		TArray<int32> BevelGroupEdges;
		if (Options.bApplyFilterBox)
		{
			FAxisAlignedBox3d QueryBox(Options.FilterBox);
			TSet<int32> FoundEdges;

			// find all mesh edges inside filter shape
			// TODO: this is hardcoded for what is supported in 5.0, ie only 3D boxes, but should be generalized
			FDynamicMeshAABBTree3 Spatial(&EditMesh, true);
			FDynamicMeshAABBTree3::FTreeTraversal EdgeTraversal;
			EdgeTraversal.NextBoxF = [&QueryBox](const FAxisAlignedBox3d& Box, int Depth) { return Box.Intersects(QueryBox); };
			EdgeTraversal.NextTriangleF = [&QueryBox, &FoundEdges, &EditMesh, &Options](int TriangleID)
			{
				FIndex3i Edges = EditMesh.GetTriEdges(TriangleID);
				for (int32 j = 0; j < 3; ++j)
				{
					if (FoundEdges.Contains(Edges[j]) == false && EditMesh.IsGroupBoundaryEdge(Edges[j]))
					{
						FVector3d A, B;
						EditMesh.GetEdgeV(Edges[j], A, B);
						A = Options.FilterBoxTransform.InverseTransformPosition(A);
						B = Options.FilterBoxTransform.InverseTransformPosition(B);
						bool bIntersects = (Options.bFullyContained) ?
							(QueryBox.Contains(A) && QueryBox.Contains(B)) :
							(QueryBox.Contains(A) || QueryBox.Contains(B));
						if (bIntersects)
						{
							FoundEdges.Add(Edges[j]);
						}
					}
				}
			};
			Spatial.DoTraversal(EdgeTraversal);

			// convert mesh edges to group topology edges
			TSet<int32> GroupEdges;
			for (int32 MeshEdgeID : FoundEdges)
			{
				int32 GroupEdgeID = Topology.FindGroupEdgeID(MeshEdgeID);
				if (GroupEdgeID >= 0)
				{
					GroupEdges.Add(GroupEdgeID);
				}
			}

			// if exclusive was requested we need to check that the entire edge is inside the box
			if (Options.bFullyContained)
			{
				for (int32 GroupEdgeID : GroupEdges)
				{
					const TArray<int>& EdgeIDs = Topology.GetGroupEdgeEdges(GroupEdgeID);
					bool bFoundMissing = false;
					for (int32 MeshEdgeID : EdgeIDs)
					{
						if (FoundEdges.Contains(MeshEdgeID) == false)
						{
							bFoundMissing = true;
							break;
						}
					}
					if (bFoundMissing == false)
					{
						BevelGroupEdges.Add(GroupEdgeID);
					}
				}
			}
			else
			{
				BevelGroupEdges = GroupEdges.Array();
			}

			if (BevelGroupEdges.Num() == 0)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshPolygroupBevel_FilterIsEmpty", "ApplyMeshPolygroupBevel: Filter box does not contain any Polygroup Edges, bevel will not be applied"));
				return;
			}
		}

		UE::Geometry::FMeshBevel Bevel;
		Bevel.InsetDistance = Options.BevelDistance;
		Bevel.MaterialIDMode = (Options.bInferMaterialID) ? FMeshBevel::EMaterialIDMode::InferMaterialID : FMeshBevel::EMaterialIDMode::ConstantMaterialID;
		Bevel.SetConstantMaterialID = Options.SetMaterialID;
		Bevel.NumSubdivisions = FMath::Clamp(Options.Subdivisions, 0, 9999);
		Bevel.RoundWeight = FMath::Clamp(Options.RoundWeight, -10.0, 10.0);
		if (BevelGroupEdges.Num() > 0)
		{
			Bevel.InitializeFromGroupTopologyEdges(EditMesh, Topology, BevelGroupEdges);
		}
		else
		{
			Bevel.InitializeFromGroupTopology(EditMesh, Topology);
		}
		Bevel.Apply(EditMesh, nullptr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




#undef LOCTEXT_NAMESPACE

