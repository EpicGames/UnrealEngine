// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshNormalsFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMeshMikkTWrapper.h"
#include "DynamicMeshEditor.h" // for sharp edge creation/deletion via overlay seam methods
#include "Operations/RepairOrientation.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Polygroups/PolygroupSet.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshNormalsFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshNormalsFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::FlipNormals( UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug )
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FlipNormals_InvalidInput", "FlipNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		EditMesh.ReverseOrientation(true);

		if (EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals() != nullptr)
		{
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			for (int elemid : Normals->ElementIndicesItr())
			{
				FVector3f Normal = Normals->GetElement(elemid);
				Normals->SetElement(elemid, -Normal);
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::FlipTriangleSelectionNormals(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection, 
	bool bFlipTriangleOrientation,
	bool bFlipNormalDirection, 
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FlipTriangleSelectionNormals_InvalidInput", "FlipTriangleSelectionNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// Return early if no flipping was requested
	if (!bFlipTriangleOrientation && !bFlipNormalDirection)
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&Selection, bFlipTriangleOrientation, bFlipNormalDirection](FDynamicMesh3& EditMesh)
	{
		TSet<int32> FlippedVIDs, FlippedEIDs;
		FDynamicMeshNormalOverlay* Normals = EditMesh.HasAttributes() ? EditMesh.Attributes()->PrimaryNormals() : nullptr;


		Selection.ProcessByTriangleID(EditMesh, 
		[&EditMesh, &FlippedVIDs, &FlippedEIDs, Normals, bFlipTriangleOrientation, bFlipNormalDirection](int32 TriangleID)
		{
			if (bFlipTriangleOrientation)
			{
				EditMesh.ReverseTriOrientation(TriangleID);
			}

			if (bFlipNormalDirection)
			{
				if (EditMesh.HasVertexNormals())
				{
					FIndex3i Tri = EditMesh.GetTriangle(TriangleID);
					for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
					{
						int32 VertexID = Tri[SubIdx];
						bool bWasInSet;
						FlippedVIDs.Add(VertexID, &bWasInSet);
						if (!bWasInSet)
						{
							EditMesh.SetVertexNormal(VertexID, -EditMesh.GetVertexNormal(VertexID));
						}
					}
				}
				if (Normals)
				{
					FIndex3i TriEls;
					if (Normals->GetTriangleIfValid(TriangleID, TriEls))
					{
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							int32 ElementID = TriEls[SubIdx];
							bool bWasInSet;
							FlippedEIDs.Add(ElementID, &bWasInSet);
							if (!bWasInSet)
							{
								Normals->SetElement(ElementID, -Normals->GetElement(ElementID));
							}
						}
					}
				}
			}
		}, false /*bProcessAllTrisIfSelectionEmpty*/);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::AutoRepairNormals(UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoRepairNormals_InvalidInput", "AutoRepairNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMeshRepairOrientation Repair(&EditMesh);
		Repair.OrientComponents();
		FDynamicMeshAABBTree3 Tree(&EditMesh, true);
		Repair.SolveGlobalOrientation(&Tree);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetPerVertexNormals(UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetPerVertexNormals_InvalidInput", "SetPerVertexNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		FMeshNormals::InitializeOverlayToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), false);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetPerFaceNormals(UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetPerFaceNormals_InvalidInput", "SetPerFaceNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMeshNormals::InitializeMeshToPerTriangleNormals(&EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::RecomputeNormals(
	UDynamicMesh* TargetMesh,
	FGeometryScriptCalculateNormalsOptions CalculateOptions,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeNormals_InvalidInput", "RecomputeNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		FMeshNormals MeshNormals(&EditMesh);
		if (EditMesh.Attributes()->PrimaryNormals()->ElementCount() == 0)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeNormals_NothingToRecompute", "RecomputeNormals: TargetMesh did not have normals to recompute; falling back to per-vertex normals. Consider using 'Set Mesh To Per Vertex Normals' or 'Compute Split Normals' instead."));
			EditMesh.Attributes()->PrimaryNormals()->CreatePerVertex(0.0f);
		}
		MeshNormals.RecomputeOverlayNormals(EditMesh.Attributes()->PrimaryNormals(), CalculateOptions.bAreaWeighted, CalculateOptions.bAngleWeighted);
		MeshNormals.CopyToOverlay(EditMesh.Attributes()->PrimaryNormals(), false);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::RecomputeNormalsForMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptCalculateNormalsOptions CalculateOptions,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeNormalsForMeshSelection_InvalidInput", "RecomputeNormalsForMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || EditMesh.Attributes()->PrimaryNormals() == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeNormalsForMeshSelection_NoAttributes", "RecomputeNormalsForMeshSelection: TargetMesh has no Normals attribute enabled"));
			return;
		}
		
		EGeometryScriptMeshSelectionType SelectionType = Selection.GetSelectionType();
		if (SelectionType == EGeometryScriptMeshSelectionType::Vertices || SelectionType == EGeometryScriptMeshSelectionType::Edges)
		{
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			TSet<int32> Elements;
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID) {
				Normals->EnumerateVertexElements(VertexID, [&](int32 tid, int32 elemid, const FVector3f&) { Elements.Add(elemid); return true; }, /*bFindUniqueElements*/false);
			}, /*bProcessAllVertsIfSelectionEmpty*/false);
			FMeshNormals::RecomputeOverlayElementNormals(EditMesh, Elements.Array(), CalculateOptions.bAreaWeighted, CalculateOptions.bAngleWeighted);
		}
		else
		{
			TArray<int32> Triangles;
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
			FMeshNormals::RecomputeOverlayTriNormals(EditMesh, Triangles, CalculateOptions.bAreaWeighted, CalculateOptions.bAngleWeighted);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetSplitNormalsAlongSelectedEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	bool bSplit,
	bool bRecalculateNormals,
	FGeometryScriptCalculateNormalsOptions CalculateOptions,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSplitNormalsAlongSelectedEdges_InvalidInput", "SetSplitNormalsAlongSelectedEdges: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() == false || EditMesh.Attributes()->PrimaryNormals() == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSplitNormalsAlongSelectedEdges_NoAttributes", "SetSplitNormalsAlongSelectedEdges: TargetMesh has no Normals attribute enabled"));
			return;
		}

		TArray<int32> EdgeArr;
		Selection.ConvertToMeshIndexArray(EditMesh, EdgeArr, EGeometryScriptIndexType::Edge);
		TSet<int32> EdgeSet(EdgeArr);
		// split/merge sharp normals (and tangents/bitangents if available)
		for (int32 NormalLayerIdx = 0; NormalLayerIdx < EditMesh.Attributes()->NumNormalLayers(); ++NormalLayerIdx)
		{
			if (bSplit)
			{
				FDynamicMeshEditor::CreateSeamsAtEdges(EdgeSet, EditMesh.Attributes()->GetNormalLayer(NormalLayerIdx));
			}
			else
			{
				FDynamicMeshEditor::RemoveSeamsAtEdges(EdgeSet, EditMesh.Attributes()->GetNormalLayer(NormalLayerIdx));
			}
		}
		if (bRecalculateNormals)
		{
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			TSet<int32> VIDs;
			TArray<int32> Elements;
			auto ProcessVert = [&VIDs, &Elements, Normals](int32 VID)
			{
				bool bAlreadyAdded = false;
				VIDs.Add(VID, &bAlreadyAdded);
				if (!bAlreadyAdded)
				{
					Normals->EnumerateVertexElements(VID, [&Elements](int32 TID, int32 ElID, const FVector3f&)
						{
							Elements.Add(ElID);
							return true;
						}, /*bFindUniqueElements*/true);
				}
			};
			for (int32 EID : EdgeArr)
			{
				FIndex2i EdgeV = EditMesh.GetEdgeV(EID);
				ProcessVert(EdgeV.A);
				ProcessVert(EdgeV.B);
			}
			FMeshNormals::RecomputeOverlayElementNormals(EditMesh, Elements, CalculateOptions.bAngleWeighted, CalculateOptions.bAngleWeighted);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::NormalsTangents, bDeferChangeNotifications);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::ComputeSplitNormals( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptSplitNormalsOptions SplitOptions, 
	FGeometryScriptCalculateNormalsOptions CalculateOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeSplitNormals_InvalidInput", "ComputeSplitNormals: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}

		float NormalDotProdThreshold = FMathf::Cos(SplitOptions.OpeningAngleDeg * FMathf::DegToRad);
		FMeshNormals FaceNormals(&EditMesh);
		FaceNormals.ComputeTriangleNormals();
		const TArray<FVector3d>& Normals = FaceNormals.GetNormals();

		TUniquePtr<FPolygroupSet> SplitByGroups;
		if (SplitOptions.bSplitByFaceGroup)
		{
			FPolygroupLayer InputGroupLayer{ SplitOptions.GroupLayer.bDefaultLayer, SplitOptions.GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh) == false)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeSplitNormals_MissingGroups", "ComputeSplitNormals: Polygroup Layer does not exist"));
				return;
			}
			SplitByGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);
		}

		// create new normal overlay topology based on splitting criteria
		EditMesh.Attributes()->PrimaryNormals()->CreateFromPredicate([&](int VID, int TA, int TB)
		{
			if (SplitOptions.bSplitByOpeningAngle && Normals[TA].Dot(Normals[TB]) < NormalDotProdThreshold)
			{
				return false;
			}
			if (SplitOptions.bSplitByFaceGroup && SplitByGroups->GetTriangleGroup(TA) != SplitByGroups->GetTriangleGroup(TB))
			{
				return false;
			}
			return true;

		}, 0);

		// recompute the normals in the overlay
		FMeshNormals RecomputeMeshNormals(&EditMesh);
		RecomputeMeshNormals.RecomputeOverlayNormals(EditMesh.Attributes()->PrimaryNormals(), CalculateOptions.bAreaWeighted, CalculateOptions.bAngleWeighted);
		RecomputeMeshNormals.CopyToOverlay(EditMesh.Attributes()->PrimaryNormals(), false);

		// todo: bAllowSharpVertices from EditNormalsOp?

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::GetMeshHasTangents(
	UDynamicMesh* TargetMesh,
	bool& bHasTangents,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshHasTangents_InvalidInput", "GetMeshHasTangents: TargetMesh is Null"));
		return TargetMesh;
	}
	bHasTangents = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->HasTangentSpace())
		{
			bHasTangents = true;
		}
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::DiscardTangents(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DiscardTangents_InvalidInput", "DiscardTangents: TargetMesh is Null"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() && EditMesh.Attributes()->HasTangentSpace())
		{
			EditMesh.Attributes()->DisableTangents();
		}
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::ComputeTangents( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptTangentsOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeTangents_InvalidInput", "ComputeTangents: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || !(Options.UVLayer < EditMesh.Attributes()->NumUVLayers()) || EditMesh.Attributes()->NumNormalLayers() == 0 )
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeTangents_NoUVLayers", "ComputeTangents: TargetMesh is missing UV Set or Normals required to compute Tangents"));
			return;
		}
		if (EditMesh.Attributes()->HasTangentSpace() == false)
		{
			EditMesh.Attributes()->EnableTangents();
		}

		if (Options.Type == EGeometryScriptTangentTypes::StandardMikkT)
		{
			if (DynamicMeshMikkTWrapper::IsSupported())
			{
				FMeshTangentsd Tangents(&EditMesh);
				if (DynamicMeshMikkTWrapper::ComputeTangents(Tangents, Options.UVLayer))
				{
					Tangents.CopyToOverlays(EditMesh);
				}
				else
				{
					UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ComputeTangents_MikkTFailed", "ComputeTangents: Standard MikkT failed to compute a result"));
				}
				return;
			}
			else
			{
				UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeTangents_StandardMikkTEditorOnly", "ComputeTangents: Standard MikkT not supported by current platform; falling back to Fast MikkT."));
				Options.Type = EGeometryScriptTangentTypes::FastMikkT;
				// continue through below to compute FastMikkT
			}
		}

		FComputeTangentsOptions TangentOptions;
		TangentOptions.bAveraged = (Options.Type == EGeometryScriptTangentTypes::FastMikkT);

		FMeshTangentsd Tangents(&EditMesh);
		Tangents.ComputeTriVertexTangents(
			EditMesh.Attributes()->PrimaryNormals(),
			EditMesh.Attributes()->GetUVLayer(Options.UVLayer),
			TangentOptions);
		Tangents.CopyToOverlays(EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}






UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetMeshTriangleNormals(
	UDynamicMesh* TargetMesh,
	int TriangleID,
	FGeometryScriptTriangle Normals,
	bool& bIsValidTriangle,
	bool bDeferChangeNotifications)
{
	bIsValidTriangle = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsTriangle(TriangleID) && EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals() != nullptr)
			{
				bIsValidTriangle = true;
				FDynamicMeshNormalOverlay* NormalOverlay = EditMesh.Attributes()->PrimaryNormals();
				int32 Elem0 = NormalOverlay->AppendElement((FVector3f)Normals.Vector0);
				int32 Elem1 = NormalOverlay->AppendElement((FVector3f)Normals.Vector1);
				int32 Elem2 = NormalOverlay->AppendElement((FVector3f)Normals.Vector2);
				NormalOverlay->SetTriangle(TriangleID, FIndex3i(Elem0, Elem1, Elem2), true);
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}




UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetMeshPerVertexNormals(
	UDynamicMesh* TargetMesh,
	FGeometryScriptVectorList VertexNormalList,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexNormals_InvalidMesh", "SetMeshPerVertexNormals: TargetMesh is Null"));
		return TargetMesh;
	}
	if (VertexNormalList.List.IsValid() == false || VertexNormalList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexNormals_InvalidList", "SetMeshPerVertexNormals: List is empty"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		const TArray<FVector>& VertexNormals = *VertexNormalList.List;
		if (VertexNormals.Num() < EditMesh.MaxVertexID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexNormals_IncorrectCount", "SetMeshPerVertexNormals: size of provided VertexNormalList is smaller than MaxVertexID of Mesh"));
		}
		else
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			Normals->ClearElements();
			TArray<int32> ElemIDs;
			ElemIDs.SetNum(EditMesh.MaxVertexID());
			for (int32 VertexID : EditMesh.VertexIndicesItr())
			{
				const FVector& Normal = VertexNormals[VertexID];
				ElemIDs[VertexID] = Normals->AppendElement((FVector3f)Normal);
			}
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
				Normals->SetTriangle(TriangleID, FIndex3i(ElemIDs[Triangle.A], ElemIDs[Triangle.B], ElemIDs[Triangle.C]) );
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::GetMeshPerVertexNormals(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptVectorList& NormalList, 
	bool& bIsValidNormalSet,
	bool& bHasVertexIDGaps,
	bool bAverageSplitVertexValues)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "GetMeshPerVertexNormals: TargetMesh is Null");
		return TargetMesh;
	}

	NormalList.Reset();
	TArray<FVector>& Normals = *NormalList.List;
	bHasVertexIDGaps = false;
	bIsValidNormalSet = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		bHasVertexIDGaps = ! ReadMesh.IsCompactV();

		if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->NumNormalLayers() > 0 )
		{
			FMeshNormals MeshNormals(&ReadMesh);
			MeshNormals.GetVertexNormalsFromOverlayNormals(bAverageSplitVertexValues ? FMeshNormals::ECombineSplitNormalsMethod::Average : FMeshNormals::ECombineSplitNormalsMethod::CopyAny);
			Normals = MeshNormals.MoveNormals();

			bIsValidNormalSet = true;
		}
	});

	return TargetMesh;
}






UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::SetMeshPerVertexTangents(
	UDynamicMesh* TargetMesh,
	FGeometryScriptVectorList TangentXList,
	FGeometryScriptVectorList TangentYList,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexTangents_InvalidMesh", "SetMeshPerVertexTangents: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TangentXList.List.IsValid() == false || TangentXList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexTangents_InvalidList", "SetMeshPerVertexTangents: TangentXList is empty"));
		return TargetMesh;
	}
	if (TangentYList.List.IsValid() == false || TangentYList.List->Num() != TangentXList.List->Num())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexTangents_InvalidListY", "SetMeshPerVertexTangents: TangentYList must be the same size as TangentXList"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		const TArray<FVector>& TangentsX = *TangentXList.List;
		const TArray<FVector>& TangentsY = *TangentYList.List;
		if (TangentsX.Num() < EditMesh.MaxVertexID() )
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexTangents_IncorrectCount", "SetMeshPerVertexTangents: size of provided TangentXList is smaller than MaxVertexID of Mesh"));
		}
		else
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}
			if (EditMesh.Attributes()->HasTangentSpace() == false)
			{
				EditMesh.Attributes()->EnableTangents();
			}
			FDynamicMeshNormalOverlay* TangentsOverlayX = EditMesh.Attributes()->PrimaryTangents();
			FDynamicMeshNormalOverlay* TangentsOverlayY = EditMesh.Attributes()->PrimaryBiTangents();
			TangentsOverlayX->ClearElements();
			TangentsOverlayY->ClearElements();
			TArray<int32> ElemIDs;
			ElemIDs.SetNum(EditMesh.MaxVertexID());
			for (int32 k = 0; k < 2; ++k)
			{
				const TArray<FVector>& UseList = (k == 0) ? TangentsX : TangentsY;
				FDynamicMeshNormalOverlay* UseOverlay = (k == 0) ? TangentsOverlayX : TangentsOverlayY;
				for (int32 VertexID : EditMesh.VertexIndicesItr())
				{
					const FVector& Tangent = UseList[VertexID];
					ElemIDs[VertexID] = UseOverlay->AppendElement((FVector3f)Tangent);
				}
				for (int32 TriangleID : EditMesh.TriangleIndicesItr())
				{
					FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
					UseOverlay->SetTriangle(TriangleID, FIndex3i(ElemIDs[Triangle.A], ElemIDs[Triangle.B], ElemIDs[Triangle.C]));
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::GetMeshPerVertexTangents(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptVectorList& TangentXList,
	FGeometryScriptVectorList& TangentYList,
	bool& bIsValidTangentSet,
	bool& bHasVertexIDGaps,
	bool bAverageSplitVertexValues)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "GetMeshPerVertexTangents: TargetMesh is Null");
		return TargetMesh;
	}

	TangentXList.Reset();
	TangentYList.Reset();
	TArray<FVector>& TangentsX = *TangentXList.List;
	TArray<FVector>& TangentsY = *TangentYList.List;
	bHasVertexIDGaps = false;
	bIsValidTangentSet = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		bHasVertexIDGaps = ! ReadMesh.IsCompactV();

		if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->HasTangentSpace() )
		{
			const FDynamicMeshNormalOverlay* TangentXOverlay = ReadMesh.Attributes()->PrimaryTangents();
			const FDynamicMeshNormalOverlay* TangentYOverlay = ReadMesh.Attributes()->PrimaryBiTangents();

			if (bAverageSplitVertexValues)
			{
				TangentsX.Init(FVector::Zero(), ReadMesh.MaxVertexID());
				TangentsY.Init(FVector::Zero(), ReadMesh.MaxVertexID());
				for (int32 tid : ReadMesh.TriangleIndicesItr())
				{
					if (TangentXOverlay->IsSetTriangle(tid) && TangentYOverlay->IsSetTriangle(tid))
					{
						FIndex3i TriV = ReadMesh.GetTriangle(tid);
						FVector3f A, B, C;
						TangentXOverlay->GetTriElements(tid, A, B, C);
						TangentsX[TriV.A] += (FVector)A;
						TangentsX[TriV.B] += (FVector)B;
						TangentsX[TriV.C] += (FVector)C;
						TangentYOverlay->GetTriElements(tid, A, B, C);
						TangentsY[TriV.A] += (FVector)A;
						TangentsY[TriV.B] += (FVector)B;
						TangentsY[TriV.C] += (FVector)C;
					}
				}

				for (int32 k = 0; k < TangentsX.Num(); ++k)
				{
					if (TangentsX[k].SquaredLength() > 0)
					{
						Normalize(TangentsX[k]);
					}
					if (TangentsY[k].SquaredLength() > 0)
					{
						Normalize(TangentsY[k]);
					}
				}
			}
			else
			{
				TangentsX.Init(FVector::UnitZ(), ReadMesh.MaxVertexID());
				TangentsY.Init(FVector::UnitZ(), ReadMesh.MaxVertexID());
				for (int32 tid : ReadMesh.TriangleIndicesItr())
				{
					if (TangentXOverlay->IsSetTriangle(tid) && TangentYOverlay->IsSetTriangle(tid))
					{
						FIndex3i TriV = ReadMesh.GetTriangle(tid);
						FVector3f A, B, C;
						TangentXOverlay->GetTriElements(tid, A, B, C);
						TangentsX[TriV.A] = (FVector)A;
						TangentsX[TriV.B] = (FVector)B;
						TangentsX[TriV.C] = (FVector)C;
						TangentYOverlay->GetTriElements(tid, A, B, C);
						TangentsY[TriV.A] = (FVector)A;
						TangentsY[TriV.B] = (FVector)B;
						TangentsY[TriV.C] = (FVector)C;
					}
				}
			}

			bIsValidTangentSet = true;
		}
	});

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshNormalsFunctions::UpdateVertexNormal(
	UDynamicMesh* TargetMesh,
	int VertexID,
	bool bUpdateNormal,
	FVector NewNormal,
	bool bUpdateTangents,
	FVector NewTangentX,
	FVector NewTangentY,
	bool& bIsValidVertex,
	bool bMergeSplitNormals,
	bool bDeferChangeNotifications)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: TargetMesh is Null");
		return TargetMesh;
	}

	TArray<FIndex3i> TriVtxElements;
	TArray<int32> UniqueElementIDs;
	TriVtxElements.Reserve(16);
	UniqueElementIDs.Reserve(16);

	// Updates VertexID's associated elements in Overlay with NewValue. 
	// Returns false if there were no elements to update (ie all "unset" triangles)
	auto UpdateOverlay = [VertexID, &TriVtxElements, &UniqueElementIDs, bMergeSplitNormals](FDynamicMesh3& EditMesh, FDynamicMeshNormalOverlay* Overlay, FVector NewValue)
	{
		TriVtxElements.Reset(); UniqueElementIDs.Reset();
		EditMesh.EnumerateVertexTriangles(VertexID, [&](int32 TriangleID)
		{
			if (Overlay->IsSetTriangle(TriangleID))
			{
				FIndex3i Tri = EditMesh.GetTriangle(TriangleID);
				int32 Index = Tri.IndexOf(VertexID);
				FIndex3i OverlayTri = Overlay->GetTriangle(TriangleID);
				TriVtxElements.Add(FIndex3i(TriangleID, Index, OverlayTri[Index]));
				UniqueElementIDs.AddUnique(OverlayTri[Index]);
			}
		});
		if (TriVtxElements.IsEmpty())
		{
			return false;
		}
		if (UniqueElementIDs.Num() == 1 || bMergeSplitNormals == false)
		{
			// just update existing elements to new normal
			for (int32 ElementID : UniqueElementIDs)
			{
				Overlay->SetElement(ElementID, (FVector3f)NewValue);
			}
		}
		else   // bMergeSplitNormals == true && UniqueElementIDs.Num() > 1
		{
			// Note the UniqueElementIDs array must not be empty, since TriVtxElements was not empty
			Overlay->SetElement(UniqueElementIDs[0], (FVector3f)NewValue);
			for (FIndex3i TriInfo : TriVtxElements)
			{
				FIndex3i OverlayTri = Overlay->GetTriangle(TriInfo.A);
				OverlayTri[TriInfo.B] = UniqueElementIDs[0];
				Overlay->SetTriangle(TriInfo.A, OverlayTri);
			}
		}
		return true;
	};

	bIsValidVertex = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false 
			|| (EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals() == nullptr) 
			|| (EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals()->ElementCount() == 0) )
		{
			UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: TargetMesh does not have valid Normals attributes. Try computing Vertex Normals before using UpdateVertexNormal.");
			return;
		}
		if (bUpdateTangents && EditMesh.Attributes()->HasTangentSpace() == false)
		{
			UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: TargetMesh does not have valid Tangents attributes. Try computing Tangents before using UpdateVertexNormal.");
			return;
		}
		if (EditMesh.IsVertex(VertexID) == false)
		{
			UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: VertexID %d is not a valid vertex in TargetMesh", VertexID);
			return;
		}
		bIsValidVertex = true;

		if (bUpdateNormal)
		{
			FDynamicMeshNormalOverlay* Normals = EditMesh.Attributes()->PrimaryNormals();
			if (UpdateOverlay(EditMesh, Normals, NewNormal) == false)
			{
				UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: VertexID %d has no existing normals in Normal Overlay. Try computing Vertex Normals before using UpdateVertexNormal.", VertexID);
				bIsValidVertex = false;
			}
		}
		if (bUpdateTangents)
		{
			FDynamicMeshNormalOverlay* TangentX = EditMesh.Attributes()->PrimaryTangents();
			FDynamicMeshNormalOverlay* TangentY = EditMesh.Attributes()->PrimaryBiTangents();
			bool bTangentXOK = UpdateOverlay(EditMesh, TangentX, NewTangentX);
			bool bTangentYOK = UpdateOverlay(EditMesh, TangentY, NewTangentY);
			if (bTangentXOK == false || bTangentYOK == false)
			{
				UE_LOGF(LogGeometry, Warning, "UpdateVertexNormal: VertexID %d has no existing tangents in Tangent Overlay. Try computing Tangents before using UpdateVertexNormal.", VertexID);
				bIsValidVertex = false;
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);

	return TargetMesh;
}



#undef LOCTEXT_NAMESPACE

