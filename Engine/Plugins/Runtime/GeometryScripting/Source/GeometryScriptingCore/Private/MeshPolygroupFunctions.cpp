// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshPolygroupFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Polygroups/PolygroupsGenerator.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPolygroupFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshPolygroupFunctions"




template<typename ReturnType> 
ReturnType SimpleMeshPolygroupQuery(UDynamicMesh* Mesh, FGeometryScriptGroupLayer ScriptGroupLayer, bool& bHasGroups, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh, const FPolygroupSet& Polygroups)> QueryFunc)
{
	bHasGroups = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			FPolygroupLayer GroupLayer{ ScriptGroupLayer.bDefaultLayer, ScriptGroupLayer.ExtendedLayerIndex };
			if (GroupLayer.CheckExists(&ReadMesh))
			{
				FPolygroupSet Groups(&ReadMesh, GroupLayer);
				bHasGroups = true;
				RetVal = QueryFunc(ReadMesh, Groups);
			}
		});
	}
	return RetVal;
}

// Version of SimpleMeshPolygroupQuery that avoids initial full mesh pass to initialize FPolygroupSet, instead passing a simple lambda to access triangle polygroups
template<typename ReturnType>
ReturnType SimpleMeshPolygroupQuery(const UDynamicMesh* Mesh, FGeometryScriptGroupLayer ScriptGroupLayer, bool& bHasGroups, ReturnType DefaultValue,
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh, TFunctionRef<int32(int32)> GetPolygroup)> QueryFunc)
{
	bHasGroups = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->ProcessMesh([ScriptGroupLayer, &bHasGroups, &RetVal, QueryFunc](const FDynamicMesh3& ReadMesh)
		{
			if (ScriptGroupLayer.bDefaultLayer)
			{
				if (ReadMesh.HasTriangleGroups())
				{
					bHasGroups = true;
					RetVal = QueryFunc(ReadMesh, [&ReadMesh](int32 TID) -> int32 
					{
						return ReadMesh.GetTriangleGroup(TID); 
					});
				}
			}
			else
			{
				if (ReadMesh.HasAttributes())
				{
					if (const FDynamicMeshPolygroupAttribute* GroupLayer = ReadMesh.Attributes()->GetPolygroupLayer(ScriptGroupLayer.ExtendedLayerIndex))
					{
						bHasGroups = true;
						RetVal = QueryFunc(ReadMesh, [GroupLayer](int32 TID) -> int32 
						{
							return GroupLayer->GetValue(TID); 
						});
					}
				}
			}
		});
	}
	return RetVal;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::EnablePolygroups( 
	UDynamicMesh* TargetMesh, 
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("EnablePolygroups_InvalidInput", "EnablePolygroups: TargetMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasTriangleGroups() == false)
		{
			EditMesh.EnableTriangleGroups(0);
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::SetNumExtendedPolygroupLayers( 
	UDynamicMesh* TargetMesh, 
	int NumLayers,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetNumExtendedPolygroupLayers_InvalidInput", "SetNumExtendedPolygroupLayers: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		if (EditMesh.Attributes()->NumPolygroupLayers() != NumLayers)
		{
			EditMesh.Attributes()->SetNumPolygroupLayers(NumLayers);
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::AddNamedPolygroupLayer(
	UDynamicMesh* TargetMesh,
	FName LayerName,
	FGeometryScriptGroupLayer& GroupLayer,
	bool& bAlreadyExisted,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AddNamedPolygroupLayer_InvalidInput", "AddNamedPolygroupLayer: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&GroupLayer, &bAlreadyExisted, LayerName](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		const int32 InitialNumLayers = EditMesh.Attributes()->NumPolygroupLayers();
		for (int32 LayerIdx = 0; LayerIdx < InitialNumLayers; ++LayerIdx)
		{
			if (EditMesh.Attributes()->GetPolygroupLayer(LayerIdx)->GetName() == LayerName)
			{
				GroupLayer.bDefaultLayer = false;
				GroupLayer.ExtendedLayerIndex = LayerIdx;
				return;
			}
		}
		// layer not found -- add it
		EditMesh.Attributes()->SetNumPolygroupLayers(InitialNumLayers + 1);
		EditMesh.Attributes()->GetPolygroupLayer(InitialNumLayers)->SetName(LayerName);
		GroupLayer.bDefaultLayer = false;
		GroupLayer.ExtendedLayerIndex = InitialNumLayers;
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

const UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::FindExtendedPolygroupLayerByName(
	const UDynamicMesh* TargetMesh,
	FName LayerName,
	FGeometryScriptGroupLayer& GroupLayer,
	EGeometryScriptSearchOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptSearchOutcomePins::NotFound;
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindExtendedPolygroupLayerByName_InvalidInput", "FindExtendedPolygroupLayerByName: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&Outcome, &GroupLayer, LayerName](const FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			return;
		}
		for (int32 LayerIndex = 0; LayerIndex < EditMesh.Attributes()->NumPolygroupLayers(); ++LayerIndex)
		{
			if (EditMesh.Attributes()->GetPolygroupLayer(LayerIndex)->GetName() == LayerName)
			{
				Outcome = EGeometryScriptSearchOutcomePins::Found;
				GroupLayer.bDefaultLayer = false;
				GroupLayer.ExtendedLayerIndex = LayerIndex;
				return;
			}
		}
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::ClearPolygroups( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	int ClearValue,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ClearPolygroups_InvalidInput", "ClearPolygroups: TargetMesh is Null"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TUniquePtr<FPolygroupSet> OutputGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ClearPolygroups_MissingGroups", "ClearPolygroups: Target Polygroup Layer does not exist"));
			return;
		}
		OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);
		for (int32 tid : EditMesh.TriangleIndicesItr())
		{
			OutputGroups->SetGroup(tid, ClearValue, EditMesh);
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::CopyPolygroupsLayer( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer FromGroupLayer,
	FGeometryScriptGroupLayer ToGroupLayer,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyPolygroupsLayer_InvalidInput", "CopyPolygroupsLayer: TargetMesh is Null"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FPolygroupLayer InputGroupLayer{ FromGroupLayer.bDefaultLayer, FromGroupLayer.ExtendedLayerIndex };
		FPolygroupLayer OutputGroupLayer{ ToGroupLayer.bDefaultLayer, ToGroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer == OutputGroupLayer)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyPolygroupsLayer_SameGroups", "CopyPolygroupsLayer: tried to copy Polygroup Layer to itself"));
			return;
		}
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyPolygroupsLayer_MissingFromGroups", "CopyPolygroupsLayer: From Polygroup Layer does not exist"));
			return;
		}
		if (OutputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyPolygroupsLayer_MissingToGroups", "CopyPolygroupsLayer: To Polygroup Layer does not exist"));
			return;
		}


		TUniquePtr<FPolygroupSet> InputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);
		TUniquePtr<FPolygroupSet> OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, OutputGroupLayer);

		for (int32 tid : EditMesh.TriangleIndicesItr())
		{
			OutputGroups->SetGroup(tid, InputGroups->GetGroup(tid), EditMesh);
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::ConvertUVIslandsToPolygroups( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	int UVLayer,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertUVIslandsToPolygroups_InvalidInput", "ConvertUVIslandsToPolygroups: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || !(UVLayer < EditMesh.Attributes()->NumUVLayers()) )
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertUVIslandsToPolygroups_InvalidUVLayers", "ConvertUVIslandsToPolygroups: Requested UV layer does not exist"));
			return;
		}

		TUniquePtr<FPolygroupSet> OutputGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertUVIslandsToPolygroups_MissingGroups", "ConvertUVIslandsToPolygroups: Target Polygroup Layer does not exist"));
			return;
		}
		OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);

		FPolygroupsGenerator Generator(&EditMesh);
		Generator.bApplyPostProcessing = false;
		Generator.bCopyToMesh = false;
		Generator.FindPolygroupsFromUVIslands(UVLayer);
		Generator.CopyPolygroupsToPolygroupSet(*OutputGroups, EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::ConvertComponentsToPolygroups( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertComponentsToPolygroups_InvalidInput", "ConvertComponentsToPolygroups: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TUniquePtr<FPolygroupSet> OutputGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertUVIslandsToPolygroups_MissingGroups", "ConvertUVIslandsToPolygroups: Target Polygroup Layer does not exist"));
			return;
		}
		OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);

		FPolygroupsGenerator Generator(&EditMesh);
		Generator.bApplyPostProcessing = false;
		Generator.bCopyToMesh = false;
		Generator.FindPolygroupsFromConnectedTris();
		Generator.CopyPolygroupsToPolygroupSet(*OutputGroups, EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::ComputePolygroupsFromAngleThreshold( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	float CreaseAngle,
	int MinGroupSize,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputePolygroupsFromAngleThreshold_InvalidInput", "ComputePolygroupsFromAngleThreshold: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TUniquePtr<FPolygroupSet> OutputGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputePolygroupsFromAngleThreshold_MissingGroups", "ComputePolygroupsFromAngleThreshold: Target Polygroup Layer does not exist"));
			return;
		}
		OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);

		FPolygroupsGenerator Generator(&EditMesh);
		Generator.bApplyPostProcessing = (MinGroupSize > 1);
		Generator.MinGroupSize = MinGroupSize;
		Generator.bCopyToMesh = false;
		double DotTolerance = 1.0 - FMathd::Cos((double)CreaseAngle * FMathd::DegToRad);
		Generator.FindPolygroupsFromFaceNormals(DotTolerance);
		Generator.CopyPolygroupsToPolygroupSet(*OutputGroups, EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::ComputePolygroupsFromPolygonDetection( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer,
	bool bRespectUVSeams,
	bool bRespectHardNormals,
	double QuadAdjacencyWeight,
	double QuadMetricClamp,
	int MaxSearchRounds,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputePolygroupsFromPolygonDetection_InvalidInput", "ComputePolygroupsFromPolygonDetection: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TUniquePtr<FPolygroupSet> OutputGroups;
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputePolygroupsFromPolygonDetection_MissingGroups", "ComputePolygroupsFromPolygonDetection: Target Polygroup Layer does not exist"));
			return;
		}
		OutputGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);

		FPolygroupsGenerator Generator(&EditMesh);
		Generator.bApplyPostProcessing = false;
		Generator.bCopyToMesh = false;
		Generator.FindSourceMeshPolygonPolygroups(
			bRespectUVSeams, bRespectHardNormals,
			QuadAdjacencyWeight, QuadMetricClamp, MaxSearchRounds);
		Generator.CopyPolygroupsToPolygroupSet(*OutputGroups, EditMesh);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



int32 UGeometryScriptLibrary_MeshPolygroupFunctions::GetTrianglePolygroupID( 
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer, 
	int TriangleID, 
	bool& bIsValidTriangle)
{
	bIsValidTriangle = false;
	bool bIsValidPolygroupLayer = false;
	return SimpleMeshPolygroupQuery<int32>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0, 
	[&](const FDynamicMesh3& Mesh, const FPolygroupSet& PolyGroups) {
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		return (bIsValidTriangle) ? PolyGroups.GetGroup(TriangleID) : 0;
	});
}



UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::DeleteTrianglesInPolygroup(
	UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	int PolygroupID,
	int& NumDeleted,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	NumDeleted = 0;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh) == false)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DeleteTrianglesInPolygroup_MissingGroups", "DeleteTrianglesInPolygroup: Specified Polygroup Layer does not exist"));
				return;
			}

			FPolygroupSet Groups(&EditMesh, InputGroupLayer);
			TArray<int32> TriangleList;
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				if (Groups.GetGroup(TriangleID) == PolygroupID)
				{
					TriangleList.Add(TriangleID);
				}
			}

			for (int32 TriangleID : TriangleList)
			{
				EMeshResult Result = EditMesh.RemoveTriangle(TriangleID);
				if (Result == EMeshResult::Ok)
				{
					NumDeleted++;
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetAllTrianglePolygroupIDs(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer, 
	FGeometryScriptIndexList& PolygroupIDsOut)
{
	PolygroupIDsOut.Reset(EGeometryScriptIndexType::PolygroupID);
	TArray<int32>& PolygroupIDs = *PolygroupIDsOut.List;

	bool bIsValidPolygroupLayer = false;
	SimpleMeshPolygroupQuery<int32>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0, 
		[&](const FDynamicMesh3& Mesh, const FPolygroupSet& PolyGroups) {
		int32 MaxTriangleID = Mesh.MaxTriangleID();
		for (int32 TriangleID = 0; TriangleID < MaxTriangleID; ++TriangleID)
		{
			int32 GroupID = Mesh.IsTriangle(TriangleID) ? PolyGroups.GetGroup(TriangleID) : -1;
			PolygroupIDs.Add(GroupID);
		}
		return 0;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetPolygroupIDsInMesh(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptGroupLayer GroupLayer, 
	FGeometryScriptIndexList& PolygroupIDsOut)
{
	PolygroupIDsOut.Reset(EGeometryScriptIndexType::PolygroupID);
	TArray<int32>& PolygroupIDs = *PolygroupIDsOut.List;

	TSet<int32> UniqueGroupIDs;
	bool bIsValidPolygroupLayer = false;
	SimpleMeshPolygroupQuery<int32>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0, 
		[&](const FDynamicMesh3& Mesh, const FPolygroupSet& PolyGroups) {
		int32 MaxTriangleID = Mesh.MaxTriangleID();
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			int32 GroupID = PolyGroups.GetGroup(tid);
			if (UniqueGroupIDs.Contains(GroupID) == false)
			{
				UniqueGroupIDs.Add(GroupID);
				PolygroupIDs.Add(GroupID);
			}
		}
		return 0;
	});
	return TargetMesh;
}


const UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetPolyGroupBoundingBox(
	const UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	int32 GroupID,
	FBox& Bounds)
{
	Bounds.Init();
	bool bIsValidPolygroupLayer = false;
	SimpleMeshPolygroupQuery<bool>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0,
		[GroupID, &Bounds](const FDynamicMesh3& Mesh, TFunctionRef<int32(int32)> GetGroup) -> bool
		{
			for (int32 TID : Mesh.TriangleIndicesItr())
			{
				if (GroupID == GetGroup(TID))
				{
					FVector3d A, B, C;
					Mesh.GetTriVertices(TID, A, B, C);
					Bounds += A;
					Bounds += B;
					Bounds += C;
				}
			}
			return (bool)Bounds.IsValid;
		});
	return TargetMesh;
}

const UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetPolyGroupUVBoundingBox(
	const UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	int32 GroupID,
	int32 UVChannel,
	FBox2D& Bounds)
{
	Bounds.Init();
	bool bIsValidPolygroupLayer = false;
	SimpleMeshPolygroupQuery<bool>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0,
		[GroupID, UVChannel, &Bounds](const FDynamicMesh3& Mesh, TFunctionRef<int32(int32)> GetGroup) -> bool
		{
			if (!Mesh.HasAttributes())
			{
				return false;
			}
			const FDynamicMeshUVOverlay* UVLayer = Mesh.Attributes()->GetUVLayer(UVChannel);
			if (!UVLayer)
			{
				return false;
			}

			for (int32 TID : Mesh.TriangleIndicesItr())
			{
				if (GroupID == GetGroup(TID))
				{
					FIndex3i Els;
					if (UVLayer->GetTriangleIfValid(TID, Els))
					{
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							Bounds += (FVector2D)UVLayer->GetElement(Els[SubIdx]);
						}
					}
				}
			}
			return Bounds.bIsValid;
		});
	return TargetMesh;
}

const UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetPolyGroupUVCentroid(
	const UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	int32 GroupID,
	int32 UVChannel,
	FVector2D& Centroid,
	bool& bIsValid)
{
	Centroid = FVector2D::Zero();
	bool bIsValidPolygroupLayer = false;
	bIsValid = SimpleMeshPolygroupQuery<bool>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0,
		[GroupID, UVChannel, &Centroid](const FDynamicMesh3& Mesh, TFunctionRef<int32(int32)> GetGroup) -> bool
		{
			if (!Mesh.HasAttributes())
			{
				return false;
			}
			const FDynamicMeshUVOverlay* UVLayer = Mesh.Attributes()->GetUVLayer(UVChannel);
			if (!UVLayer)
			{
				return false;
			}

			double WtSum = 0.0;
			FVector2D Sum = FVector2D::Zero();
			for (int32 TID : Mesh.TriangleIndicesItr())
			{
				if (GroupID == GetGroup(TID))
				{
					FIndex3i Els;
					if (UVLayer->GetTriangleIfValid(TID, Els))
					{
						FVector2f A = UVLayer->GetElement(Els.A), B = UVLayer->GetElement(Els.B), C = UVLayer->GetElement(Els.C);
						FVector2D TriCenter = FVector2D(A + B + C) * (1.0 / 3.0);
						double Wt = (double)VectorUtil::Area(A, B, C);
						Sum += TriCenter * Wt;
						WtSum += Wt;
					}
				}
			}
			if (WtSum > 0)
			{
				Centroid = Sum / WtSum;
				return true;
			}
			else
			{
				return false;
			}
		});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::GetTrianglesInPolygroup(
	UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	int PolygroupID,
	FGeometryScriptIndexList& TriangleIDsOut)
{
	TriangleIDsOut.Reset(EGeometryScriptIndexType::Triangle);
	TArray<int32>& TriangleIDs = *TriangleIDsOut.List;

	bool bIsValidPolygroupLayer = false;
	SimpleMeshPolygroupQuery<int32>(TargetMesh, GroupLayer, bIsValidPolygroupLayer, 0, 
		[&](const FDynamicMesh3& Mesh, const FPolygroupSet& PolyGroups) {
		for (int32 TriangleID : Mesh.TriangleIndicesItr() )
		{
			if (PolyGroups.GetGroup(TriangleID) == PolygroupID)
			{
				TriangleIDs.Add(TriangleID);
			}
		}
		return 0;
	});
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPolygroupFunctions::SetPolygroupForMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	FGeometryScriptMeshSelection Selection,
	int& SetPolygroupIDOut,
	int SetPolygroupID,
	bool bGenerateNewPolygroup,
	bool bDeferChangeNotifications)
{
	SetPolygroupIDOut = SetPolygroupID;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh) == false)
			{
				UE_LOGF(LogGeometry, Warning, "SetPolygroupForMeshSelection: Specified Polygroup Layer does not exist");
				return;
			}

			FPolygroupSet Groups(&EditMesh, InputGroupLayer);
			if (bGenerateNewPolygroup)
			{
				SetPolygroupID = Groups.AllocateNewGroupID();
				SetPolygroupIDOut = SetPolygroupID;
			}
			Selection.ProcessByTriangleID(EditMesh,
				[&](int32 TriangleID) { Groups.SetGroup(TriangleID, SetPolygroupID, EditMesh); } );

		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;
}




#undef LOCTEXT_NAMESPACE
