// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshPoolFunctions.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPoolFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshPoolFunctions"

namespace UE::Private::MeshPoolHelper
{
	UDynamicMeshPool* GlobalMeshPoolHelper(bool bDestroy = false)
	{
		static UDynamicMeshPool* MeshPool = nullptr;
		if (!MeshPool && !bDestroy)
		{
			MeshPool = NewObject<UDynamicMeshPool>();
			MeshPool->AddToRoot();
		}
		else if (MeshPool && bDestroy)
		{
			MeshPool->FreeAllMeshes();
			MeshPool->RemoveFromRoot();
			MeshPool = nullptr;
		}
		return MeshPool;
	}
}



UDynamicMeshPool* UGeometryScriptLibrary_MeshPoolFunctions::GetGlobalMeshPool()
{
	return UE::Private::MeshPoolHelper::GlobalMeshPoolHelper(false);
}


void UGeometryScriptLibrary_MeshPoolFunctions::DiscardGlobalMeshPool()
{
	UE::Private::MeshPoolHelper::GlobalMeshPoolHelper(true);
}



#undef LOCTEXT_NAMESPACE

