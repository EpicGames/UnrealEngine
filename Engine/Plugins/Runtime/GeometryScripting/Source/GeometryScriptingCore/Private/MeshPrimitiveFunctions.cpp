// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshPrimitiveFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"

#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/SphereGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Generators/DiscMeshGenerator.h"
#include "Generators/SweepGenerator.h"
#include "Generators/FlatTriangulationMeshGenerator.h"
#include "Generators/RevolveGenerator.h"
#include "Generators/StairGenerator.h"
#include "ConstrainedDelaunay2.h"
#include "Arrangement2d.h"
#include "CompGeom/Delaunay2.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "CompGeom/PolygonTriangulation.h"

#include "Physics/ComponentCollisionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPrimitiveFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshPrimitiveFunctions"


static void ApplyPrimitiveOptionsToMesh(
	FDynamicMesh3& Mesh, const FTransform& Transform, 
	FGeometryScriptPrimitiveOptions PrimitiveOptions, 
	FVector3d PreTranslate = FVector3d::Zero(),
	TOptional<FQuaterniond> PreRotate = TOptional<FQuaterniond>())
{
	bool bHasTranslate = PreTranslate.SquaredLength() > 0;
	if (PreRotate.IsSet())
	{
		FFrame3d Frame(PreTranslate, *PreRotate);
		MeshTransforms::FrameCoordsToWorld(Mesh, Frame);
	}
	else if (bHasTranslate)
	{
		MeshTransforms::Translate(Mesh, PreTranslate);
	}

	MeshTransforms::ApplyTransform(Mesh, (FTransformSRT3d)Transform, true);
	if (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::SingleGroup)
	{
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			Mesh.SetTriangleGroup(tid, 0);
		}
	}
	if (PrimitiveOptions.bFlipOrientation)
	{
		Mesh.ReverseOrientation(true);
		if (Mesh.HasAttributes())
		{
			FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
			for (int elemid : Normals->ElementIndicesItr())
			{
				Normals->SetElement(elemid, -Normals->GetElement(elemid));
			}
		}
	}
	// if we have a valid non-zero MaterialID, set it on all triangles (0 is default)
	if (PrimitiveOptions.MaterialID > 0)
	{
		// Make sure material IDs are enabled
		Mesh.EnableAttributes();
		Mesh.Attributes()->EnableMaterialID();
		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();
		for (int32 TID : Mesh.TriangleIndicesItr())
		{
			MaterialIDs->SetValue(TID, PrimitiveOptions.MaterialID);
		}
	}
}

static void AppendPrimitive(
	UDynamicMesh* TargetMesh, 
	FMeshShapeGenerator* Generator, 
	FTransform Transform, 
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FVector3d PreTranslate = FVector3d::Zero(),
	TOptional<FQuaterniond> PreRotate = TOptional<FQuaterniond>())
{
	if (TargetMesh->IsEmpty())
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh.Copy(Generator);
			ApplyPrimitiveOptionsToMesh(EditMesh, Transform, PrimitiveOptions, PreTranslate, PreRotate);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	else
	{
		FDynamicMesh3 TempMesh(Generator);
		ApplyPrimitiveOptionsToMesh(TempMesh, Transform, PrimitiveOptions, PreTranslate, PreRotate);
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FMeshIndexMappings TmpMappings;
			FDynamicMeshEditor Editor(&EditMesh);
			Editor.AppendMesh(&TempMesh, TmpMappings);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
}

static void AppendPrimitiveWithVertexIndexMap(
	UDynamicMesh* TargetMesh,
	TArray<int32>& OutVertexIndexMap,
	FMeshShapeGenerator* Generator,
	FTransform Transform,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FVector3d PreTranslate = FVector3d::Zero())
{
	if (TargetMesh->IsEmpty())
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh.Copy(Generator);
			ApplyPrimitiveOptionsToMesh(EditMesh, Transform, PrimitiveOptions, PreTranslate);
			// For a clean/empty mesh, the vertex map is just an identity map
			OutVertexIndexMap.SetNum(EditMesh.VertexCount());
			for (int32 Idx = 0; Idx < OutVertexIndexMap.Num(); ++Idx)
			{
				OutVertexIndexMap[Idx] = Idx;
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	else
	{
		FDynamicMesh3 TempMesh(Generator);
		ApplyPrimitiveOptionsToMesh(TempMesh, Transform, PrimitiveOptions, PreTranslate);
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FMeshIndexMappings TmpMappings;
			FDynamicMeshEditor Editor(&EditMesh);
			Editor.AppendMesh(&TempMesh, TmpMappings);
			OutVertexIndexMap.Init(INDEX_NONE, Generator->Vertices.Num());
			for (TPair<int32, int32> IndexToID : TmpMappings.GetVertexMap().GetForwardMap())
			{
				OutVertexIndexMap[IndexToID.Key] = IndexToID.Value;
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
}



static void AppendPrimitiveMesh(
	UDynamicMesh* TargetMesh, 
	FDynamicMesh3& AppendMesh,
	FTransform Transform, 
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FVector3d PreTranslate = FVector3d::Zero())
{
	auto ApplyOptionsToMesh = [&Transform, &PrimitiveOptions, PreTranslate](FDynamicMesh3& Mesh)
	{
		if (PreTranslate.SquaredLength() > 0)
		{
			MeshTransforms::Translate(Mesh, PreTranslate);
		}

		MeshTransforms::ApplyTransform(Mesh, (FTransformSRT3d)Transform, true);
		if (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::SingleGroup)
		{
			for (int32 tid : Mesh.TriangleIndicesItr())
			{
				Mesh.SetTriangleGroup(tid, 0);
			}
		}
		if (PrimitiveOptions.bFlipOrientation)
		{
			Mesh.ReverseOrientation(true);
			if (Mesh.HasAttributes())
			{
				FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
				for (int elemid : Normals->ElementIndicesItr())
				{
					Normals->SetElement(elemid, -Normals->GetElement(elemid));
				}
			}
		}
	};

	if (TargetMesh->IsEmpty())
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh.Copy(AppendMesh);
			ApplyOptionsToMesh(EditMesh);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
	else
	{
		ApplyOptionsToMesh(AppendMesh);
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			FMeshIndexMappings TmpMappings;
			FDynamicMeshEditor Editor(&EditMesh);
			Editor.AppendMesh(&AppendMesh, TmpMappings);
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}
}

namespace UE::Private::AppendMeshHelper
{
	static UDynamicMesh* AppendBox(UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		const FTransform& Transform,
		const UE::Geometry::FOrientedBox3d& OrientedBox,
		int32 StepsX,
		int32 StepsY,
		int32 StepsZ,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendBox", "AppendBox: TargetMesh is Null"));
			return TargetMesh;
		}

		// todo: if Steps X/Y/Z are zero, can use trivial box generator

		FGridBoxMeshGenerator GridBoxGenerator;
		GridBoxGenerator.Box = OrientedBox;
		GridBoxGenerator.EdgeVertices = FIndex3i(FMath::Max(0, StepsX), FMath::Max(0, StepsY), FMath::Max(0, StepsZ));
		GridBoxGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		GridBoxGenerator.Generate();

		AppendPrimitive(TargetMesh, &GridBoxGenerator, Transform, PrimitiveOptions, FVector3d::Zero());

		if (SimpleCollision)
		{
			FVector3d Dims = Transform.GetScale3D() * 2 * OrientedBox.Extents;
			FKBoxElem BoxElem(Dims.X, Dims.Y, Dims.Z);
			BoxElem.Center = Transform.TransformPosition(OrientedBox.Center());
			BoxElem.Rotation = (Transform.GetRotation() * FQuat(OrientedBox.Frame.Rotation)).Rotator();
			SimpleCollision->AggGeom.BoxElems.Add(BoxElem);
		}

		return TargetMesh;
	}

	static UDynamicMesh* AppendBox(UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX,
		float DimensionY,
		float DimensionZ,
		int32 StepsX,
		int32 StepsY,
		int32 StepsZ,
		EGeometryScriptPrimitiveOriginMode Origin,
		UGeometryScriptDebug* Debug)
	{
		UE::Geometry::FOrientedBox3d OrientedBox(
			FVector3d(0, 0, Origin == EGeometryScriptPrimitiveOriginMode::Base ? DimensionZ * .5f : 0.f),
			FVector3d(DimensionX * .5f, DimensionY * .5f, DimensionZ * .5f)); // Extents are half-dimensions
		return UE::Private::AppendMeshHelper::AppendBox(TargetMesh, SimpleCollision, PrimitiveOptions, Transform, OrientedBox, StepsX, StepsY, StepsZ, Debug);
	}

	static UDynamicMesh* AppendBoundingBox(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FBox Box,
		int32 StepsX,
		int32 StepsY,
		int32 StepsZ,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendBoundingBox", "AppendBoundingBox: TargetMesh is Null"));
			return TargetMesh;
		}

		if (!Box.IsValid)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendBoundingBox_InvalidBox", "AppendBoundingBox: Bounding Box is empty, and cannot be appended."));
			return TargetMesh;
		}

		// todo: if Steps X/Y/Z are zero, can use trivial box generator

		FGridBoxMeshGenerator GridBoxGenerator;
		GridBoxGenerator.Box = UE::Geometry::FOrientedBox3d( UE::Geometry::FAxisAlignedBox3d(Box) );
		GridBoxGenerator.EdgeVertices = FIndex3i(FMath::Max(0, StepsX), FMath::Max(0, StepsY), FMath::Max(0, StepsZ));
		GridBoxGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		GridBoxGenerator.Generate();

		AppendPrimitive(TargetMesh, &GridBoxGenerator, Transform, PrimitiveOptions, FVector3d::Zero());

		if (SimpleCollision)
		{
			FVector3d Dims = Transform.GetScale3D() * Box.GetSize();
			FKBoxElem BoxElem(Dims.X, Dims.Y, Dims.Z);
			BoxElem.Center = Transform.TransformPosition(Box.GetCenter());
			BoxElem.Rotation = Transform.Rotator();
			SimpleCollision->AggGeom.BoxElems.Add(BoxElem);
		}

		return TargetMesh;
	}

	void AppendSphereSimpleCollision(FKAggregateGeom& AggGeom, const FTransform& Transform, float Radius, FVector Center)
	{
		FKSphereElem Sphere;
		Sphere.Center = Transform.TransformPosition(Center);
		Sphere.Radius = Radius * Transform.GetMinimumAxisScale();
		AggGeom.SphereElems.Add(Sphere);
	}

	UDynamicMesh* AppendSphereLatLong(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius,
		int32 StepsPhi,
		int32 StepsTheta,
		EGeometryScriptPrimitiveOriginMode Origin,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendSphereLatLong", "AppendSphereLatLong: TargetMesh is Null"));
			return TargetMesh;
		}

		FSphereGenerator SphereGenerator;
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		SphereGenerator.NumPhi = FMath::Max(3, StepsPhi);
		SphereGenerator.NumTheta = FMath::Max(3, StepsTheta);
		SphereGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		SphereGenerator.Generate();

		FVector3d OriginShift = (Origin == EGeometryScriptPrimitiveOriginMode::Base) ? FVector3d(0, 0, Radius) : FVector3d::Zero();
		AppendPrimitive(TargetMesh, &SphereGenerator, Transform, PrimitiveOptions, OriginShift);

		if (SimpleCollision)
		{
			AppendSphereSimpleCollision(SimpleCollision->AggGeom, Transform, Radius, OriginShift);
		}

		return TargetMesh;
	}

	UDynamicMesh* AppendSphereBox(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius,
		int32 StepsX,
		int32 StepsY,
		int32 StepsZ,
		EGeometryScriptPrimitiveOriginMode Origin,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendSphereBox", "AppendSphereBox: TargetMesh is Null"));
			return TargetMesh;
		}

		FBoxSphereGenerator SphereGenerator;
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		SphereGenerator.EdgeVertices = FIndex3i(FMath::Max(0, StepsX), FMath::Max(0, StepsY), FMath::Max(0, StepsZ));
		SphereGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		SphereGenerator.Generate();

		FVector3d OriginShift = (Origin == EGeometryScriptPrimitiveOriginMode::Base) ? FVector3d(0, 0, Radius) : FVector3d::Zero();
		AppendPrimitive(TargetMesh, &SphereGenerator, Transform, PrimitiveOptions, OriginShift);

		if (SimpleCollision)
		{
			AppendSphereSimpleCollision(SimpleCollision->AggGeom, Transform, Radius, OriginShift);
		}

		return TargetMesh;
	}

	UDynamicMesh* AppendCapsule(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision* SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius,
		float LineLength,
		int32 HemisphereSteps,
		int32 CircleSteps,
		int32 SegmentSteps,
		EGeometryScriptPrimitiveOriginMode Origin,
		UGeometryScriptDebug* Debug)
	{
		if (TargetMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendCapsule", "AppendCapsule: TargetMesh is Null"));
			return TargetMesh;
		}

		FCapsuleGenerator CapsuleGenerator;
		CapsuleGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
		CapsuleGenerator.SegmentLength = FMath::Max(FMathf::ZeroTolerance, LineLength);
		CapsuleGenerator.NumHemisphereArcSteps = FMath::Max(2, HemisphereSteps);
		CapsuleGenerator.NumCircleSteps = FMath::Max(3, CircleSteps);
		CapsuleGenerator.NumSegmentSteps = FMath::Max(0, SegmentSteps);
		CapsuleGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		CapsuleGenerator.Generate();

		FVector3d OriginShift = FVector3d::Zero();
		OriginShift.Z = (Origin == EGeometryScriptPrimitiveOriginMode::Center) ? -(LineLength / 2) : Radius;
		AppendPrimitive(TargetMesh, &CapsuleGenerator, Transform, PrimitiveOptions, OriginShift);

		if (SimpleCollision)
		{
			FKSphylElem CapsuleElem;
			FVector3d Center(0, 0, Origin == EGeometryScriptPrimitiveOriginMode::Center ? 0.0 : CapsuleGenerator.SegmentLength * .5 + CapsuleGenerator.Radius);
			CapsuleElem.Center = Transform.TransformPosition(Center);
			CapsuleElem.Rotation = Transform.Rotator();
			FVector3d Scale = Transform.GetScale3D();
			CapsuleElem.Length = Scale.Z * CapsuleGenerator.SegmentLength;
			CapsuleElem.Radius = CapsuleGenerator.Radius * FMath::Max(FMath::Abs(Scale.X), FMath::Abs(Scale.Y));
			SimpleCollision->AggGeom.SphylElems.Add(CapsuleElem);
		}

		return TargetMesh;
	}


}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	float DimensionZ,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendBox(TargetMesh, nullptr, PrimitiveOptions, Transform, DimensionX, DimensionY, DimensionZ, StepsX, StepsY, StepsZ, Origin, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBoxWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	float DimensionZ,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendBox(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform, DimensionX, DimensionY, DimensionZ, StepsX, StepsY, StepsZ, Origin, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBoundingBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FBox Box,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendBoundingBox(TargetMesh, nullptr, PrimitiveOptions, Transform, Box, StepsX, StepsY, StepsZ, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendBoundingBoxWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FBox Box,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendBoundingBox(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform, Box, StepsX, StepsY, StepsZ, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendOrientedBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FOrientedBox OrientedBox,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendBox(TargetMesh, nullptr, PrimitiveOptions, Transform, 
		UE::Geometry::FOrientedBox3d(OrientedBox), StepsX, StepsY, StepsZ, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendOrientedBoxWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FOrientedBox OrientedBox,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendBox(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform,
		UE::Geometry::FOrientedBox3d(OrientedBox), StepsX, StepsY, StepsZ, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereLatLong(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	int32 StepsPhi,
	int32 StepsTheta,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendSphereLatLong(TargetMesh, nullptr, PrimitiveOptions, Transform, Radius, StepsPhi, StepsTheta, Origin, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereLatLongWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	int32 StepsPhi,
	int32 StepsTheta,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendSphereLatLong(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform, Radius, StepsPhi, StepsTheta, Origin, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendSphereBox(TargetMesh, nullptr, PrimitiveOptions, Transform, Radius, StepsX, StepsY, StepsZ, Origin, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereBoxWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendSphereBox(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform, Radius, StepsX, StepsY, StepsZ, Origin, Debug);
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendCapsule(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	float LineLength,
	int32 HemisphereSteps,
	int32 CircleSteps,
	int32 SegmentSteps,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	return UE::Private::AppendMeshHelper::AppendCapsule(TargetMesh, nullptr, PrimitiveOptions, Transform, Radius, LineLength, HemisphereSteps, CircleSteps, SegmentSteps, Origin, Debug);
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendCapsuleWithCollision(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	float LineLength,
	int32 HemisphereSteps,
	int32 CircleSteps,
	int32 SegmentSteps,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	SimpleCollision.AggGeom.EmptyElements(); // Clear the existing elements since BP can re-use these output structs
	return UE::Private::AppendMeshHelper::AppendCapsule(TargetMesh, &SimpleCollision, PrimitiveOptions, Transform, Radius, LineLength, HemisphereSteps, CircleSteps, SegmentSteps, Origin, Debug);
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendCylinder(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	float Height,
	int32 RadialSteps,
	int32 HeightSteps,
	bool bCapped,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendCylinder", "AppendCylinder: TargetMesh is Null"));
		return TargetMesh;
	}

	FCylinderGenerator CylinderGenerator;
	CylinderGenerator.Radius[0] = FMath::Max(FMathf::ZeroTolerance, Radius);
	CylinderGenerator.Radius[1] = CylinderGenerator.Radius[0];
	CylinderGenerator.Height = FMath::Max(FMathf::ZeroTolerance, Height);
	CylinderGenerator.AngleSamples = FMath::Max(3, RadialSteps);
	CylinderGenerator.LengthSamples = FMath::Max(0, HeightSteps);
	CylinderGenerator.bCapped = bCapped;
	CylinderGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	CylinderGenerator.Generate();

	FVector3d OriginShift = (Origin == EGeometryScriptPrimitiveOriginMode::Center) ? FVector3d(0, 0, -(Height/2)) : FVector3d::Zero();
	AppendPrimitive(TargetMesh, &CylinderGenerator, Transform, PrimitiveOptions, OriginShift);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendCone(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float BaseRadius,
	float TopRadius,
	float Height,
	int32 RadialSteps,
	int32 HeightSteps,
	bool bCapped,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendCone", "AppendCone: TargetMesh is Null"));
		return TargetMesh;
	}

	FCylinderGenerator CylinderGenerator;
	CylinderGenerator.Radius[0] = FMath::Max(FMathf::ZeroTolerance, BaseRadius);
	CylinderGenerator.Radius[1] = FMath::Max(0, TopRadius);
	CylinderGenerator.Height = FMath::Max(FMathf::ZeroTolerance, Height);
	CylinderGenerator.AngleSamples = FMath::Max(3, RadialSteps);
	CylinderGenerator.LengthSamples = FMath::Max(0, HeightSteps);
	CylinderGenerator.bCapped = bCapped;
	CylinderGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	CylinderGenerator.Generate();

	FVector3d OriginShift = (Origin == EGeometryScriptPrimitiveOriginMode::Center) ? FVector3d(0, 0, -(Height/2)) : FVector3d::Zero();
	AppendPrimitive(TargetMesh, &CylinderGenerator, Transform, PrimitiveOptions, OriginShift);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTorus(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FGeometryScriptRevolveOptions RevolveOptions,
	float MajorRadius,
	float MinorRadius,
	int32 MajorSteps,
	int32 MinorSteps,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	FPolygon2d Circle = FPolygon2d::MakeCircle(FMath::Max(FMathf::ZeroTolerance, MinorRadius), FMath::Max(3, MinorSteps));
	TArray<FVector2D> PolygonVertices;
	FVector2d Shift = (Origin == EGeometryScriptPrimitiveOriginMode::Base) ? FVector2d(0, MinorRadius) : FVector2d::Zero();
	for (FVector2d v : Circle.GetVertices())
	{
		PolygonVertices.Add((FVector2D)(v+Shift));
	}
	return AppendRevolvePolygon(TargetMesh, PrimitiveOptions, Transform, PolygonVertices, RevolveOptions, MajorRadius, MajorSteps, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRevolvePolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	FGeometryScriptRevolveOptions RevolveOptions,
	float Radius,
	int32 Steps,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendRevolvePolygon_NullMesh", "AppendRevolvePolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendRevolvePolygon_InvalidPolygon", "AppendRevolvePolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}

	FRevolvePlanarPolygonGenerator RevolveGen;
	for (FVector2D Point : PolygonVertices)
	{
		RevolveGen.PolygonVertices.Add(FVector2d(Point.X + Radius, Point.Y));
	}
	RevolveGen.Steps = FMath::Max(Steps, 2);
	RevolveGen.RevolveDegrees = RevolveOptions.RevolveDegrees;
	RevolveGen.DegreeOffset = RevolveOptions.DegreeOffset;
	RevolveGen.bReverseDirection = RevolveOptions.bReverseDirection;
	RevolveGen.bProfileAtMidpoint = RevolveOptions.bProfileAtMidpoint;
	RevolveGen.bFillPartialRevolveEndcaps = RevolveOptions.bFillPartialRevolveEndcaps;

	FDynamicMesh3 ResultMesh = RevolveGen.GenerateMesh();

	// generate hard normals
	if (RevolveOptions.bHardNormals)
	{
		float NormalDotProdThreshold = FMathf::Cos( FMathf::Clamp(RevolveOptions.HardNormalAngle, 0.0, 180.0) * FMathf::DegToRad);
		FMeshNormals FaceNormalsCalc(&ResultMesh);
		FaceNormalsCalc.ComputeTriangleNormals();
		const TArray<FVector3d>& FaceNormals = FaceNormalsCalc.GetNormals();
		ResultMesh.Attributes()->PrimaryNormals()->CreateFromPredicate([&](int VID, int TA, int TB) {
			return !(FaceNormals[TA].Dot(FaceNormals[TB]) < NormalDotProdThreshold);
		}, 0);
		FMeshNormals RecomputeMeshNormals(&ResultMesh);
		RecomputeMeshNormals.RecomputeOverlayNormals(ResultMesh.Attributes()->PrimaryNormals(), true, true);
		RecomputeMeshNormals.CopyToOverlay(ResultMesh.Attributes()->PrimaryNormals(), false);
	}

	AppendPrimitiveMesh(TargetMesh, ResultMesh, Transform, PrimitiveOptions);
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSpiralRevolvePolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	FGeometryScriptRevolveOptions RevolveOptions,
	float Radius,
	int Steps,
	float RisePerRevolution,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSpiralRevolvePolygon_NullMesh", "AppendSpiralRevolvePolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSpiralRevolvePolygon_InvalidPolygon", "AppendSpiralRevolvePolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}

	FSpiralRevolvePlanarPolygonGenerator RevolveGen;
	for (FVector2D Point : PolygonVertices)
	{
		RevolveGen.PolygonVertices.Add(FVector2d(Point.X + Radius, Point.Y));
	}
	RevolveGen.Steps = FMath::Max(Steps, 2);
	RevolveGen.RevolveDegrees = FMath::Max(0.0001, RevolveOptions.RevolveDegrees);
	RevolveGen.DegreeOffset = RevolveOptions.DegreeOffset;
	RevolveGen.bReverseDirection = RevolveOptions.bReverseDirection;
	RevolveGen.bProfileAtMidpoint = RevolveOptions.bProfileAtMidpoint;
	RevolveGen.bFillPartialRevolveEndcaps = RevolveOptions.bFillPartialRevolveEndcaps;
	RevolveGen.RisePerFullRevolution = RisePerRevolution;

	FDynamicMesh3 ResultMesh = RevolveGen.GenerateMesh();

	// generate hard normals
	if (RevolveOptions.bHardNormals)
	{
		float NormalDotProdThreshold = FMathf::Cos( FMathf::Clamp(RevolveOptions.HardNormalAngle, 0.0, 180.0) * FMathf::DegToRad);
		FMeshNormals FaceNormalsCalc(&ResultMesh);
		FaceNormalsCalc.ComputeTriangleNormals();
		const TArray<FVector3d>& FaceNormals = FaceNormalsCalc.GetNormals();
		ResultMesh.Attributes()->PrimaryNormals()->CreateFromPredicate([&](int VID, int TA, int TB) {
			return !(FaceNormals[TA].Dot(FaceNormals[TB]) < NormalDotProdThreshold);
		}, 0);
		FMeshNormals RecomputeMeshNormals(&ResultMesh);
		RecomputeMeshNormals.RecomputeOverlayNormals(ResultMesh.Attributes()->PrimaryNormals(), true, true);
		RecomputeMeshNormals.CopyToOverlay(ResultMesh.Attributes()->PrimaryNormals(), false);
	}

	AppendPrimitiveMesh(TargetMesh, ResultMesh, Transform, PrimitiveOptions);
	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRevolvePath(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PathVertices,
	FGeometryScriptRevolveOptions RevolveOptions,
	int32 Steps,
	bool bCapped,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendRevolvePath_NullMesh", "AppendRevolvePath: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PathVertices.Num() < 2)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendRevolvePath_InvalidPolygon", "AppendRevolvePath: PathVertices array requires at least 2 positions"));
		return TargetMesh;
	}

	FRevolvePlanarPathGenerator RevolveGen;
	for (FVector2D Point : PathVertices)
	{
		RevolveGen.PathVertices.Add((FVector2d)Point);
	}
	RevolveGen.Steps = FMath::Max(Steps, 2);
	RevolveGen.bCapped = bCapped;
	RevolveGen.RevolveDegrees = RevolveOptions.RevolveDegrees;
	RevolveGen.DegreeOffset = RevolveOptions.DegreeOffset;
	RevolveGen.bReverseDirection = RevolveOptions.bReverseDirection;
	RevolveGen.bProfileAtMidpoint = RevolveOptions.bProfileAtMidpoint;
	RevolveGen.bFillPartialRevolveEndcaps = RevolveOptions.bFillPartialRevolveEndcaps;

	FDynamicMesh3 ResultMesh = RevolveGen.GenerateMesh();

	// generate hard normals
	if (RevolveOptions.bHardNormals)
	{
		float NormalDotProdThreshold = FMathf::Cos(FMathf::Clamp(RevolveOptions.HardNormalAngle, 0.0, 180.0) * FMathf::DegToRad);
		FMeshNormals FaceNormalsCalc(&ResultMesh);
		FaceNormalsCalc.ComputeTriangleNormals();
		const TArray<FVector3d>& FaceNormals = FaceNormalsCalc.GetNormals();
		ResultMesh.Attributes()->PrimaryNormals()->CreateFromPredicate([&](int VID, int TA, int TB) {
			return !(FaceNormals[TA].Dot(FaceNormals[TB]) < NormalDotProdThreshold);
		}, 0);
		FMeshNormals RecomputeMeshNormals(&ResultMesh);
		RecomputeMeshNormals.RecomputeOverlayNormals(ResultMesh.Attributes()->PrimaryNormals(), true, true);
		RecomputeMeshNormals.CopyToOverlay(ResultMesh.Attributes()->PrimaryNormals(), false);
	}

	AppendPrimitiveMesh(TargetMesh, ResultMesh, Transform, PrimitiveOptions);
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSweepPolyline(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolylineVertices,
	const TArray<FTransform>& SweepPath,
	const TArray<float>& PolylineTexParamU,
	const TArray<float>& SweepPathTexParamV,
	bool bLoop,
	float StartScale,
	float EndScale,
	float RotationAngleDeg,
	float MiterLimit,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolyline_NullMesh", "AppendSweepPolyline: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolylineVertices.Num() < 2)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolyline_InvalidPolygon", "AppendSweepPolyline: Polyline array requires at least 2 positions"));
		return TargetMesh;
	}
	if (SweepPath.Num() < 2)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolyline_InvalidSweepPath", "AppendSweepPolyline: SweepPath array requires at least 2 positions"));
		return TargetMesh;
	}
	if (PolylineTexParamU.Num() != 0 && PolylineTexParamU.Num() != PolylineVertices.Num())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolyline_InvalidTexParamU", "AppendSweepPolyline: Polyline Texture Parameter U array must be same length as PolylineVertices"));
		return TargetMesh;
	}
	if (SweepPathTexParamV.Num() != 0 && SweepPathTexParamV.Num() != (SweepPath.Num() + (bLoop?1:0)) )
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolyline_InvalidTexParamV", "AppendSweepPolyline: SweepPath Texture Parameter V array must be same length as SweepPath, or (length+1) for a closed loop"));
		return TargetMesh;
	}

	FMatrix2d Rotation2D = FMatrix2d::RotationDeg(-RotationAngleDeg);
	FGeneralizedCylinderGenerator SweepGen;
	for (const FVector2D& Point : PolylineVertices)
	{
		SweepGen.CrossSection.AppendVertex(Rotation2D * FVector2d(Point.X, Point.Y));
	}
	for (const FTransform& SweepXForm : SweepPath)
	{
		SweepGen.Path.Add(SweepXForm.GetLocation());
		FQuaterniond Rotation(SweepXForm.GetRotation());
		SweepGen.PathFrames.Add(
			FFrame3d(SweepXForm.GetLocation(), Rotation.AxisY(), Rotation.AxisZ(), Rotation.AxisX())
		);
		FVector3d Scale = SweepXForm.GetScale3D();
		SweepGen.PathScales.Add(FVector2d(Scale.Y, Scale.Z));
	}

	SweepGen.bProfileCurveIsClosed = false;
	SweepGen.bLoop = bLoop;
	if (PolylineTexParamU.Num() > 0)
	{
		SweepGen.CrossSectionTexCoord = PolylineTexParamU;
	}
	if (SweepPathTexParamV.Num() > 0)
	{
		SweepGen.PathTexCoord = SweepPathTexParamV;
	}
	SweepGen.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	SweepGen.InitialFrame = FFrame3d(SweepGen.Path[0]);
	SweepGen.StartScale = StartScale;
	SweepGen.EndScale = EndScale;
	if (MiterLimit > 1.0)
	{
		SweepGen.EnableMitering((double)MiterLimit);
	}

	SweepGen.Generate();

	AppendPrimitive(TargetMesh, &SweepGen, Transform, PrimitiveOptions);
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSimpleExtrudePolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	float Height,
	int32 HeightSteps,
	bool bCapped,
	EGeometryScriptPrimitiveOriginMode Origin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSimpleExtrudePolygon_NullMesh", "AppendSimpleExtrudePolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSimpleExtrudePolygon_InvalidPolygon", "AppendSimpleExtrudePolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}

	FGeneralizedCylinderGenerator ExtrudeGen;
	for (const FVector2D& Point : PolygonVertices)
	{
		ExtrudeGen.CrossSection.AppendVertex((FVector2d)Point);
	}

	int32 NumDivisions = FMath::Max(1, HeightSteps - 1);
	int32 NumPathSteps = NumDivisions + 1;
	double StepSize = (double)Height / (double)NumDivisions;

	for (int32 k = 0; k <= NumPathSteps; ++k)
	{
		double StepHeight = (k == NumPathSteps) ? Height : ((double)k * StepSize);
		ExtrudeGen.Path.Add(FVector3d(0, 0, StepHeight));
	}

	ExtrudeGen.InitialFrame = FFrame3d();
	ExtrudeGen.bCapped = bCapped;
	ExtrudeGen.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	ExtrudeGen.Generate();

	FVector3d OriginShift = (Origin == EGeometryScriptPrimitiveOriginMode::Center) ? FVector3d(0, 0, -(Height/2)) : FVector3d::Zero();
	AppendPrimitive(TargetMesh, &ExtrudeGen, Transform, PrimitiveOptions, OriginShift);
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSimpleSweptPolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	const TArray<FVector>& SweepPath,
	bool bLoop,
	bool bCapped,
	float StartScale,
	float EndScale,
	float RotationAngleDeg,
	float MiterLimit,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSimpleSweptPolygon_NullMesh", "AppendSimpleSweptPolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSimpleSweptPolygon_InvalidPolygon", "AppendSimpleSweptPolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}
	if (SweepPath.Num() < 2)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSimpleSweptPolygon_InvalidSweepPath", "AppendSimpleSweptPolygon: SweepPath array requires at least 2 positions"));
		return TargetMesh;
	}

	FGeneralizedCylinderGenerator SweepGen;
	FMatrix2d Rotation2D = FMatrix2d::RotationDeg(-RotationAngleDeg);
	for (FVector2D Point : PolygonVertices)
	{
		SweepGen.CrossSection.AppendVertex(Rotation2D * FVector2d(Point.X, Point.Y));
	}
	for (FVector SweepPathPos : SweepPath)
	{
		SweepGen.Path.Add(SweepPathPos);
	}

	SweepGen.bLoop = bLoop;
	SweepGen.bCapped = bCapped;
	SweepGen.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	SweepGen.InitialFrame = FFrame3d(SweepGen.Path[0]);
	SweepGen.StartScale = StartScale;
	SweepGen.EndScale = EndScale;
	if (MiterLimit > 1.0f)
	{
		SweepGen.EnableMitering((double)MiterLimit);
	}

	SweepGen.Generate();

	AppendPrimitive(TargetMesh, &SweepGen, Transform, PrimitiveOptions);
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSweepPolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	const TArray<FTransform>& SweepPath,
	bool bLoop,
	bool bCapped,
	float StartScale,
	float EndScale,
	float RotationAngleDeg,
	float MiterLimit,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolygon_NullMesh", "AppendSweepPolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolygon_InvalidPolygon", "AppendSweepPolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}
	if (SweepPath.Num() < 2)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendSweepPolygon_InvalidSweepPath", "AppendSweepPolygon: SweepPath array requires at least 2 positions"));
		return TargetMesh;
	}

	FMatrix2d Rotation2D = FMatrix2d::RotationDeg(-RotationAngleDeg);
	FGeneralizedCylinderGenerator SweepGen;
	for (const FVector2D& Point : PolygonVertices)
	{
		SweepGen.CrossSection.AppendVertex( Rotation2D * FVector2d(Point.X, Point.Y) );
	}
	for (const FTransform& SweepXForm : SweepPath)
	{
		SweepGen.Path.Add(SweepXForm.GetLocation());
		FQuaterniond Rotation(SweepXForm.GetRotation());
		SweepGen.PathFrames.Add(
			FFrame3d(SweepXForm.GetLocation(), Rotation.AxisY(), Rotation.AxisZ(), Rotation.AxisX())
		);
		FVector3d Scale = SweepXForm.GetScale3D();
		SweepGen.PathScales.Add(FVector2d(Scale.Y, Scale.Z));
	}

	SweepGen.bLoop = bLoop;
	SweepGen.bCapped = bCapped;
	SweepGen.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	SweepGen.InitialFrame = FFrame3d(SweepGen.Path[0]);
	SweepGen.StartScale = StartScale;
	SweepGen.EndScale = EndScale;
	if (MiterLimit > 1.0f)
	{
		SweepGen.EnableMitering((double)MiterLimit);
	}

	SweepGen.Generate();

	AppendPrimitive(TargetMesh, &SweepGen, Transform, PrimitiveOptions);
	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRectangleXY(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	int32 StepsWidth,
	int32 StepsHeight,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendRectangleXY", "AppendRectangle: TargetMesh is Null"));
		return TargetMesh;
	}

	FRectangleMeshGenerator RectGenerator;
	RectGenerator.Origin = FVector3d(0, 0, 0);
	RectGenerator.Normal = FVector3f::UnitZ();
	RectGenerator.Width = DimensionX;
	RectGenerator.Height = DimensionY;
	RectGenerator.WidthVertexCount = FMath::Max(0, StepsWidth);
	RectGenerator.HeightVertexCount = FMath::Max(0, StepsHeight);
	RectGenerator.bSinglePolyGroup = (PrimitiveOptions.PolygroupMode != EGeometryScriptPrimitivePolygroupMode::PerQuad);
	RectGenerator.Generate();

	AppendPrimitive(TargetMesh, &RectGenerator, Transform, PrimitiveOptions);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRectangle_Compatibility_5_0(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	int32 StepsWidth,
	int32 StepsHeight,
	UGeometryScriptDebug* Debug)
{
	return AppendRectangleXY(TargetMesh, PrimitiveOptions, Transform, DimensionX * 0.5, DimensionY * 0.5, StepsWidth, StepsHeight, Debug);
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRoundRectangleXY(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	float CornerRadius,
	int32 StepsWidth,
	int32 StepsHeight,
	int32 StepsRound,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendRoundRectangleXY", "AppendRoundRectangle: TargetMesh is Null"));
		return TargetMesh;
	}

	FRoundedRectangleMeshGenerator RectGenerator;
	RectGenerator.Origin = FVector3d(0, 0, 0);
	RectGenerator.Normal = FVector3f::UnitZ();
	RectGenerator.Width = DimensionX;
	RectGenerator.Height = DimensionY;
	RectGenerator.WidthVertexCount = FMath::Max(0, StepsWidth);
	RectGenerator.HeightVertexCount = FMath::Max(0, StepsHeight);
	RectGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, CornerRadius);
	RectGenerator.AngleSamples = FMath::Max(StepsRound, 3);
	RectGenerator.bSinglePolyGroup = (PrimitiveOptions.PolygroupMode != EGeometryScriptPrimitivePolygroupMode::PerQuad);
	RectGenerator.Generate();

	AppendPrimitive(TargetMesh, &RectGenerator, Transform, PrimitiveOptions);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendRoundRectangle_Compatibility_5_0(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float DimensionX,
	float DimensionY,
	float CornerRadius,
	int32 StepsWidth,
	int32 StepsHeight,
	int32 StepsRound,
	UGeometryScriptDebug* Debug)
{
	return AppendRoundRectangleXY(TargetMesh, PrimitiveOptions, Transform, DimensionX * 0.5, DimensionY * 0.5, CornerRadius, StepsWidth, StepsHeight, StepsRound, Debug);
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendDisc(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float Radius,
	int32 AngleSteps,
	int32 SpokeSteps,
	float StartAngle,
	float EndAngle,
	float HoleRadius,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendDisc", "AppendDisc: TargetMesh is Null"));
		return TargetMesh;
	}

	FDiscMeshGenerator DiscGenerator;
	FPuncturedDiscMeshGenerator PuncturedDiscGenerator;
	FDiscMeshGenerator* UseGenerator = &DiscGenerator;

	if (HoleRadius > 0)
	{
		PuncturedDiscGenerator.HoleRadius = HoleRadius;
		UseGenerator = &PuncturedDiscGenerator;
	}

	UseGenerator->Radius = FMath::Max(FMathf::ZeroTolerance, Radius);
	UseGenerator->Normal = FVector3f::UnitZ();
	UseGenerator->AngleSamples = FMath::Max(3, AngleSteps);
	UseGenerator->RadialSamples = FMath::Max(1, SpokeSteps);
	UseGenerator->StartAngle = StartAngle;
	UseGenerator->EndAngle = EndAngle;
	UseGenerator->bSinglePolygroup = (PrimitiveOptions.PolygroupMode != EGeometryScriptPrimitivePolygroupMode::PerQuad);
	UseGenerator->Generate();
	AppendPrimitive(TargetMesh, UseGenerator, Transform, PrimitiveOptions);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon3D(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector>& PolygonVertices,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendTriangulatedPolygon3D_InvalidInput", "AppendTriangulatedPolygon3D: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendTriangulatedPolygon3D_InvalidPolygon", "AppendTriangulatedPolygon3D: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}

	FVector3d Normal, UnusedPlanePoint = FVector::ZeroVector;
	PolygonTriangulation::ComputePolygonPlane<double>(PolygonVertices, Normal, UnusedPlanePoint);

	// Create 2D basis for projected points
	FVector BasisX, BasisY;
	VectorUtil::MakePerpVectors(Normal, BasisX, BasisY);
	
	TArray<FVector2d> ProjectedPoints;
	ProjectedPoints.SetNumUninitialized(PolygonVertices.Num());
	for (int32 Idx = 0; Idx < PolygonVertices.Num(); ++Idx)
	{
		FVector V = PolygonVertices[Idx];
		ProjectedPoints[Idx] = FVector2d(BasisX.Dot(V), BasisY.Dot(V));
	}

	// Triangulate via the projected positions
	TArray<FIndex3i> Triangles;
	PolygonTriangulation::TriangulateSimplePolygon(ProjectedPoints, Triangles, false);

	// Find 2D bounds to find UV origin and scale
	FAxisAlignedBox2d Bounds;
	for (FVector2d Pt : ProjectedPoints)
	{
		Bounds.Contain(Pt);
	}
	double Width = Bounds.Width(), Height = Bounds.Height();
	double UVScale = .01;
	if (PrimitiveOptions.UVMode == EGeometryScriptPrimitiveUVMode::ScaleToFill)
	{
		UVScale = 1.0 / FMath::Max3(Width, Height, FMathd::ZeroTolerance);
	}

	// Create the polygon mesh w/ projected UVs and fixed normal
	FDynamicMesh3 PolygonMesh;
	PolygonMesh.EnableAttributes();
	FDynamicMeshUVOverlay* UVOverlay = PolygonMesh.Attributes()->PrimaryUV();
	FDynamicMeshNormalOverlay* NormalOverlay = PolygonMesh.Attributes()->PrimaryNormals();
	FVector3f Normalf = FVector3f(Normal);
	for (int32 Idx = 0; Idx < PolygonVertices.Num(); ++Idx)
	{
		PolygonMesh.AppendVertex(PolygonVertices[Idx]);
		NormalOverlay->AppendElement(Normalf);
		FVector2f UV = FVector2f((ProjectedPoints[Idx] - Bounds.Min) * UVScale);
		UVOverlay->AppendElement(UV);
	}
	for (FIndex3i Tri : Triangles)
	{
		int32 TID = PolygonMesh.AppendTriangle(Tri);
		if (TID >= 0)
		{
			UVOverlay->SetTriangle(TID, Tri, false);
			NormalOverlay->SetTriangle(TID, Tri, false);
		}
	}

	// Append to the target
	AppendPrimitiveMesh(
		TargetMesh,
		PolygonMesh,
		Transform,
		PrimitiveOptions);

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendTriangulatedPolygon(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& PolygonVertices,
	bool bAllowSelfIntersections,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendTriangulatedPolygon_InvalidInput", "AppendTriangulatedPolygon: TargetMesh is Null"));
		return TargetMesh;
	}
	if (PolygonVertices.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AppendTriangulatedPolygon_InvalidPolygon", "AppendTriangulatedPolygon: PolygonVertices array requires at least 3 positions"));
		return TargetMesh;
	}

	FPolygon2d Polygon;
	for (FVector2D Vertex : PolygonVertices)
	{
		Polygon.AppendVertex(Vertex);
	}
	FGeneralPolygon2d GeneralPolygon(Polygon);

	FConstrainedDelaunay2d Triangulator;
	if (bAllowSelfIntersections)
	{
		FArrangement2d Arrangement(Polygon.Bounds());
		// arrangement2d builds a general 2d graph that discards orientation info ...
		Triangulator.FillRule = FConstrainedDelaunay2d::EFillRule::Odd;
		Triangulator.bOrientedEdges = false;
		Triangulator.bSplitBowties = true;
		for (FSegment2d Seg : GeneralPolygon.GetOuter().Segments())
		{
			Arrangement.Insert(Seg);
		}
		Triangulator.Add(Arrangement.Graph);
	}
	else
	{
		Triangulator.Add(GeneralPolygon);
	}

	bool bTriangulationSuccess = Triangulator.Triangulate([&GeneralPolygon](const TArray<FVector2d>& Vertices, FIndex3i Tri) 
	{
		return GeneralPolygon.Contains((Vertices[Tri.A] + Vertices[Tri.B] + Vertices[Tri.C]) / 3.0);	// keep triangles based on the input polygon's winding
	});

	// even if bTriangulationSuccess is false, there may still be some triangles, so only fail if the mesh is empty
	if (Triangulator.Triangles.Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AppendTriangulatedPolygon_Failed", "AppendTriangulatedPolygon: Failed to triangulate polygon"));
		return TargetMesh;
	}

	FFlatTriangulationMeshGenerator TriangulationMeshGen;
	TriangulationMeshGen.Vertices2D = Triangulator.Vertices;
	TriangulationMeshGen.Triangles2D = Triangulator.Triangles;
	AppendPrimitive(TargetMesh, &TriangulationMeshGen.Generate(), Transform, PrimitiveOptions);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendLinearStairs(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float StepWidth,
	float StepHeight,
	float StepDepth,
	int NumSteps,
	bool bFloating,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendLinearStairs", "AppendLinearStairs: TargetMesh is Null"));
		return TargetMesh;
	}

	TUniquePtr<FLinearStairGenerator> StairGenerator = (bFloating) ?
		MakeUnique<FFloatingStairGenerator>() : MakeUnique<FLinearStairGenerator>();

	StairGenerator->StepDepth = StepDepth;
	StairGenerator->StepWidth = StepWidth;
	StairGenerator->StepHeight = StepHeight;
	StairGenerator->NumSteps = FMath::Max(1, NumSteps);
	StairGenerator->bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	StairGenerator->bScaleUVByAspectRatio = (PrimitiveOptions.UVMode == EGeometryScriptPrimitiveUVMode::Uniform) ? true : false;
	StairGenerator->Generate();
	AppendPrimitive(TargetMesh, StairGenerator.Get(), Transform, PrimitiveOptions);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendCurvedStairs(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	float StepWidth,
	float StepHeight,
	float InnerRadius,
	float CurveAngle,
	int NumSteps,
	bool bFloating,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendCurvedStairs", "AppendCurvedStairs: TargetMesh is Null"));
		return TargetMesh;
	}

	TUniquePtr<FCurvedStairGenerator> StairGenerator = (bFloating) ?
		MakeUnique<FSpiralStairGenerator>() : MakeUnique<FCurvedStairGenerator>();

	StairGenerator->StepWidth = StepWidth;
	StairGenerator->StepHeight = StepHeight;
	StairGenerator->NumSteps = FMath::Max(1, NumSteps);
	StairGenerator->InnerRadius = InnerRadius;
	StairGenerator->CurveAngle = CurveAngle;
	StairGenerator->bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
	StairGenerator->bScaleUVByAspectRatio = (PrimitiveOptions.UVMode == EGeometryScriptPrimitiveUVMode::Uniform) ? true : false;
	StairGenerator->Generate();
	AppendPrimitive(TargetMesh, StairGenerator.Get(), Transform, PrimitiveOptions);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendVoronoiDiagram2D(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& VoronoiSites,
	FGeometryScriptVoronoiOptions VoronoiOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendVoronoiDiagram2D_NullTarget", "AppendVoronoiDiagram2D: TargetMesh is Null"));
		return TargetMesh;
	}

	if (VoronoiSites.Num() < 1)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendVoronoiDiagram2D_TooFewSites", "AppendVoronoiDiagram2D: VoronoiSites array requires at least 1 position"));
		return TargetMesh;
	}

	UE::Geometry::FDelaunay2 Delaunay;
	Delaunay.Triangulate(VoronoiSites);

	if (!Delaunay.CanComputeVoronoiCells())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("PrimitiveFunctions_AppendVoronoiDiagram2D_GenFailed", "AppendVoronoiDiagram2D: Voronoi diagram generation failed"));
		return TargetMesh;
	}

	TAxisAlignedBox2<double> AABB;
	if (VoronoiOptions.Bounds.IsValid)
	{
		AABB.Max = FVector2d(VoronoiOptions.Bounds.Max.X, VoronoiOptions.Bounds.Max.Y);
		AABB.Min = FVector2d(VoronoiOptions.Bounds.Min.X, VoronoiOptions.Bounds.Min.Y);
	}
	else
	{
		AABB.Contain(VoronoiSites);
	}
	TArray<TArray<FVector2d>> Polygons = Delaunay.ComputeVoronoiCells<double>(VoronoiSites, VoronoiOptions.bIncludeBoundary, AABB, (double)VoronoiOptions.BoundsExpand);

	if (Polygons.Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("PrimitiveFunctions_AppendVoronoiDiagram2D_NoPolygons", "AppendVoronoiDiagram2D: No Voronoi cells constructed"));
		return TargetMesh;
	}

	FFlatTriangulationMeshGenerator TriangulationMeshGen;
	auto AddCell = [&TriangulationMeshGen, &Polygons, &Debug](int32 PolyIdx)
	{
		if (PolyIdx < 0 || PolyIdx >= Polygons.Num())
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendVoronoiDiagram2D_BadID", "AppendVoronoiDiagram2D: Requested invalid cell ID"));
			return;
		}
		int32 Start = TriangulationMeshGen.Vertices2D.Num();
		TriangulationMeshGen.Vertices2D.Append(Polygons[PolyIdx]);
		for (int32 Off = 1; Off + 1 < Polygons[PolyIdx].Num(); ++Off)
		{
			TriangulationMeshGen.Triangles2D.Emplace(Start, Start + Off + 1, Start + Off);
			TriangulationMeshGen.Triangles2DPolygroups.Emplace(PolyIdx);
		}
	};

	if (VoronoiOptions.CreateCells.IsEmpty())
	{
		for (int32 PolyIdx = 0; PolyIdx < Polygons.Num(); ++PolyIdx)
		{
			AddCell(PolyIdx);
		}
	}
	else
	{
		for (int32 PolyIdx : VoronoiOptions.CreateCells)
		{
			AddCell(PolyIdx);
		}
	}

	if (TriangulationMeshGen.Vertices2D.Num() > 2 && TriangulationMeshGen.Triangles2D.Num() > 0)
	{
		AppendPrimitive(TargetMesh, &TriangulationMeshGen.Generate(), Transform, PrimitiveOptions);
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendDelaunayTriangulation2D(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const TArray<FVector2D>& VertexPositions,
	const TArray<FIntPoint>& ConstrainedEdges,
	FGeometryScriptConstrainedDelaunayTriangulationOptions TriangulationOptions,
	TArray<int32>& PositionsToVertexIDs,
	bool& bHasDuplicateVertices,
	UGeometryScriptDebug* Debug)
{
	bHasDuplicateVertices = false;
	PositionsToVertexIDs.Reset();

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendDelaunayTriangulation2D_NullTarget", "AppendDelaunayTriangulation2D: TargetMesh is Null"));
		return TargetMesh;
	}

	if (VertexPositions.Num() < 3)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendDelaunayTriangulation2D_TooFewVertices", "AppendDelaunayTriangulation2D: VertexPositions array requires at least 3 positions"));
		return TargetMesh;
	}

	UE::Geometry::FDelaunay2 Delaunay;
	TArray<FIndex2i> ConvertedEdges;
	if (!ConstrainedEdges.IsEmpty())
	{
		ConvertedEdges.Reserve(ConstrainedEdges.Num());
		for (FIntPoint Pt : ConstrainedEdges)
		{
			if (VertexPositions.IsValidIndex(Pt.X) && VertexPositions.IsValidIndex(Pt.Y))
			{
				ConvertedEdges.Emplace(Pt.X, Pt.Y);
			}
			else if (TriangulationOptions.bValidateEdgesInResult)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendDelaunayTriangulation2D_InvalidEdges", "AppendDelaunayTriangulation2D: Constrained Edges must only contain valid indices into Vertex Positions"));
				return TargetMesh;
			}
		}
		Delaunay.bValidateEdges = TriangulationOptions.bValidateEdgesInResult;
		Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
	}
	else
	{
		Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = TriangulationOptions.bRemoveDuplicateVertices;
	}
	bool bTriSuccess = Delaunay.Triangulate(VertexPositions, ConvertedEdges);

	if (!bTriSuccess)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("PrimitiveFunctions_AppendDelaunayTriangulation2D_GenFailed", "AppendDelaunayTriangulation2D: Delaunay triangulation failed"));
		return TargetMesh;
	}
	
	bHasDuplicateVertices = Delaunay.HasDuplicates();

	FFlatTriangulationMeshGenerator TriangulationMeshGen;
	TriangulationMeshGen.bReverseOrientation = true; // because FDelaunay2 generates triangles with reversed orientation vs what we want
	TriangulationMeshGen.Vertices2D = VertexPositions;
	if (ConstrainedEdges.IsEmpty() || TriangulationOptions.ConstrainedEdgesFillMode == EGeometryScriptPolygonFillMode::All)
	{
		TriangulationMeshGen.Triangles2D = Delaunay.GetTriangles();
	}
	else
	{
		// Convert FillMode enum to the GeometryCore library equivalent.  Note the "All" case is handled above.
		EGeometryScriptPolygonFillMode GeometryScriptFillMode = TriangulationOptions.ConstrainedEdgesFillMode;
		FDelaunay2::EFillMode GeometryCoreFillMode =
			GeometryScriptFillMode == EGeometryScriptPolygonFillMode::Solid ? FDelaunay2::EFillMode::Solid :
			GeometryScriptFillMode == EGeometryScriptPolygonFillMode::PositiveWinding ? FDelaunay2::EFillMode::PositiveWinding :
			GeometryScriptFillMode == EGeometryScriptPolygonFillMode::NonZeroWinding ? FDelaunay2::EFillMode::NonZeroWinding :
			GeometryScriptFillMode == EGeometryScriptPolygonFillMode::NegativeWinding ? FDelaunay2::EFillMode::NegativeWinding :
			FDelaunay2::EFillMode::OddWinding;
		TriangulationMeshGen.Triangles2D = Delaunay.GetFilledTriangles(ConvertedEdges, GeometryCoreFillMode);
	}

	if (TriangulationMeshGen.Vertices2D.Num() > 2 && TriangulationMeshGen.Triangles2D.Num() > 0)
	{
		AppendPrimitiveWithVertexIndexMap(TargetMesh, PositionsToVertexIDs, &TriangulationMeshGen.Generate(), Transform, PrimitiveOptions);
		
		// If requested, remove duplicate vertices from the final mesh
		if (TriangulationOptions.bRemoveDuplicateVertices && bHasDuplicateVertices)
		{
			TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
			{
				for (int32 Idx = 0; Idx < PositionsToVertexIDs.Num(); ++Idx)
				{
					int32 RemapIdx = Delaunay.RemapIfDuplicate(Idx);
					if (Idx != RemapIdx)
					{
						int32 DupID = PositionsToVertexIDs[Idx];
						checkSlow(!EditMesh.IsReferencedVertex(DupID));
						EditMesh.RemoveVertex(DupID);
						PositionsToVertexIDs[Idx] = INDEX_NONE;
					}
				}
			});
		}
	}

	return TargetMesh;
}

TArray<FIntPoint> UGeometryScriptLibrary_MeshPrimitiveFunctions::CreateConstrainedEdgesLoop(int32 NumVertices, int32 Start)
{
	TArray<FIntPoint> Loop;
	if (NumVertices < 3)
	{
		return Loop;
	}

	Loop.Reserve(NumVertices);
	for (int32 Idx = 0, Prev = NumVertices - 1; Idx < NumVertices; Prev = Idx++)
	{
		Loop.Emplace(Prev + Start, Idx + Start);
	}
	return Loop;
}

TArray<FIntPoint> UGeometryScriptLibrary_MeshPrimitiveFunctions::CreateConstrainedEdgesChain(int32 NumVertices, int32 Start)
{
	TArray<FIntPoint> Chain;
	if (NumVertices < 2)
	{
		return Chain;
	}

	Chain.Reserve(NumVertices - 1);
	for (int32 Idx = 0; Idx + 1 < NumVertices; ++Idx)
	{
		Chain.Emplace(Idx, Idx + 1);
	}
	return Chain;
}

UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendPolygonListTriangulation(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	FGeometryScriptGeneralPolygonList PolygonList,
	FGeometryScriptPolygonsTriangulationOptions TriangulationOptions,
	bool& bTriangulationError,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendPolygonListTriangulation_NullTarget", "AppendDelaunayTriangulation2D: TargetMesh is Null"));
		return TargetMesh;
	}
	if (!PolygonList.Polygons)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendPolygonListTriangulation_NullPolygonList", "AppendPolygonListTriangulation: Polygon List is not set"));
		return TargetMesh;
	}


	FFlatTriangulationMeshGenerator TriangulationMeshGen;
	TriangulationMeshGen.bReverseOrientation = true; // because FDelaunay2 generates triangles with reversed orientation vs what we want

	// Triangulate and append each polygon separately
	bTriangulationError = false;
	for (const UE::Geometry::FGeneralPolygon2d& Polygon : *PolygonList.Polygons)
	{
		UE::Geometry::FDelaunay2 Delaunay;
		Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
		TArray<FIndex3i> Triangles;
		TArray<FVector2d> Vertices;
		const bool bSuccess = Delaunay.Triangulate(Polygon, &Triangles, &Vertices, true);
		if (!bSuccess)
		{
			bTriangulationError = true;
			if (!TriangulationOptions.bStillAppendOnTriangulationError)
			{
				return TargetMesh;
			}
		}
		int32 VertStart = TriangulationMeshGen.Vertices2D.Num();
		TriangulationMeshGen.Vertices2D.Append(Vertices);
		TriangulationMeshGen.Triangles2D.Reserve(TriangulationMeshGen.Triangles2D.Num() + Triangles.Num());
		for (const FIndex3i& Tri : Triangles)
		{
			TriangulationMeshGen.Triangles2D.Emplace(Tri.A + VertStart, Tri.B + VertStart, Tri.C + VertStart);
		}
	}
	if (TriangulationMeshGen.Vertices2D.Num() > 2 && TriangulationMeshGen.Triangles2D.Num() > 0)
	{
		AppendPrimitive(TargetMesh, &TriangulationMeshGen.Generate(), Transform, PrimitiveOptions);
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSimpleCollisionShapes(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const FGeometryScriptSimpleCollision& SimpleCollision,
	FGeometryScriptSimpleCollisionTriangulationOptions TriangulationOptions,
	UGeometryScriptDebug* Debug
)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendSimpleCollisionShapes", "AppendSimpleCollisionShapes: TargetMesh is Null"));
		return TargetMesh;
	}

	FSimpleCollisionTriangulationSettings UseTriOptions;
	UseTriOptions.bUseBoxSphere = true;
	UseTriOptions.BoxSphereStepsPerSide = TriangulationOptions.SphereStepsPerSide;
	UseTriOptions.CapsuleCircleSteps = TriangulationOptions.CapsuleCircleSteps;
	UseTriOptions.CapsuleHemisphereSteps = TriangulationOptions.CapsuleHemisphereSteps;
	UseTriOptions.bApproximateLevelSetWithCubes = TriangulationOptions.bApproximateLevelSetsWithCubes;

	FSimpleCollisionToMeshAttributeSettings UseAttributeOptions(true /*enable attributes*/, false /*per-triangle normals*/, true /*initialize UVs*/);

	FVector ExternalScale = Transform.GetScale3D();
	Transform.SetScale3D(FVector::OneVector);
	ConvertSimpleCollisionToDynamicMeshes(
		SimpleCollision.AggGeom, ExternalScale,
		[&](int32 Index, const FKShapeElem& ShapeElem, FDynamicMesh3& Mesh)
		{
			AppendPrimitiveMesh(TargetMesh, Mesh, Transform, PrimitiveOptions);
		},
		UseTriOptions,
		UseAttributeOptions
	);

	if (!SimpleCollision.AggGeom.TaperedCapsuleElems.IsEmpty() || !SimpleCollision.AggGeom.SkinnedLevelSetElems.IsEmpty())
	{
		// Tapered capsules and level sets are not supported yet
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("PrimitiveFunctions_AppendSimpleCollisionShapes Tapered Capsules Unsupported", "AppendSimpleCollisionShapes: Tapered Capsules are not supported and will be skipped"));
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSphereCovering(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPrimitiveOptions PrimitiveOptions,
	FTransform Transform,
	const FGeometryScriptSphereCovering& SphereCovering,
	int32 StepsX,
	int32 StepsY,
	int32 StepsZ,
	UGeometryScriptDebug* Debug
)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendSphereCovering_NullTargetMesh", "AppendSphereCovering: TargetMesh is Null"));
		return TargetMesh;
	}
	if (!SphereCovering.Spheres.IsValid())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("PrimitiveFunctions_AppendSphereCovering_NullSphereCovering", "AppendSphereCovering: Sphere Covering is Null"));
		return TargetMesh;
	}

	for (int32 SphereIdx = 0; SphereIdx < SphereCovering.Spheres->Num(); ++SphereIdx)
	{
		FBoxSphereGenerator SphereGenerator;
		SphereGenerator.Box.Frame.Origin = SphereCovering.Spheres->GetCenter(SphereIdx);
		SphereGenerator.Radius = FMath::Max(FMathf::ZeroTolerance, SphereCovering.Spheres->GetRadius(SphereIdx));
		SphereGenerator.EdgeVertices = FIndex3i(StepsX, StepsY, StepsZ);
		SphereGenerator.bPolygroupPerQuad = (PrimitiveOptions.PolygroupMode == EGeometryScriptPrimitivePolygroupMode::PerQuad);
		SphereGenerator.Generate();
		AppendPrimitive(TargetMesh, &SphereGenerator, Transform, PrimitiveOptions);
	}

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE