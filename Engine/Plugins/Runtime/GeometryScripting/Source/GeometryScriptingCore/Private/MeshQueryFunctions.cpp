// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshQueryFunctions.h"
#include "UDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshQueries.h"
#include "MeshBoundaryLoops.h"
#include "Selections/MeshConnectedComponents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshQueryFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshQueryFunctions"

template<typename ReturnType> 
ReturnType SimpleMeshQuery(UDynamicMesh* Mesh, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh)> QueryFunc)
{
	if (Mesh)
	{
		ReturnType RetVal;
		Mesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			RetVal = QueryFunc(ReadMesh);
		});
		return RetVal;
	}
	return DefaultValue;
}


template<typename ReturnType> 
ReturnType SimpleMeshUVSetQuery(UDynamicMesh* Mesh, int32 UVSetIndex, bool& bIsValidUVSet, ReturnType DefaultValue, 
	TFunctionRef<ReturnType(const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVOverlay)> QueryFunc)
{
	bIsValidUVSet = false;
	ReturnType RetVal = DefaultValue;
	if (Mesh)
	{
		Mesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (ReadMesh.HasAttributes() && UVSetIndex < ReadMesh.Attributes()->NumUVLayers() )
			{
				const FDynamicMeshUVOverlay* Overlay = ReadMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (Overlay != nullptr)
				{
					bIsValidUVSet = true;
					RetVal = QueryFunc(ReadMesh, *ReadMesh.Attributes()->GetUVLayer(UVSetIndex));
				}
			}
		});
	}
	return RetVal;
}


namespace UE::MeshQueryFunctionLocals
{
	static bool ValidateMeshVertexID(const FDynamicMesh3& Mesh, int32 VID, bool bLogError = true)
	{
		if (!Mesh.IsVertex(VID))
		{
			if (bLogError)
			{
				UE::Geometry::MakeScriptError(EGeometryScriptErrorType::InvalidInputs, 
					FText::Format(LOCTEXT("MeshQueryInvalidVertexID", "Mesh queried with invalid vertex ID: {0} (Max ID: {1})"),
						VID, Mesh.MaxVertexID()));
			}
			return false;
		}
		return true;
	}
	
	static bool ValidateMeshTriangleID(const FDynamicMesh3& Mesh, int32 TID, bool bLogError = true)
	{
		if (!Mesh.IsTriangle(TID))
		{
			if (bLogError)
			{
				UE::Geometry::MakeScriptError(EGeometryScriptErrorType::InvalidInputs,
					FText::Format(LOCTEXT("MeshQueryInvalidTriangleID", "Mesh queried with invalid triangle ID: {0} (Max ID: {1})"),
						TID, Mesh.MaxTriangleID()));
			}
			return false;
		}
		return true;
	}
}

FString UGeometryScriptLibrary_MeshQueryFunctions::GetMeshInfoString(UDynamicMesh* TargetMesh)
{
	return SimpleMeshQuery<FString>(TargetMesh, FString(TEXT("Mesh is Null")), [&](const FDynamicMesh3& Mesh) { return Mesh.MeshInfoString(); });
}


bool UGeometryScriptLibrary_MeshQueryFunctions::GetIsDenseMesh( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, true, [&](const FDynamicMesh3& Mesh) { return Mesh.IsCompact(); });
}

bool UGeometryScriptLibrary_MeshQueryFunctions::GetMeshHasAttributeSet( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, true, [&](const FDynamicMesh3& Mesh) { return Mesh.HasAttributes(); });
}

FBox UGeometryScriptLibrary_MeshQueryFunctions::GetMeshBoundingBox( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<FBox>(TargetMesh, FBox(EForceInit::ForceInit), [&](const FDynamicMesh3& Mesh) { return (FBox)Mesh.GetBounds(true); });
}

void UGeometryScriptLibrary_MeshQueryFunctions::GetMeshVolumeArea( UDynamicMesh* TargetMesh, float& SurfaceArea, float& Volume )
{
	SurfaceArea = Volume = 0;
	SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh);
		Volume = VolArea.X;
		SurfaceArea = VolArea.Y;
		return true;
	});
}

void UGeometryScriptLibrary_MeshQueryFunctions::GetMeshVolumeAreaCenter(UDynamicMesh* TargetMesh, float& SurfaceArea, float& Volume, FVector& CenterOfMass)
{
	SurfaceArea = Volume = 0;
	CenterOfMass = FVector::ZeroVector;
	SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		FVector3d CoM;
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeAreaCenter(Mesh, CoM);
		Volume = VolArea.X;
		SurfaceArea = VolArea.Y;
		CenterOfMass = (FVector)CoM;
		return true;
	});
}

bool UGeometryScriptLibrary_MeshQueryFunctions::GetIsClosedMesh( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, true, [&](const FDynamicMesh3& Mesh) { return Mesh.IsClosed(); });
}

int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumOpenBorderEdges( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { 
		int32 Count = 0;
		for (int32 eid : Mesh.EdgeIndicesItr())
		{
			if (Mesh.IsBoundaryEdge(eid))
			{
				Count++;
			}
		}
		return Count;
	});
}

int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumOpenBorderLoops( UDynamicMesh* TargetMesh, bool& bAmbiguousTopologyFound )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { 
		FMeshBoundaryLoops BoundaryLoops(&Mesh, false);
		bAmbiguousTopologyFound = (BoundaryLoops.Compute() == false);
		return BoundaryLoops.GetLoopCount();
	});
}


int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumConnectedComponents( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { 
		FMeshConnectedComponents Components(&Mesh);
		Components.FindConnectedTriangles();
		return Components.Num();
	});
}



int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumTriangleIDs( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { return Mesh.MaxTriangleID(); });
}

bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasTriangleIDGaps( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) { return !Mesh.IsCompactT(); });
}

bool UGeometryScriptLibrary_MeshQueryFunctions::IsValidTriangleID( UDynamicMesh* TargetMesh, int32 TriangleID )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) { return Mesh.IsTriangle(TriangleID); });
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllTriangleIDs(UDynamicMesh* TargetMesh, FGeometryScriptIndexList& TriangleIDList, bool& bHasTriangleIDGaps)
{
	TriangleIDList.Reset(EGeometryScriptIndexType::Triangle);
	bHasTriangleIDGaps = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			TriangleIDList.List->Reserve(ReadMesh.TriangleCount());
			for (int32 tid : ReadMesh.TriangleIndicesItr())
			{
				TriangleIDList.List->Add(tid);
			}
			bHasTriangleIDGaps = ! ReadMesh.IsCompactT();
		});
	}
	return TargetMesh;
}

FIntVector UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleIndices(UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle)
{
	return SimpleMeshQuery<FIntVector>(TargetMesh, FIntVector::NoneValue, [&](const FDynamicMesh3& Mesh) {
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		return (bIsValidTriangle) ? (FIntVector)Mesh.GetTriangle(TriangleID) : FIntVector::NoneValue;
	});
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllTriangleIndices(UDynamicMesh* TargetMesh, FGeometryScriptTriangleList& TriangleList, bool bSkipGaps, bool& bHasTriangleIDGaps)
{
	TriangleList.Reset();
	TArray<FIntVector>& Triangles = *TriangleList.List;
	bHasTriangleIDGaps = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (bSkipGaps)
			{
				Triangles.Reserve(ReadMesh.TriangleCount());
				for (int32 tid : ReadMesh.TriangleIndicesItr())
				{
					Triangles.Add( (FIntVector)ReadMesh.GetTriangle(tid) );
				}
				bHasTriangleIDGaps = false;
			}
			else
			{
				Triangles.Init(FIntVector::NoneValue, ReadMesh.MaxTriangleID());
				for (int32 tid : ReadMesh.TriangleIndicesItr())
				{
					Triangles[tid] = (FIntVector)ReadMesh.GetTriangle(tid);
				}
				bHasTriangleIDGaps = ! ReadMesh.IsCompactT();
			}
		});
	}

	return TargetMesh;
}


void UGeometryScriptLibrary_MeshQueryFunctions::GetTrianglePositions(UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle, FVector& Vertex1, FVector& Vertex2, FVector& Vertex3)
{
	bIsValidTriangle = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) 
	{
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		if (bIsValidTriangle)
		{
			Mesh.GetTriVertices<FVector>(TriangleID, Vertex1, Vertex2, Vertex3);
		}
		else
		{
			Vertex1 = Vertex2 = Vertex3 = FVector::ZeroVector;
		}
		return bIsValidTriangle;
	});
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTrianglePosition(UDynamicMesh* TargetMesh, int32 TriangleID, FVector BarycentricCoords, bool& bIsValidTriangle, FVector& InterpolatedPosition)
{
	InterpolatedPosition = FVector::Zero();
	bIsValidTriangle = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) 
	{
		if (Mesh.IsTriangle(TriangleID))
		{
			FVector A,B,C;
			Mesh.GetTriVertices<FVector>(TriangleID, A, B, C);
			InterpolatedPosition = BarycentricCoords.X*A + BarycentricCoords.Y*B + BarycentricCoords.Z*C;
			return true;
		}
		return false;
	});
	return TargetMesh;
}



FVector UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleFaceNormal(UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle)
{
	return SimpleMeshQuery<FVector>(TargetMesh, FVector::ZeroVector, [&](const FDynamicMesh3& Mesh) {
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		return (bIsValidTriangle) ? (FVector)Mesh.GetTriNormal(TriangleID) : FVector::ZeroVector;
	});
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::ComputeTriangleBarycentricCoords(UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle, FVector Point, FVector& Vertex1, FVector& Vertex2, FVector& Vertex3, FVector& BarycentricCoords)
{
	BarycentricCoords = SimpleMeshQuery<FVector>(TargetMesh, FVector::Zero(), [&](const FDynamicMesh3& Mesh) {
		bIsValidTriangle = Mesh.IsTriangle(TriangleID);
		if (bIsValidTriangle)
		{
			Mesh.GetTriVertices(TriangleID, Vertex1, Vertex2, Vertex3);
			if ( VectorUtil::NormalDirection(Vertex1, Vertex2, Vertex3).SquaredLength() > FMathd::ZeroTolerance )
			{
				FVector BaryCoords = VectorUtil::BarycentricCoords(Point, Vertex1, Vertex2, Vertex3);
				// Point may not be inside the triangle in which case the coords are invalid, but we want to return something that won't cause later interpolations to explode...
				BaryCoords.X = FMathd::Clamp(BaryCoords.X, 0, 1);
				BaryCoords.Y = FMathd::Clamp(BaryCoords.Y, 0, 1);
				BaryCoords.Z = FMathd::Clamp(BaryCoords.Z, 0, 1);
				return BaryCoords;
			}
		}
		return FVector::Zero();
	});
	return TargetMesh;
}


int32 UGeometryScriptLibrary_MeshQueryFunctions::GetVertexCount( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { return Mesh.VertexCount(); });
}

int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumVertexIDs( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { return Mesh.MaxVertexID(); });
}

bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasVertexIDGaps( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) { return !Mesh.IsCompactV(); });
}

bool UGeometryScriptLibrary_MeshQueryFunctions::IsValidVertexID( UDynamicMesh* TargetMesh, int32 VertexID )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) { return Mesh.IsVertex(VertexID); });
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllVertexIDs(UDynamicMesh* TargetMesh, FGeometryScriptIndexList& VertexIDList, bool& bHasVertexIDGaps)
{
	VertexIDList.Reset(EGeometryScriptIndexType::Vertex);
	bHasVertexIDGaps = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			VertexIDList.List->Reserve(ReadMesh.VertexCount());
			for (int32 vid : ReadMesh.VertexIndicesItr())
			{
				VertexIDList.List->Add(vid);
			}
			bHasVertexIDGaps = ! ReadMesh.IsCompactV();
		});
	}
	return TargetMesh;
}

FVector UGeometryScriptLibrary_MeshQueryFunctions::GetVertexPosition(UDynamicMesh* TargetMesh, int32 VertexID, bool& bIsValidVertex)
{
	return SimpleMeshQuery<FVector>(TargetMesh, FVector::ZeroVector, [&](const FDynamicMesh3& Mesh) {
		bIsValidVertex = Mesh.IsVertex(VertexID);
		return (bIsValidVertex) ? (FVector)Mesh.GetVertex(VertexID) : FVector::ZeroVector;
	});
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllVertexPositions(UDynamicMesh* TargetMesh, FGeometryScriptVectorList& PositionList, bool bSkipGaps, bool& bHasVertexIDGaps)
{
	PositionList.Reset();
	TArray<FVector>& VertexPositions = *PositionList.List;
	bHasVertexIDGaps = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (bSkipGaps)
			{
				VertexPositions.Reserve(ReadMesh.VertexCount());
				for (int32 vid : ReadMesh.VertexIndicesItr())
				{
					VertexPositions.Add( (FVector)ReadMesh.GetVertex(vid) );
				}
				bHasVertexIDGaps = false;
			}
			else
			{
				VertexPositions.Init(FVector::ZeroVector, ReadMesh.MaxVertexID());
				for (int32 vid : ReadMesh.VertexIndicesItr())
				{
					VertexPositions[vid] = (FVector)ReadMesh.GetVertex(vid);
				}
				bHasVertexIDGaps = ! ReadMesh.IsCompactV();
			}
		});
	}
	
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetVertexConnectedTriangles(UDynamicMesh* TargetMesh, int32 VertexID, TArray<int32>& Triangles)
{
	SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) {
		if (!UE::MeshQueryFunctionLocals::ValidateMeshVertexID(Mesh, VertexID))
		{
			return 0;
		}
		Mesh.GetVtxTriangles(VertexID, Triangles);
		return 0;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetVertexConnectedVertices(UDynamicMesh* TargetMesh, int32 VertexID, TArray<int32>& Vertices)
{
	SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) {
		if (!UE::MeshQueryFunctionLocals::ValidateMeshVertexID(Mesh, VertexID))
		{
			return 0;
		}
		Mesh.EnumerateVertexVertices(VertexID, [&](int32 vid) { Vertices.Add(vid); });
		return 0;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllVertexPositionsAtEdges(UDynamicMesh* TargetMesh, const FGeometryScriptIndexList& EdgeIDs, FGeometryScriptVectorList& Start, FGeometryScriptVectorList& End)
{
	Start.Reset();
	End.Reset();
	
	if (!EdgeIDs.IsCompatibleWith(EGeometryScriptIndexType::Edge) && EdgeIDs.List.IsValid() && EdgeIDs.List->Num() > 0)
	{
		return TargetMesh;
	}

	SimpleMeshQuery<bool>(TargetMesh, false,
		[&EdgeIDs, &Start, &End](const FDynamicMesh3& Mesh)
		{
			for (const int EID : *EdgeIDs.List)
			{
				if (Mesh.IsEdge(EID))
				{
					FIndex2i VIDs = Mesh.GetEdgeV(EID);
					Start.List->Add(Mesh.GetVertex(VIDs[0]));
					End.List->Add(Mesh.GetVertex(VIDs[1]));
				}
			}
			return true;
		});
	return TargetMesh;
}


int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumUVSets( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int32>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) { 
		return Mesh.HasAttributes() ? Mesh.Attributes()->NumUVLayers() : 0; });
}

FBox2D UGeometryScriptLibrary_MeshQueryFunctions::GetUVSetBoundingBox(UDynamicMesh* TargetMesh, int UVSetIndex, bool& bIsValidUVSet, bool& bUVSetIsEmpty)
{
	return SimpleMeshUVSetQuery<FBox2D>(TargetMesh, UVSetIndex, bIsValidUVSet, FBox2D(EForceInit::ForceInit), 
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVSet)
	{
		FAxisAlignedBox2d Bounds = FAxisAlignedBox2d::Empty();
		int32 ValidTriCount = 0;
		for (int32 TriangleID : Mesh.TriangleIndicesItr())
		{
			FVector2f A, B, C;
			if (UVSet.IsSetTriangle(TriangleID))
			{
				UVSet.GetTriElements(TriangleID, A, B, C);
				Bounds.Contain((FVector2d)A); Bounds.Contain((FVector2d)B); Bounds.Contain((FVector2d)C);
				ValidTriCount++;
			}
		}
		bUVSetIsEmpty = (ValidTriCount == 0);
		return (ValidTriCount > 0) ? (FBox2D)Bounds : FBox2D(EForceInit::ForceInit);
	});
}

double UGeometryScriptLibrary_MeshQueryFunctions::GetMeshUVArea(UDynamicMesh* TargetMesh, int UVChannelIndex, bool& bIsValidUVChannel)
{
	return SimpleMeshUVSetQuery<double>(TargetMesh, UVChannelIndex, bIsValidUVChannel, 0.0,
		[](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVOverlay) -> double
		{
			double AreaSum = 0;
			for (int32 TriangleID : Mesh.TriangleIndicesItr())
			{
				FTriangle2f Tri;
				if (UVOverlay.IsSetTriangle(TriangleID))
				{
					UVOverlay.GetTriElements(TriangleID, Tri.V[0], Tri.V[1], Tri.V[2]);
					AreaSum += (double)Tri.Area();
				}
			}
			return AreaSum;
		});
}

void UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleUVs(UDynamicMesh* TargetMesh, int32 UVSetIndex, int32 TriangleID, 
	FVector2D& UV1, FVector2D& UV2, FVector2D& UV3, bool& bHaveValidUVs)
{
	bHaveValidUVs = SimpleMeshUVSetQuery<bool>(TargetMesh, UVSetIndex, bHaveValidUVs, false, 
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVSet) 
	{
		if (Mesh.IsTriangle(TriangleID) && UVSet.IsSetTriangle(TriangleID))
		{
			FVector2f A, B, C;
			UVSet.GetTriElements(TriangleID, A, B, C);
			UV1 = (FVector2D)A; UV2 = (FVector2D)B; UV3 = (FVector2D)C;
			return true;
		}
		else
		{
			UV1 = UV2 = UV3 = FVector2D::ZeroVector;
			return false;
		}
	});
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllSplitUVsAtVertex(UDynamicMesh* TargetMesh, int32 UVSetIndex, int32 VertexID,
	TArray<int32>& ElementIDs, TArray<FVector2D>& ElementUVs, bool& bHaveValidUVs)
{
	bHaveValidUVs = SimpleMeshUVSetQuery<bool>(TargetMesh, UVSetIndex, bHaveValidUVs, false, 
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVSet) 
	{
		if (Mesh.IsVertex(VertexID))
		{
			UVSet.EnumerateVertexElements(VertexID, [&](int TriangleID, int ElementID, const FVector2f& UVValue)
			{
				ElementIDs.Add(ElementID);
				ElementUVs.Add((FVector2D)UVValue);
				return true;
			});
			return true;
		}
		return false;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleUV(UDynamicMesh* TargetMesh, UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, int32 TriangleID, FVector BarycentricCoords, bool& bHaveValidUVs, FVector2D& InterpolatedUV)
{
	InterpolatedUV = FVector2D::Zero();
	bHaveValidUVs = SimpleMeshUVSetQuery<bool>(TargetMesh, UVSetIndex, bHaveValidUVs, false, 
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVSet) 
	{
		if (Mesh.IsTriangle(TriangleID) && UVSet.IsSetTriangle(TriangleID))
		{
			FVector2f A, B, C;
			UVSet.GetTriElements(TriangleID, A, B, C);
			InterpolatedUV = BarycentricCoords.X*(FVector2D)A + BarycentricCoords.Y*(FVector2D)B + BarycentricCoords.Z*(FVector2D)C;
			return true;
		}
		return false;
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetAllUVSeamEdges(UDynamicMesh* TargetMesh, int32 UVSetIndex, bool& bHaveValidUVs, FGeometryScriptIndexList& ElementIDs)
{
	ElementIDs.Reset(EGeometryScriptIndexType::Edge);
	bHaveValidUVs = SimpleMeshUVSetQuery<bool>(TargetMesh, UVSetIndex, bHaveValidUVs, false,
		[&](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVSet)
		{
			for (int EID : Mesh.EdgeIndicesItr())
			{
				if (UVSet.IsSeamEdge(EID))
				{
					ElementIDs.List->Add(EID);
				}
			}
			return true;
		});
	return TargetMesh;
}

int32 UGeometryScriptLibrary_MeshQueryFunctions::GetNumUVIslands(UDynamicMesh* TargetMesh, int32 UVChannel, bool& bIsValidUVChannel)
{
	return SimpleMeshUVSetQuery<int32>(TargetMesh, UVChannel, bIsValidUVChannel, -1,
		[](const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVOverlay) -> int32
		{
			auto UVIslandPredicate = [&UVOverlay](int32 Triangle0, int32 Triangle1)
				{
					return UVOverlay.AreTrianglesConnected(Triangle0, Triangle1);
				};

			FMeshConnectedComponents UVComponents(&Mesh);
			UVComponents.FindConnectedTriangles(UVIslandPredicate);
			int32 NumIslands = 0;
			// only count the islands with set triangles
			for (const FMeshConnectedComponents::FComponent& Component : UVComponents.Components)
			{
				NumIslands += (int32)(!Component.Indices.IsEmpty() && UVOverlay.IsSetTriangle(Component.Indices[0]));
			}
			return NumIslands;
		});
}


bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasTriangleNormals( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		return (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals() != nullptr);
	});
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleNormals(UDynamicMesh* TargetMesh, int32 TriangleID, FVector& Normal1, FVector& Normal2, FVector& Normal3, bool& bTriHasValidNormals)
{
	bTriHasValidNormals = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) 
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return false;
		}
		if (Mesh.HasAttributes())
		{
			if (const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals())
			{
				if ( Normals->IsSetTriangle(TriangleID) )
				{
					FVector3f A,B,C;
					Normals->GetTriElements(TriangleID, A,B,C);
					Normal1 = (FVector)A;
					Normal2 = (FVector)B;
					Normal3 = (FVector)C;
					return true;
				}
			}
		}
		return false;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleNormal(UDynamicMesh* TargetMesh, int32 TriangleID, FVector BarycentricCoords, bool& bTriHasValidNormals, FVector& InterpolatedNormal)
{
	bTriHasValidNormals = false;
	InterpolatedNormal = SimpleMeshQuery<FVector>(TargetMesh, FVector::UnitZ(), [&](const FDynamicMesh3& Mesh)
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return FVector::UnitZ();
		}
		if (Mesh.HasAttributes())
		{
			if (const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals())
			{
				if ( Normals->IsSetTriangle(TriangleID) )
				{
					bTriHasValidNormals = true;
					FVector3f A,B,C;
					Normals->GetTriElements(TriangleID, A,B,C);
					return BarycentricCoords.X*(FVector)A + BarycentricCoords.Y*(FVector)B + BarycentricCoords.Z*(FVector)C;
				}
			}
		}
		return FVector::UnitZ();
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleNormalTangents(UDynamicMesh* TargetMesh, int32 TriangleID, 
	bool& bTriHasValidElements, FGeometryScriptTriangle& NormalsTri, FGeometryScriptTriangle& TangentsTri, FGeometryScriptTriangle& BiTangentsTri)
{
	bTriHasValidElements = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) 
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return false;
		}
		if (Mesh.HasAttributes() && Mesh.Attributes()->HasTangentSpace() )
		{
			const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
			const FDynamicMeshNormalOverlay* Tangents = Mesh.Attributes()->PrimaryTangents();
			const FDynamicMeshNormalOverlay* BiTangents = Mesh.Attributes()->PrimaryBiTangents();
			if ( Normals->IsSetTriangle(TriangleID) && Tangents->IsSetTriangle(TriangleID) && BiTangents->IsSetTriangle(TriangleID) )
			{
				FVector3f A,B,C;
				Normals->GetTriElements(TriangleID, A,B,C);
				NormalsTri.Vector0 = (FVector)A; NormalsTri.Vector1 = (FVector)B; NormalsTri.Vector2 = (FVector)C;
				Tangents->GetTriElements(TriangleID, A,B,C);
				TangentsTri.Vector0 = (FVector)A; TangentsTri.Vector1 = (FVector)B; TangentsTri.Vector2 = (FVector)C;
				BiTangents->GetTriElements(TriangleID, A,B,C);
				BiTangentsTri.Vector0 = (FVector)A; BiTangentsTri.Vector1 = (FVector)B; BiTangentsTri.Vector2 = (FVector)C;
				return true;
			}
		}
		return false;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleNormalTangents(UDynamicMesh* TargetMesh, int32 TriangleID, FVector BarycentricCoords, 
	bool& bTriHasValidElements, FVector& InterpolatedNormal, FVector& InterpolatedTangent, FVector& InterpolatedBiTangent)
{
	bTriHasValidElements = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh)
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return false;
		}
		if (Mesh.HasAttributes() && Mesh.Attributes()->HasTangentSpace() )
		{
			const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
			const FDynamicMeshNormalOverlay* Tangents = Mesh.Attributes()->PrimaryTangents();
			const FDynamicMeshNormalOverlay* BiTangents = Mesh.Attributes()->PrimaryBiTangents();
			if ( Normals->IsSetTriangle(TriangleID) && Tangents->IsSetTriangle(TriangleID) && BiTangents->IsSetTriangle(TriangleID) )
			{
				FVector3f A,B,C;
				Normals->GetTriElements(TriangleID, A,B,C);
				InterpolatedNormal = BarycentricCoords.X*(FVector)A + BarycentricCoords.Y*(FVector)B + BarycentricCoords.Z*(FVector)C;
				Tangents->GetTriElements(TriangleID, A,B,C);
				InterpolatedTangent = BarycentricCoords.X*(FVector)A + BarycentricCoords.Y*(FVector)B + BarycentricCoords.Z*(FVector)C;
				BiTangents->GetTriElements(TriangleID, A,B,C);
				InterpolatedBiTangent = BarycentricCoords.X*(FVector)A + BarycentricCoords.Y*(FVector)B + BarycentricCoords.Z*(FVector)C;
				return true;
			}
		}
		return false;
	});
	return TargetMesh;
}



bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasVertexColors( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		return (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryColors() != nullptr);
	});
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleVertexColors(UDynamicMesh* TargetMesh, int32 TriangleID, FLinearColor& Color1, FLinearColor& Color2, FLinearColor& Color3, bool& bHaveValidVertexColors)
{
	bHaveValidVertexColors = SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) 
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return false;
		}
		if (Mesh.HasAttributes())
		{
			if (const FDynamicMeshColorOverlay* Colors = Mesh.Attributes()->PrimaryColors())
			{
				if ( Colors->IsSetTriangle(TriangleID) )
				{
					FVector4f A,B,C;
					Colors->GetTriElements(TriangleID, A,B,C);
					Color1 = (FLinearColor)A;
					Color2 = (FLinearColor)B;
					Color3 = (FLinearColor)C;
					return true;
				}
			}
		}
		return false;
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleVertexColor(UDynamicMesh* TargetMesh, int32 TriangleID, FVector BarycentricCoords, FLinearColor DefaultColor, bool& bHaveValidVertexColors, FLinearColor& InterpolatedColor)
{
	bHaveValidVertexColors = false;
	InterpolatedColor = SimpleMeshQuery<FLinearColor>(TargetMesh, DefaultColor, [&](const FDynamicMesh3& Mesh) 
	{
		if (!UE::MeshQueryFunctionLocals::ValidateMeshTriangleID(Mesh, TriangleID))
		{
			return DefaultColor;
		}
		if (Mesh.HasAttributes())
		{
			if (const FDynamicMeshColorOverlay* Colors = Mesh.Attributes()->PrimaryColors())
			{
				if ( Colors->IsSetTriangle(TriangleID) )
				{
					FVector4f A,B,C;
					Colors->GetTriElements(TriangleID, A,B,C);
					FVector4f Interpolated = BarycentricCoords.X*A + BarycentricCoords.Y*B + BarycentricCoords.Z*C;
					bHaveValidVertexColors = true;
					return (FLinearColor)Interpolated;
				}
			}
		}
		return DefaultColor;
	});
	return TargetMesh;
}


bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasMaterialIDs( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		return (Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID() && Mesh.Attributes()->GetMaterialID() != nullptr);
	});
}

bool UGeometryScriptLibrary_MeshQueryFunctions::GetHasPolygroups( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<bool>(TargetMesh, false, [&](const FDynamicMesh3& Mesh) {
		return Mesh.HasTriangleGroups();
	});
}

int UGeometryScriptLibrary_MeshQueryFunctions::GetNumExtendedPolygroupLayers( UDynamicMesh* TargetMesh )
{
	return SimpleMeshQuery<int>(TargetMesh, 0, [&](const FDynamicMesh3& Mesh) {
		return Mesh.HasAttributes() ? Mesh.Attributes()->NumPolygroupLayers() : 0;
	});
}

FGeometryScriptWeightMapHandle UGeometryScriptLibrary_MeshQueryFunctions::GetWeightMapHandle(UDynamicMesh* TargetMesh, FName Name, EGeometryScriptSearchOutcomePins& FoundWeights)
{
	FGeometryScriptWeightMapHandle Handle;
	Handle.WeightMapAttributeLayerIndex = SimpleMeshQuery<int32>(TargetMesh, (int32)-1, [&Name](const FDynamicMesh3& Mesh) -> int32
	{
		if (Mesh.HasAttributes())
		{
			for (int32 LayerIndex = 0; LayerIndex < Mesh.Attributes()->NumWeightLayers(); ++LayerIndex)
			{
				if (Mesh.Attributes()->GetWeightLayer(LayerIndex)->GetName() == Name)
				{
					return LayerIndex;
				}
			}
		}

		UE::Geometry::MakeScriptWarning(EGeometryScriptErrorType::InvalidInputs,
			FText::Format(LOCTEXT("InvalidWeightMapName", "Weight Map Layer with name \"{0}\" not found on TargetMesh"), FText::FromName(Name)));

		return -1;
	});

	FoundWeights = Handle.IsValid() ? EGeometryScriptSearchOutcomePins::Found : EGeometryScriptSearchOutcomePins::NotFound;

	return Handle;
}


#undef LOCTEXT_NAMESPACE

