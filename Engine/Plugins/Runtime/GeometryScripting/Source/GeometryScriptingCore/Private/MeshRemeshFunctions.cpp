// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshRemeshFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshAttributeUtil.h"
#include "MeshConstraintsUtil.h"
#include "ProjectionTargets.h"
#include "DynamicMesh/MeshNormals.h"
#include "CleaningOps/RemeshMeshOp.h"

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshRemeshFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshRemeshFunctions"

namespace GeometryScriptRemeshLocals
{
	static EEdgeRefineFlags MakeEdgeRefineFlagsFromConstraintType(EGeometryScriptRemeshEdgeConstraintType ConstraintType)
	{
		switch (ConstraintType)
		{
		case EGeometryScriptRemeshEdgeConstraintType::Fixed: return EEdgeRefineFlags::FullyConstrained;
		case EGeometryScriptRemeshEdgeConstraintType::Refine: return EEdgeRefineFlags::SplitsOnly;
		case EGeometryScriptRemeshEdgeConstraintType::Free: return EEdgeRefineFlags::NoFlip;
		case EGeometryScriptRemeshEdgeConstraintType::Ignore: return EEdgeRefineFlags::NoConstraint;
		}
		return EEdgeRefineFlags::NoFlip;
	}

	template<typename OptionsT>
	void CreateRemeshOp(TSharedPtr<FDynamicMesh3> SourceMesh,
		const FGeometryScriptRemeshOptions& RemeshOptions,
		const OptionsT& Options,
		FRemeshMeshOp& RemeshOp)
	{
		TSharedPtr<FDynamicMeshAABBTree3> SourceSpatial;
		if (RemeshOptions.bReprojectToInputMesh)
		{
			SourceSpatial = MakeShared<FDynamicMeshAABBTree3>(SourceMesh.Get(), true);
		}

		RemeshOp.OriginalMesh = SourceMesh;
		RemeshOp.OriginalMeshSpatial = SourceSpatial;

		RemeshOp.bDiscardAttributes = RemeshOptions.bDiscardAttributes;

		RemeshOp.RemeshType = (RemeshOptions.bUseFullRemeshPasses) ? ERemeshType::FullPass : ERemeshType::Standard;

		RemeshOp.RemeshIterations = RemeshOptions.RemeshIterations;
		RemeshOp.MaxRemeshIterations = RemeshOptions.RemeshIterations;
		RemeshOp.ExtraProjectionIterations = 0;		// unused for regular remeshing
		RemeshOp.TriangleCountHint = 0;				// unused for regular remeshing

		// smoothing options
		RemeshOp.SmoothingStrength = FMath::Clamp(RemeshOptions.SmoothingRate, 0.0f, 1.0f);
		switch (RemeshOptions.SmoothingType)
		{
		case EGeometryScriptRemeshSmoothingType::Mixed:
			RemeshOp.SmoothingType = ERemeshSmoothingType::MeanValue;
			break;
		case EGeometryScriptRemeshSmoothingType::UVPreserving:
			RemeshOp.SmoothingType = ERemeshSmoothingType::Cotangent;
			break;
		case EGeometryScriptRemeshSmoothingType::Uniform:
		default:
			RemeshOp.SmoothingType = ERemeshSmoothingType::Uniform;
			break;
		}

		if (Options.TargetType == EGeometryScriptUniformRemeshTargetType::TriangleCount)
		{
			RemeshOp.TargetEdgeLength = FRemeshMeshOp::CalculateTargetEdgeLength(SourceMesh.Get(), Options.TargetTriangleCount);
		}
		else
		{
			RemeshOp.TargetEdgeLength = Options.TargetEdgeLength;
		}

		// currently not exposing this option. It seems to control multiple things that should be independent...
		RemeshOp.bPreserveSharpEdges = (RemeshOp.bDiscardAttributes == false);
		RemeshOp.bFlips = RemeshOptions.bAllowFlips;
		RemeshOp.bSplits = RemeshOptions.bAllowSplits;
		RemeshOp.bCollapses = RemeshOptions.bAllowCollapses;
		RemeshOp.bPreventNormalFlips = RemeshOptions.bPreventNormalFlips;
		RemeshOp.bPreventTinyTriangles = RemeshOptions.bPreventTinyTriangles;

		RemeshOp.MeshBoundaryConstraint = MakeEdgeRefineFlagsFromConstraintType(RemeshOptions.MeshBoundaryConstraint);
		RemeshOp.GroupBoundaryConstraint = MakeEdgeRefineFlagsFromConstraintType(RemeshOptions.GroupBoundaryConstraint);
		RemeshOp.MaterialBoundaryConstraint = MakeEdgeRefineFlagsFromConstraintType(RemeshOptions.MaterialBoundaryConstraint);

		RemeshOp.bReproject = RemeshOptions.bReprojectToInputMesh;
		RemeshOp.ProjectionTarget = nullptr;
		RemeshOp.ProjectionTargetSpatial = nullptr;


		RemeshOp.bReprojectConstraints = false;
		RemeshOp.BoundaryCornerAngleThreshold = 45.0;


		RemeshOp.TargetMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.ToolMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.bUseWorldSpace = false;
		RemeshOp.bParallel = true;
	}

}

UDynamicMesh* UGeometryScriptLibrary_RemeshingFunctions::ApplyUniformRemesh(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptRemeshOptions RemeshOptions,
	FGeometryScriptUniformRemeshOptions UniformOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyUniformRemesh_InvalidInput", "ApplyUniformRemesh: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(EditMesh));

		FRemeshMeshOp RemeshOp;
		GeometryScriptRemeshLocals::CreateRemeshOp(SourceMesh, RemeshOptions, UniformOptions, RemeshOp);

		RemeshOp.CalculateResult(nullptr);
		if (RemeshOp.GetResultInfo().Result == EGeometryResultType::Success)
		{ 
			TUniquePtr<FDynamicMesh3> ResultMesh = RemeshOp.ExtractResult();
			EditMesh = MoveTemp(*ResultMesh);
		}
		else
		{
			EditMesh = MoveTemp(*SourceMesh);
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyUniformRemesh_ComputeError", "ApplyUniformRemesh: Error computing result, returning input mesh"));
		}

		// compact the input mesh if enabled
		if (RemeshOptions.bAutoCompact)
		{
			EditMesh.CompactInPlace();
		}

		// if we discarded attributes, re-enable the standard attributes, and initialize with per-vertex normals
		if (RemeshOptions.bDiscardAttributes)
		{
			EditMesh.EnableTriangleGroups();
			EditMesh.EnableAttributes();
			FMeshNormals::InitializeOverlayToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), false);
			EditMesh.Attributes()->EnableMaterialID();
		}


	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_RemeshingFunctions::ApplyAdaptiveRemesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptRemeshOptions RemeshOptions,
	FGeometryScriptAdaptiveRemeshOptions AdaptiveOptions,
	FGeometryScriptWeightMapHandle WeightMapHandle,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdaptiveRemesh_InvalidInput", "ApplyAdaptiveRemesh: TargetMesh is Null"));
		return TargetMesh;
	}

	if (!WeightMapHandle.IsValid() || WeightMapHandle.WeightMapAttributeLayerIndex >= TargetMesh->GetMeshRef().Attributes()->NumWeightLayers())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdaptiveRemesh_InvalidWeightMap", "ApplyAdaptiveRemesh: WeightMapHandle is not valid"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(EditMesh));

		FRemeshMeshOp RemeshOp;
		GeometryScriptRemeshLocals::CreateRemeshOp(SourceMesh, RemeshOptions, AdaptiveOptions, RemeshOp);

		if (SourceMesh->HasAttributes())
		{
			const int32 WeightMapAttributeLayerIndex = WeightMapHandle.WeightMapAttributeLayerIndex;
			RemeshOp.bNeedsWeightMaps = true;

			RemeshOp.CustomEdgeLengthScaleF = [WeightMapAttributeLayerIndex, RelativeDensity = AdaptiveOptions.RelativeDensity](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
			{
				const FDynamicMeshWeightAttribute* const AttributeLayer = Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex);

				float WeightValueA;
				AttributeLayer->GetValue(VertexA, &WeightValueA);
				WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

				float WeightValueB;
				AttributeLayer->GetValue(VertexB, &WeightValueB);
				WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

				const double AvgWeight = 0.5 * (WeightValueA + WeightValueB);

				return FMath::Pow(0.5, -RelativeDensity * AvgWeight);
			};
		}

		RemeshOp.CalculateResult(nullptr);
		if (RemeshOp.GetResultInfo().Result == EGeometryResultType::Success)
		{
			TUniquePtr<FDynamicMesh3> ResultMesh = RemeshOp.ExtractResult();
			EditMesh = MoveTemp(*ResultMesh);
		}
		else
		{
			EditMesh = MoveTemp(*SourceMesh);
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdaptiveRemesh_ComputeError", "ApplyAdaptiveRemesh: Error computing result, returning input mesh"));
		}

		// compact the input mesh if enabled
		if (RemeshOptions.bAutoCompact)
		{
			EditMesh.CompactInPlace();
		}

		// if we discarded attributes, re-enable the standard attributes, and initialize with per-vertex normals
		if (RemeshOptions.bDiscardAttributes)
		{
			EditMesh.EnableTriangleGroups();
			EditMesh.EnableAttributes();
			FMeshNormals::InitializeOverlayToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), false);
			EditMesh.Attributes()->EnableMaterialID();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
