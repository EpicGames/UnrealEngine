// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshRepairFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "Polygroups/PolygroupSet.h"
#include "MeshBoundaryLoops.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshResolveTJunctions.h"
#include "Operations/RemoveOccludedTriangles.h"
#include "Operations/DetectExteriorVisibility.h"
#include "Selections/GeometrySelection.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "UDynamicMesh.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"

#include "CleaningOps/HoleFillOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshRepairFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshRepairFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::CompactMesh(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CompactMesh_InvalidInput", "CompactMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		EditMesh.CompactInPlace();

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::RemoveUnusedVertices(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemoveUnusedVertices_InvalidInput", "RemoveUnusedVertices: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		EditMesh.RemoveUnusedVertices();
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::SnapMeshOpenBoundaries(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSnapBoundariesOptions SnapOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SnapMeshOpenBoundaries_InvalidInput", "SnapMeshOpenBoundaries: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMeshSnapOpenBoundaries Snapper(&EditMesh);
		Snapper.DistanceTolerance = 2 * SnapOptions.Tolerance;
		Snapper.MaxIterations = SnapOptions.MaxIterations;
		Snapper.bSnapToEdges = SnapOptions.bSnapToEdges;
		bool bResolveOK = Snapper.Apply();
		if (!bResolveOK)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("SnapMeshOpenBoundaries_Error", "SnapMeshOpenBoundaries: SnapMeshOpenBoundaries Operation returned error flag"));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::ResolveMeshTJunctions(
	UDynamicMesh* TargetMesh,
	FGeometryScriptResolveTJunctionOptions ResolveOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ResolveMeshTJunctions_InvalidInput", "ResolveMeshTJunctions: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMeshResolveTJunctions Resolver(&EditMesh);
		Resolver.DistanceTolerance = 2 * ResolveOptions.Tolerance;
		bool bResolveOK = Resolver.Apply();
		if (!bResolveOK)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ResolveMeshTJunctions_Error", "ResolveMeshTJunctions: ResolveTJunctions Operation returned error flag"));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::WeldMeshEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptWeldEdgesOptions WeldOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("WeldMeshEdges_InvalidInput", "WeldMeshEdges: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FMergeCoincidentMeshEdges Welder(&EditMesh);
		Welder.MergeVertexTolerance = WeldOptions.Tolerance;
		Welder.OnlyUniquePairs = WeldOptions.bOnlyUniquePairs;
		bool bOK = Welder.Apply();
		if (!bOK)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("WeldMeshEdges_Error", "WeldMeshEdges: Weld Operation returned error flag"));
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::FillAllMeshHoles(
	UDynamicMesh* TargetMesh,
	FGeometryScriptFillHolesOptions FillOptions,
	int32& NumFilledHoles,
	int32& NumFailedHoleFills,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FillAllMeshHoles_InvalidInput", "FillAllMeshHoles: TargetMesh is Null"));
		return TargetMesh;
	}

	TSharedPtr<FDynamicMesh3> MeshCopy = MakeShared<FDynamicMesh3>();
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh) { MeshCopy->Copy(EditMesh); });

	FMeshBoundaryLoops Loops(MeshCopy.Get(), true);

	FHoleFillOp HoleFiller;
	HoleFiller.OriginalMesh = MeshCopy;
	HoleFiller.Loops = MoveTemp(Loops.Loops);

	switch (FillOptions.FillMethod)
	{
	case EGeometryScriptFillHolesMethod::Automatic:
		// fall through
	case EGeometryScriptFillHolesMethod::MinimalFill:
		HoleFiller.FillType = EHoleFillOpFillType::Minimal;
		break;
	case EGeometryScriptFillHolesMethod::PolygonTriangulation:
		HoleFiller.FillType = EHoleFillOpFillType::PolygonEarClipping;
		break;
	case EGeometryScriptFillHolesMethod::TriangleFan:
		HoleFiller.FillType = EHoleFillOpFillType::TriangleFan;
		break;
	case EGeometryScriptFillHolesMethod::PlanarProjection:
		HoleFiller.FillType = EHoleFillOpFillType::Planar;
		break;
	}
	HoleFiller.FillOptions.bRemoveIsolatedTriangles = FillOptions.bDeleteIsolatedTriangles;
	HoleFiller.FillOptions.bQuickFillSmallHoles = true;

	HoleFiller.CalculateResult(nullptr);

	NumFailedHoleFills = HoleFiller.NumFailedLoops;
	NumFilledHoles = HoleFiller.Loops.Num() - NumFailedHoleFills;

	TUniquePtr<FDynamicMesh3> NewResultMesh = HoleFiller.ExtractResult();
	FDynamicMesh3* NewResultMeshPtr = NewResultMesh.Release();
	TargetMesh->SetMesh(MoveTemp(*NewResultMeshPtr));
	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::RemoveSmallComponents(
	UDynamicMesh* TargetMesh,
	FGeometryScriptRemoveSmallComponentOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemoveSmallComponents_InvalidInput", "RemoveSmallComponents: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FDynamicMeshEditor Editor(&EditMesh);
		Editor.RemoveSmallComponents(Options.MinVolume, Options.MinArea, Options.MinTriangleCount);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::RemoveHiddenTriangles(
	UDynamicMesh* TargetMesh,
	FGeometryScriptRemoveHiddenTrianglesOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemoveHiddenTriangles_InvalidInput", "RemoveHiddenTriangles: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TRemoveOccludedTriangles<FDynamicMesh3> Jacket(&EditMesh);

		Jacket.InsideMode = static_cast<UE::Geometry::EOcclusionCalculationMode>(Options.Method);
		//Jacket.TriangleSamplingMethod = UE::Geometry::EOcclusionTriangleSampling::Centroids
		Jacket.TriangleSamplingMethod = UE::Geometry::EOcclusionTriangleSampling::VerticesAndCentroids;
		Jacket.WindingIsoValue = Options.WindingIsoValue;
		Jacket.NormalOffset = Options.NormalOffset;
		Jacket.AddRandomRays = Options.RaysPerSample;
		Jacket.AddTriangleSamples = Options.SamplesPerTriangle;

		TArray<FTransformSRT3d> NoTransforms;
		NoTransforms.Add(FTransformSRT3d::Identity());

		//  set up AABBTree and FWNTree lists
		FDynamicMeshAABBTree3 Spatial(&EditMesh);
		TArray<FDynamicMeshAABBTree3*> OccluderTrees; 
		OccluderTrees.Add(&Spatial);
		
		TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial, false);
		TArray<TFastWindingTree<FDynamicMesh3>*> OccluderWindings; 
		if (Options.Method == EGeometryScriptRemoveHiddenTrianglesMethod::FastWindingNumber)
		{
			FastWinding.Build();
		}
		OccluderWindings.Add(&FastWinding);

		Jacket.Select(NoTransforms, OccluderTrees, OccluderWindings, NoTransforms);

		if (Options.ShrinkSelection > 0)
		{
			FMeshFaceSelection Selection(&EditMesh);
			Selection.Select(Jacket.RemovedT);
			Selection.ContractBorderByOneRingNeighbours(Options.ShrinkSelection);
			Jacket.RemovedT = Selection.AsArray();
		}
		
		if (Jacket.RemovedT.Num() > 0)
		{
			Jacket.RemoveSelected();

			if (EditMesh.TriangleCount() == 0)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("RemoveHiddenTriangles_EmptyMesh", "RemoveHiddenTriangles: Operation resulted in an empty mesh"));
			}

			if (Options.bCompactResult)
			{
				EditMesh.CompactInPlace();
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::SelectHiddenTrianglesFromOutside(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSelectHiddenTrianglesFromOutsideOptions Options,
	FGeometryScriptMeshSelection IgnoreSelection,
	FGeometryScriptMeshSelection TransparentSelection,
	FGeometryScriptMeshSelection& ResultSelection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SelectHiddenTrianglesFromOutside_InvalidInput", "SelectHiddenTrianglesFromOutside: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&Options, &IgnoreSelection, &TransparentSelection, &ResultSelection](const FDynamicMesh3& Mesh)
		{
			FExteriorVisibilitySampling SamplingParameters;
			SamplingParameters.bDoubleSided = Options.bDoubleSided;
			SamplingParameters.NumSearchDirections = Options.NumSearchDirections;
			SamplingParameters.SamplingDensity = Options.SampleSpacing;
			const bool bPerPolyGroup = Options.bPerPolyGroup;

			TSet<int32> TransparentTriangles, IgnoreTriangles;
			TransparentSelection.ProcessByTriangleID(Mesh, [&TransparentTriangles](int32 TriangleID)
				{
					TransparentTriangles.Add(TriangleID);
				});
			IgnoreSelection.ProcessByTriangleID(Mesh, [&IgnoreTriangles](int32 TriangleID)
				{
					IgnoreTriangles.Add(TriangleID);
				});
			auto IsTriTransparent = [&TransparentTriangles](int32 TID)
				{
					return TransparentTriangles.Contains(TID);
				};
			TArray<bool> SkipTris;
			if (!IgnoreTriangles.IsEmpty())
			{
				SkipTris.SetNumZeroed(Mesh.MaxTriangleID());
				for (int32 IgnoreTID : IgnoreTriangles)
				{
					SkipTris[IgnoreTID] = true;
				}
			}

			TArray<bool> ResultTriOccluded;

			if (!Options.RestrictToViewDirections.IsEmpty())
			{
				TArray<FVector3d> NormalizedRestrictDirections = Options.RestrictToViewDirections;
				for (FVector3d& V : NormalizedRestrictDirections)
				{
					V = Normalized(V);
				}

				TArray<FVector3d> FullTestDirections = FExteriorVisibilitySampling::GetRestrictedViewDirections(NormalizedRestrictDirections, Options.ViewDirectionAngleRange, Options.NumSearchDirections);
				FExteriorVisibilitySampling::ComputePerTriangleOcclusionFromDirections(Mesh, IsTriTransparent, ResultTriOccluded, bPerPolyGroup, NormalizedRestrictDirections, FullTestDirections, false, SamplingParameters, SkipTris);
			}
			else
			{
				FExteriorVisibilitySampling::ComputePerTriangleOcclusion(Mesh, IsTriTransparent, ResultTriOccluded, bPerPolyGroup, SamplingParameters, SkipTris);
			}

			
			// transfer boolean array to a geometry script selection result
			FGeometrySelection NewSelection;
			for (int32 TID = 0; TID < ResultTriOccluded.Num(); ++TID)
			{
				if (ResultTriOccluded[TID])
				{
					NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TID).Encoded());
				}
			}
			ResultSelection.SetSelection(MoveTemp(NewSelection));
		}
	);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::SplitMeshBowties(
	UDynamicMesh* TargetMesh,
	bool bMeshBowties,
	bool bAttributeBowties,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SplitMeshBowties_InvalidInput", "SplitMeshBowties: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (bMeshBowties)
		{
			FDynamicMeshEditor Editor(&EditMesh);
			FDynamicMeshEditResult EditResult;
			Editor.SplitBowties(EditResult);
		}

		if (bAttributeBowties)
		{
			if (EditMesh.HasAttributes())
			{
				EditMesh.Attributes()->SplitAllBowties(true);
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshRepairFunctions::RepairMeshDegenerateGeometry(
	UDynamicMesh* TargetMesh,
	FGeometryScriptDegenerateTriangleOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RepairMeshDegenerateGeometry_InvalidInput", "RepairMeshDegenerateGeometry: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bModified = false;

	auto DeleteByAreaPass = [&bModified, &Options](FDynamicMesh3& EditMesh)
	{
		TArray<int32> ToDelete;
		for (int32 tid : EditMesh.TriangleIndicesItr())
		{
			if (EditMesh.GetTriArea(tid) < Options.MinTriangleArea)
			{
				ToDelete.Add(tid);
			}
		}
		FDynamicMeshEditor Editor(&EditMesh);
		Editor.RemoveTriangles(ToDelete, true);
		bModified = true;
	};

	auto DeleteByEdgePass = [&bModified, &Options](FDynamicMesh3& EditMesh)
	{
		TArray<int32> ToDelete;
		for (int32 eid : EditMesh.EdgeIndicesItr())
		{
			FDynamicMesh3::FEdge Edge = EditMesh.GetEdge(eid);
			if ( Distance(EditMesh.GetVertex(Edge.Vert.A), EditMesh.GetVertex(Edge.Vert.B)) < Options.MinEdgeLength )
			{
				ToDelete.AddUnique(Edge.Tri.A);
				if (Edge.Tri.B != IndexConstants::InvalidID)
				{
					ToDelete.AddUnique(Edge.Tri.B);
				}
			}
		}
		FDynamicMeshEditor Editor(&EditMesh);
		Editor.RemoveTriangles(ToDelete, true);
		bModified = true;
	};


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		bool bTryRepair = (Options.Mode != EGeometryScriptRepairMeshMode::DeleteOnly);
		bool bDoDelete = (Options.Mode != EGeometryScriptRepairMeshMode::RepairOrSkip);

		if (bTryRepair)
		{
			FQEMSimplification Simplifier(&EditMesh);
			Simplifier.ProjectionMode = FQEMSimplification::ETargetProjectionMode::NoProjection;
			Simplifier.DEBUG_CHECK_LEVEL = 0;
			Simplifier.bRetainQuadricMemory = false;
			if (EditMesh.HasAttributes())
			{
				Simplifier.bAllowSeamCollapse = true;
				Simplifier.SetEdgeFlipTolerance(1.e-5);
				EditMesh.Attributes()->SplitAllBowties();	// eliminate any bowties that might have formed on attribute seams.
			}
			FMeshConstraints Constraints;
			FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, EditMesh,
				EEdgeRefineFlags::NoFlip, EEdgeRefineFlags::NoConstraint, EEdgeRefineFlags::NoConstraint,
				false, false, true);
			Simplifier.SetExternalConstraints(MoveTemp(Constraints));
			Simplifier.SimplifyToEdgeLength(Options.MinEdgeLength);

			//TODO: this would not resolve sliver triangles that have all long-edges...
			//TODO: could we do explicit collapse passes here? we would need to still respect constraints...

			bModified = true;
		}

		if (bDoDelete)
		{
			DeleteByAreaPass(EditMesh);
			DeleteByEdgePass(EditMesh);
		}

		if (bModified && Options.bCompactOnCompletion)
		{
			EditMesh.CompactInPlace();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE

