// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSamplingFunctions.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Sampling/MeshSurfacePointSampling.h"
#include "Scene/SceneCapturePhotoSet.h"
#include "GameFramework/Actor.h"
#include "Async/ParallelFor.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSamplingFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSamplingFunctions"


const UDynamicMesh* UGeometryScriptLibrary_MeshSamplingFunctions::ComputeUniformRandomPointSampling(
	const UDynamicMesh* TargetMesh,
	int32 NumSamples,
	TArray<FTransform>& Samples,
	FGeometryScriptIndexList& TriangleIDs,
	int RandomSeed,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeUniformRandomPointSampling_InvalidInput", "ComputeUniformRandomPointSampling: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([Debug, NumSamples, RandomSeed, &Samples, &TriangleIDs](const FDynamicMesh3& ReadMesh)
	{
		if (ReadMesh.TriangleCount() == 0)
		{
			return;
		}
		FMeshUniformSurfaceSampling MeshUniformSampling(ReadMesh);
		if (!MeshUniformSampling.IsValid())
		{
			// cannot sample mesh (e.g., if there are no triangles)
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeUniformRandomPointSampling_EmptyMesh", "ComputeUniformRandomPointSampling: TargetMesh does not have triangles to sample."));
			return;
		}
		FRandomStream RandomStream(RandomSeed);
		FTriangle3d Tri;
		TriangleIDs.Reset(EGeometryScriptIndexType::Triangle);
		TArray<int>& IndexList = *TriangleIDs.List;
		for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
		{
			int32 TID = MeshUniformSampling.DrawTriangleIDSample(RandomStream.GetFraction(), RandomStream.GetFraction());
			if (!ensure(ReadMesh.IsTriangle(TID))) // guard against sampling invalid IDs (should never happen)
			{
				continue;
			}
			ReadMesh.GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
			Samples.Add(FFrame3d(VectorUtil::UniformSampleTrianglePoint<double>(RandomStream.GetFraction(), RandomStream.GetFraction(), Tri.V[0], Tri.V[1], Tri.V[2]), Tri.Normal()).ToFTransform());
			IndexList.Add(TID);
		}
	});

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSamplingFunctions::ComputePointSampling(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshPointSamplingOptions Options,
	TArray<FTransform>& Samples,
	FGeometryScriptIndexList& TriangleIDs,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputePointSampling", "ComputePointSampling: TargetMesh is Null"));
		return TargetMesh;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(GS_ComputePointSampling);

	FMeshSurfacePointSampling Sampler;
	Sampler.SampleRadius = Options.SamplingRadius;
	Sampler.MaxSamples = Options.MaxNumSamples;
	Sampler.SubSampleDensity = Options.SubSampleDensity;
	Sampler.RandomSeed = Options.RandomSeed;
	Sampler.SamplingMethodVersion = Options.SamplingMethodVersion;

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		Sampler.ComputePoissonSampling(ReadMesh);
	});

	TriangleIDs.Reset(EGeometryScriptIndexType::Triangle);
	TArray<int>& IndexList = *TriangleIDs.List;
	int32 NumSamples = Sampler.Samples.Num();
	for ( int32 k = 0; k < NumSamples; ++k )
	{
		const FFrame3d& Frame = Sampler.Samples[k];
		Samples.Add( Frame.ToFTransform() );

		IndexList.Add( Sampler.TriangleIDs[k] );
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSamplingFunctions::ComputeNonUniformPointSampling(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshPointSamplingOptions Options,
	FGeometryScriptNonUniformPointSamplingOptions NonUniformOptions,
	TArray<FTransform>& Samples,
	TArray<double>& SampleRadii,
	FGeometryScriptIndexList& TriangleIDs,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeNonUniformPointSampling", "ComputeNonUniformPointSampling: TargetMesh is Null"));
		return TargetMesh;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(GS_ComputeNonUniformPointSampling);

	FMeshSurfacePointSampling Sampler;
	Sampler.SampleRadius = Options.SamplingRadius;
	Sampler.MaxSamples = Options.MaxNumSamples;
	Sampler.SubSampleDensity = Options.SubSampleDensity;
	Sampler.RandomSeed = Options.RandomSeed;
	Sampler.SamplingMethodVersion = Options.SamplingMethodVersion;
	if (NonUniformOptions.MaxSamplingRadius > Options.SamplingRadius)
	{
		Sampler.MaxSampleRadius = NonUniformOptions.MaxSamplingRadius;
		Sampler.SizeDistribution = (FMeshSurfacePointSampling::ESizeDistribution)(int)(NonUniformOptions.SizeDistribution);
		Sampler.SizeDistributionPower = FMath::Clamp(NonUniformOptions.SizeDistributionPower, 1.0, 10.0);
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		Sampler.ComputePoissonSampling(ReadMesh);
	});

	TriangleIDs.Reset(EGeometryScriptIndexType::Triangle);
	TArray<int>& IndexList = *TriangleIDs.List;
	int32 NumSamples = Sampler.Samples.Num();
	for ( int32 k = 0; k < NumSamples; ++k )
	{
		Samples.Add( Sampler.Samples[k].ToFTransform() );
		SampleRadii.Add( Sampler.Radii[k]);

		IndexList.Add( Sampler.TriangleIDs[k] );
	}

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshSamplingFunctions::ComputeVertexWeightedPointSampling(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshPointSamplingOptions Options,
	FGeometryScriptNonUniformPointSamplingOptions NonUniformOptions,
	FGeometryScriptScalarList VertexWeights,
	TArray<FTransform>& Samples,
	TArray<double>& SampleRadii,
	FGeometryScriptIndexList& TriangleIDs,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeVertexWeightedPointSampling_InvalidInput", "ComputeVertexWeightedPointSampling: TargetMesh is Null"));
		return TargetMesh;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(GS_ComputeVertexWeightedPointSampling);

	FMeshSurfacePointSampling Sampler;
	Sampler.SampleRadius = Options.SamplingRadius;
	Sampler.MaxSamples = Options.MaxNumSamples;
	Sampler.SubSampleDensity = Options.SubSampleDensity;
	Sampler.RandomSeed = Options.RandomSeed;
	Sampler.SamplingMethodVersion = Options.SamplingMethodVersion;
	bool bIsNonUniform = false;
	if (NonUniformOptions.MaxSamplingRadius > Options.SamplingRadius)
	{
		bIsNonUniform = true;
		Sampler.MaxSampleRadius = NonUniformOptions.MaxSamplingRadius;
		Sampler.SizeDistribution = (FMeshSurfacePointSampling::ESizeDistribution)(int)(NonUniformOptions.SizeDistribution);
		Sampler.SizeDistributionPower = FMath::Clamp(NonUniformOptions.SizeDistributionPower, 1.0, 10.0);
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if ( bIsNonUniform && VertexWeights.List.IsValid() && VertexWeights.List->Num() == ReadMesh.MaxVertexID() )
		{
			Sampler.VertexWeights = *VertexWeights.List.Get();
			Sampler.bUseVertexWeights = true;
			Sampler.InterpretWeightMode = (FMeshSurfacePointSampling::EInterpretWeightMode)(int)(NonUniformOptions.WeightMode);
			Sampler.bInvertWeights = NonUniformOptions.bInvertWeights;
		}
		Sampler.ComputePoissonSampling(ReadMesh);
	});

	TriangleIDs.Reset(EGeometryScriptIndexType::Triangle);
	TArray<int>& IndexList = *TriangleIDs.List;
	int32 NumSamples = Sampler.Samples.Num();
	for ( int32 k = 0; k < NumSamples; ++k )
	{
		Samples.Add( Sampler.Samples[k].ToFTransform() );
		SampleRadii.Add( Sampler.Radii[k]);

		IndexList.Add( Sampler.TriangleIDs[k] );
	}

	return TargetMesh;
}


// TODO Move this function into SceneCapturePhotoSet.h and use it in place of ComputeExteriorSpatialPhotoParameters
void UGeometryScriptLibrary_MeshSamplingFunctions::ComputeRenderCaptureCamerasForBox(
	TArray<FGeometryScriptRenderCaptureCamera>& Cameras,
	FBox Box,
	const FGeometryScriptRenderCaptureCamerasForBoxOptions& Options,
	UGeometryScriptDebug* Debug)
{
	TArray<FVector> Directions;

	if (Options.bViewFromBoxFaces)
	{
		Directions.Add( FVector3d::UnitX());
		Directions.Add(-FVector3d::UnitX());
		Directions.Add( FVector3d::UnitY());
		Directions.Add(-FVector3d::UnitY());
		Directions.Add( FVector3d::UnitZ());
		Directions.Add(-FVector3d::UnitZ());
	}

	if (Options.bViewFromUpperCorners)
	{
		Directions.Add(Normalized(FVector3d( 1,  1, -1)));
		Directions.Add(Normalized(FVector3d(-1,  1, -1)));
		Directions.Add(Normalized(FVector3d( 1, -1, -1)));
		Directions.Add(Normalized(FVector3d(-1, -1, -1)));
	}

	if (Options.bViewFromLowerCorners)
	{
		Directions.Add(Normalized(FVector3d( 1,  1, 1)));
		Directions.Add(Normalized(FVector3d(-1,  1, 1)));
		Directions.Add(Normalized(FVector3d( 1, -1, 1)));
		Directions.Add(Normalized(FVector3d(-1, -1, 1)));
	}

	if (Options.bViewFromUpperEdges)
	{
		Directions.Add(Normalized(FVector3d(-1,  0, -1)));
		Directions.Add(Normalized(FVector3d( 1,  0, -1)));
		Directions.Add(Normalized(FVector3d( 0, -1, -1)));
		Directions.Add(Normalized(FVector3d( 0,  1, -1)));
	}

	if (Options.bViewFromLowerEdges)
	{
		Directions.Add(Normalized(FVector3d(-1,  0, 1)));
		Directions.Add(Normalized(FVector3d( 1,  0, 1)));
		Directions.Add(Normalized(FVector3d( 0, -1, 1)));
		Directions.Add(Normalized(FVector3d( 0,  1, 1)));
	}

	if (Options.bViewFromSideEdges)
	{
		Directions.Add(Normalized(FVector3d( 1,  1, 0)));
		Directions.Add(Normalized(FVector3d(-1,  1, 0)));
		Directions.Add(Normalized(FVector3d( 1, -1, 0)));
		Directions.Add(Normalized(FVector3d(-1, -1, 0)));
	}

	for (FVector Position : Options.ExtraViewFromPositions)
	{
		if (Position.Normalize(TMathUtil<double>::ZeroTolerance))
		{
			Directions.Add(-Position);
		}
		else
		{
			const FText Message = FText::Format(
				LOCTEXT("ComputeRenderCaptureCamerasForBox", "ComputeRenderCaptureCamerasForBox: Unable to normalize view direction for position ({0}, {1}, {2})"),
				Position.X,
				Position.Y,
				Position.Z);
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, Message);
		}
	}

	FSphere Sphere = FBoxSphereBounds(Box).GetSphere();

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(Options.FieldOfViewDegrees) * .5f;
	const float CameraToCenterDist = Sphere.W / FMath::Tan(HalfFOVRadians);

	for (FVector ViewDirection : Directions)
	{
		ensure(IsNormalized(ViewDirection));

		FGeometryScriptRenderCaptureCamera Camera;
		Camera.NearPlaneDist = CameraToCenterDist - Sphere.W;
		Camera.FieldOfViewDegrees = Options.FieldOfViewDegrees;
		Camera.Resolution = FMath::Max(1, Options.Resolution);
		Camera.ViewDirection = ViewDirection;
		Camera.ViewPosition = Box.GetCenter() - CameraToCenterDist * Camera.ViewDirection;

		Cameras.Add(Camera);
	}
}



void UGeometryScriptLibrary_MeshSamplingFunctions::ComputeRenderCapturePointSampling(
	TArray<FTransform>& Samples,
	const TArray<AActor*>& Actors,
	const TArray<FGeometryScriptRenderCaptureCamera>& Cameras,
	UGeometryScriptDebug* Debug)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GS_ComputeRenderCapturePointSampling);

	// It's possible to pass nullptrs in Actors so we filter these out here
	const TArray<AActor*> ValidActors = Actors.FilterByPredicate([](AActor* Actor) { return Actor != nullptr; });
	if (ValidActors.IsEmpty())
	{
		return;
	}

	// Setup and configure SceneCapture
	TUniquePtr<FSceneCapturePhotoSet> SceneCapture = MakeUnique<FSceneCapturePhotoSet>();
	{
		ForEachCaptureType([&SceneCapture](ERenderCaptureType CaptureType)
		{
			const bool bCaptureTypeEnabled =
				CaptureType == ERenderCaptureType::DeviceDepth ||
				CaptureType == ERenderCaptureType::WorldNormal;
			SceneCapture->SetCaptureTypeEnabled(CaptureType, bCaptureTypeEnabled);

			FRenderCaptureConfig Config;
			Config.bAntiAliasing = false;
			SceneCapture->SetCaptureConfig(CaptureType, Config);
		});

		SceneCapture->SetCaptureSceneActors(ValidActors[0]->GetWorld(), ValidActors);

		TArray<FSpatialPhotoParams> SpatialParams;
		for (const FGeometryScriptRenderCaptureCamera& Camera : Cameras)
		{

			FSpatialPhotoParams Params;
			Params.NearPlaneDist = Camera.NearPlaneDist;
			Params.HorzFOVDegrees = Camera.FieldOfViewDegrees;
			Params.Dimensions = FImageDimensions(FMath::Max(1, Camera.Resolution), FMath::Max(1, Camera.Resolution));
			Params.Frame.AlignAxis(0, Camera.ViewDirection);
			Params.Frame.ConstrainedAlignAxis(2, FVector3d::UnitZ(), Params.Frame.X());
			Params.Frame.Origin = Camera.ViewPosition;
			SpatialParams.Add(Params);
		}
		SceneCapture->SetSpatialPhotoParams(SpatialParams);
	}

	SceneCapture->Compute();

	// Extract the Samples
	{
		FSceneCapturePhotoSet::FSceneSamples SceneSamples;
		TArray<FFrame3f> WorldOrientedPoints;
		SceneSamples.WorldOrientedPoints = &WorldOrientedPoints;
		SceneCapture->GetSceneSamples(SceneSamples);

		Samples.SetNum(WorldOrientedPoints.Num());
		ParallelFor(WorldOrientedPoints.Num(), [&Samples, &WorldOrientedPoints](int32 Index)
		{
			Samples[Index] = WorldOrientedPoints[Index].ToFTransform();
		});
	}
}



#undef LOCTEXT_NAMESPACE
