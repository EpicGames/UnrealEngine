// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSculptLayersFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSculptLayersFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSculptLayerFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::EnableSculptLayers(
	UDynamicMesh* TargetMesh,
	int32 NumLayers,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("EnableSculptLayers_InvalidInput", "EnableSculptLayers: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([NumLayers](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			EditMesh.EnableAttributes();
		}
		EditMesh.Attributes()->EnableSculptLayers(NumLayers);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::SetSculptLayerWeight(
	UDynamicMesh* TargetMesh,
	int32 LayerIndex,
	double Weight,
	FGeometryScriptSculptLayerUpdateOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSculptLayerWeight_InvalidInput", "SetSculptLayerWeight: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([LayerIndex, Weight, &Options, Debug](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSculptLayerWeight_NoAttribs", "SetSculptLayerWeight: TargetMesh does not have attributes enabled"));
			return;
		}
		if (LayerIndex >= 0 && LayerIndex < EditMesh.Attributes()->NumSculptLayers())
		{
			TArray<double> Wts(EditMesh.Attributes()->GetSculptLayers()->GetLayerWeights());
			Wts[LayerIndex] = Weight;
			EditMesh.Attributes()->GetSculptLayers()->UpdateLayerWeights(Wts);
			if (Options.bRecomputeNormals)
			{
				FMeshNormals::QuickRecomputeOverlayNormals(EditMesh);
			}
		}
		else
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSculptLayerWeight_LayerNotFound", "SetSculptLayerWeight: Requested sculpt layer was not enabled on TargetMesh. Use EnableSculptLayers first."));
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::SetSculptLayerWeightsArray(
	UDynamicMesh* TargetMesh,
	TArray<double> SetWeights,
	FGeometryScriptSculptLayerUpdateOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSculptLayerWeight_InvalidInput", "SetSculptLayerWeight: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&SetWeights, &Options, Debug](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetSculptLayerWeight_NoAttribs", "SetSculptLayerWeight: TargetMesh does not have attributes enabled"));
			return;
		}
		if (SetWeights.Num() > 0 && EditMesh.Attributes()->NumSculptLayers() > 0)
		{
			TArray<double> Wts(EditMesh.Attributes()->GetSculptLayers()->GetLayerWeights());
			int32 NumToCopy = FMath::Min(Wts.Num(), SetWeights.Num());
			for (int32 Idx = 0; Idx < NumToCopy; ++Idx)
			{
				Wts[Idx] = SetWeights[Idx];
			}
			EditMesh.Attributes()->GetSculptLayers()->UpdateLayerWeights(Wts);
			if (Options.bRecomputeNormals)
			{
				FMeshNormals::QuickRecomputeOverlayNormals(EditMesh);
			}
		}
		else
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetActiveSculptLayer_LayerNotFound", "SetActiveSculptLayer: Requested sculpt layer was not enabled on TargetMesh. Use EnableSculptLayers first."));
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

TArray<double> UGeometryScriptLibrary_MeshSculptLayersFunctions::GetSculptLayerWeightsArray(UDynamicMesh* TargetMesh)
{
	TArray<double> Weights;

	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Error, "GetActiveSculptLayer: TargetMesh is Null");
		return Weights;
	}

	TargetMesh->ProcessMesh([&Weights](const FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->NumSculptLayers())
		{
			UE_LOGF(LogGeometry, Error, "GetActiveSculptLayer: TargetMesh does not have sculpt layers enabled");
			return;
		}
		Weights = EditMesh.Attributes()->GetSculptLayers()->GetLayerWeights();
	});

	return Weights;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::SetActiveSculptLayer(
	UDynamicMesh* TargetMesh,
	int32 LayerIndex,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetActiveSculptLayer_InvalidInput", "SetActiveSculptLayer: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([LayerIndex, Debug](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetActiveSculptLayer_NoAttribs", "SetActiveSculptLayer: TargetMesh does not have attributes enabled"));
			return;
		}
		if (LayerIndex >= 0 && LayerIndex < EditMesh.Attributes()->NumSculptLayers())
		{
			EditMesh.Attributes()->GetSculptLayers()->SetActiveLayer(LayerIndex);
		}
		else
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetActiveSculptLayer_LayerNotFound", "SetActiveSculptLayer: Requested sculpt layer was not enabled on TargetMesh. Use EnableSculptLayers first."));
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

int32 UGeometryScriptLibrary_MeshSculptLayersFunctions::GetNumSculptLayers(const UDynamicMesh* TargetMesh)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Error, "GetNumSculptLayers: TargetMesh is Null");
		return -1;
	}

	int32 FoundLayerCount = -1;
	TargetMesh->ProcessMesh([&FoundLayerCount](const FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			UE_LOGF(LogGeometry, Error, "GetNumSculptLayers: TargetMesh does not have attributes enabled");
			return;
		}
		FoundLayerCount = EditMesh.Attributes()->NumSculptLayers();
	});
	return FoundLayerCount;
}

int32 UGeometryScriptLibrary_MeshSculptLayersFunctions::GetActiveSculptLayer(const UDynamicMesh* TargetMesh)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Error, "GetActiveSculptLayer: TargetMesh is Null");
		return INDEX_NONE;
	}

	int32 ActiveSculptLayer = INDEX_NONE;
	TargetMesh->ProcessMesh([&ActiveSculptLayer](const FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes() || !EditMesh.Attributes()->NumSculptLayers())
		{
			UE_LOGF(LogGeometry, Error, "GetActiveSculptLayer: TargetMesh does not have sculpt layers enabled");
			return;
		}
		ActiveSculptLayer = EditMesh.Attributes()->GetSculptLayers()->GetActiveLayer();
	});
	return ActiveSculptLayer;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::DiscardSculptLayers(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DiscardSculptLayers_InvalidInput", "DiscardSculptLayers: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			return;
		}
		EditMesh.Attributes()->DiscardSculptLayers();
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSculptLayersFunctions::MergeSculptLayers(
	UDynamicMesh* TargetMesh,
	int32& OutActiveLayer,
	int32 MergeLayerStart,
	int32 MergeLayerNum,
	bool bUseWeights,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MergeSculptLayers_InvalidInput", "MergeSculptLayers: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([MergeLayerStart, MergeLayerNum, &OutActiveLayer, bUseWeights, Debug](FDynamicMesh3& EditMesh)
	{
		if (!EditMesh.HasAttributes())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MergeSculptLayers_NoAttribs", "MergeSculptLayers: TargetMesh does not have attributes enabled"));
			return;
		}
		if (FDynamicMeshSculptLayers* SculptLayers = EditMesh.Attributes()->GetSculptLayers())
		{
			if (MergeLayerStart >= 0 && MergeLayerStart < SculptLayers->NumLayers())
			{
				int32 EndLayer = FMath::Min(SculptLayers->NumLayers() - 1, MergeLayerStart + MergeLayerNum);
				if (EndLayer > MergeLayerStart)
				{
					SculptLayers->MergeSculptLayers(MergeLayerStart, EndLayer, bUseWeights);
				}
			}
			else
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("MergeSculptLayers_LayerNotFound", "MergeSculptLayers: Requested merge start layer not found on TargetMesh."));
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
