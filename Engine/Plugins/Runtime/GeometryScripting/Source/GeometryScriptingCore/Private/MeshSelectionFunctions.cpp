// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSelectionFunctions.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "Selections/GeometrySelection.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "SphereTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshEdgeSelection.h"
#include "Selections/MeshVertexSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSelectionFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSelectionFunctions"



namespace UE::MeshSelectionLocals
{

// Helper to create a selection for all triangles matching a given filter (or all vertices/groups referencing those triangles)
static void SelectByTriangleAttribute(const FDynamicMesh3& ReadMesh, 
	FGeometryScriptMeshSelection& Selection, EGeometryScriptMeshSelectionType SelectionType,
	TFunctionRef<bool(int32)> TriangleFilter
)
{
	FGeometrySelection NewSelection;
	if (SelectionType == EGeometryScriptMeshSelectionType::Vertices)
	{
		NewSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
		for (int32 TID : ReadMesh.TriangleIndicesItr())
		{
			if (TriangleFilter(TID))
			{
				FIndex3i Tri = ReadMesh.GetTriangle(TID);

				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					NewSelection.Selection.Add(FGeoSelectionID::MeshVertex(Tri[SubIdx]).Encoded());
				}
			}
		}
	}
	else if (SelectionType == EGeometryScriptMeshSelectionType::Triangles)
	{
		NewSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
		for (int32 TID : ReadMesh.TriangleIndicesItr())
		{
			if (TriangleFilter(TID))
			{
				NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TID).Encoded());
			}
		}
	}
	else if (SelectionType == EGeometryScriptMeshSelectionType::Edges)
	{
		NewSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
		for (int32 TID : ReadMesh.TriangleIndicesItr())
		{
			if (TriangleFilter(TID))
			{
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					// Note edges are added per triangle that contains them, ala half-edges
					NewSelection.Selection.Add(FGeoSelectionID::MeshEdge(FMeshTriEdgeID(TID, SubIdx)).Encoded());
				}
			}
		}
	}
	else
	{
		NewSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);
		TSet<int32> UniqueGroupIDs;
		for (int32 TID : ReadMesh.TriangleIndicesItr())
		{
			if (!TriangleFilter(TID))
			{
				continue;
			}

			int32 GroupID = ReadMesh.GetTriangleGroup(TID);
			if (UniqueGroupIDs.Contains(GroupID) == false)
			{
				NewSelection.Selection.Add(FGeoSelectionID::GroupFace(TID, GroupID).Encoded());
				UniqueGroupIDs.Add(GroupID);
			}
		}
	}
	Selection.SetSelection(MoveTemp(NewSelection));
}

// Helper to select elements based on position/normal of triangles
static void SelectMeshElementsWithContainmentTest(
	UDynamicMesh* TargetMesh,
	TFunctionRef<bool(const FVector3d& Position, const FVector3d& Normal)> ContainmentFunc,
	FGeometryScriptMeshSelection& SelectionOut,
	EGeometryScriptMeshSelectionType SelectionType,
	int NumTrianglePoints,
	bool bNeedsNormals)
{
	NumTrianglePoints = FMath::Clamp(NumTrianglePoints, 1, 3);

	FGeometrySelection GeoSelection;
	if (SelectionType == EGeometryScriptMeshSelectionType::Vertices)
	{
		GeoSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);

		TargetMesh->ProcessMesh([&ContainmentFunc, &GeoSelection, bNeedsNormals](const FDynamicMesh3& Mesh)
		{
			for (int32 vid : Mesh.VertexIndicesItr())
			{
				FVector3d UseNormal = (bNeedsNormals) ? FMeshNormals::ComputeVertexNormal(Mesh,vid) : FVector3d::UnitZ();
				if ( ContainmentFunc(Mesh.GetVertex(vid), UseNormal) )
				{
					GeoSelection.Selection.Add( FGeoSelectionID::MeshVertex(vid).Encoded() );
				}
			}
		});
	}
	else if (SelectionType == EGeometryScriptMeshSelectionType::Edges)
	{
		GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
		TargetMesh->ProcessMesh([&ContainmentFunc, &GeoSelection, bNeedsNormals, NumTrianglePoints](const FDynamicMesh3& Mesh)
		{
			int32 UseNumEdgePoints = FMath::Clamp(NumTrianglePoints, 1, 2);
			for (int32 EID : Mesh.EdgeIndicesItr())
			{
				FIndex2i EdgeV = Mesh.GetEdgeV(EID);
				FVector3d UseNormal = bNeedsNormals ? Mesh.GetEdgeNormal(EID) : FVector3d::UnitZ();
				int32 NumContained =
					int32(ContainmentFunc(Mesh.GetVertex(EdgeV.A), UseNormal)) +
					int32(ContainmentFunc(Mesh.GetVertex(EdgeV.B), UseNormal));
				if (NumContained >= UseNumEdgePoints)
				{
					Mesh.EnumerateTriEdgeIDsFromEdgeID(EID, [&GeoSelection](FMeshTriEdgeID TriEdgeID)
					{
						GeoSelection.Selection.Add(TriEdgeID.Encoded());
					});
				}
			}
		});
	}
	else
	{
		GeoSelection.InitializeTypes(EGeometryElementType::Face, 
			(SelectionType == EGeometryScriptMeshSelectionType::Triangles) ? EGeometryTopologyType::Triangle : EGeometryTopologyType::Polygroup);

		TargetMesh->ProcessMesh([&ContainmentFunc, &GeoSelection, SelectionType, NumTrianglePoints, bNeedsNormals](const FDynamicMesh3& Mesh)
		{
			for (int32 tid : Mesh.TriangleIndicesItr())
			{
				FVector3d UseNormal = (bNeedsNormals) ? Mesh.GetTriNormal(tid) : FVector3d::UnitZ();
				FIndex3i Tri = Mesh.GetTriangle(tid);
				// may be wasteful to test each vertex multiple times...could accumulate a cache at cost of some memory allocation...
				int NumContained =
					(ContainmentFunc(Mesh.GetVertex(Tri.A), UseNormal) ? 1 : 0) +
					(ContainmentFunc(Mesh.GetVertex(Tri.B), UseNormal) ? 1 : 0) +
					(ContainmentFunc(Mesh.GetVertex(Tri.C), UseNormal) ? 1 : 0);

				if ( NumContained >= NumTrianglePoints )
				{
					if (SelectionType == EGeometryScriptMeshSelectionType::Triangles)
					{
						GeoSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
					}
					else
					{
						int32 gid = Mesh.GetTriangleGroup(tid);
						GeoSelection.Selection.Add( FGeoSelectionID::GroupFace(tid, gid).Encoded() );
					}
				}
			}
		});

	}

	SelectionOut.SetSelection(MoveTemp(GeoSelection));
}

static void SelectEdgesWithFilter(const FDynamicMesh3& ReadMesh, FGeometryScriptMeshSelection& SelectionOut, TFunctionRef<bool(int32)> EdgeFilter, bool bExcludeMeshBoundaryEdges = false)
{
	FGeometrySelection GeoSelection;
	GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
	for (int32 EID : ReadMesh.EdgeIndicesItr())
	{
		if (bExcludeMeshBoundaryEdges)
		{
			FIndex2i EdgeT = ReadMesh.GetEdgeT(EID);
			if (EdgeT.B == INDEX_NONE)
			{
				continue;
			}
		}
		if (EdgeFilter(EID))
		{
			ReadMesh.EnumerateTriEdgeIDsFromEdgeID(EID, [&GeoSelection](FMeshTriEdgeID TriEdgeID)
			{
				GeoSelection.Selection.Add(TriEdgeID.Encoded());
			});
		}
	}
	SelectionOut.SetSelection(MoveTemp(GeoSelection));
}
} // namespace UE::MeshSelectionLocals


void UGeometryScriptLibrary_MeshSelectionFunctions::GetMeshSelectionInfo(
	FGeometryScriptMeshSelection Selection,
	EGeometryScriptMeshSelectionType& SelectionType,
	int& NumSelected)
{
	SelectionType = Selection.GetSelectionType();
	NumSelected = Selection.GetNumSelected();
}

void UGeometryScriptLibrary_MeshSelectionFunctions::GetMeshUniqueSelectionInfo(
	const UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	EGeometryScriptMeshSelectionType& SelectionType,
	int& NumSelected
)
{
	SelectionType = Selection.GetSelectionType();
	if (!TargetMesh)
	{
		UE_LOGF(LogGeometry, Warning, "GetMeshUniqueSelectionInfo: TargetMesh is Null");
		NumSelected = Selection.GetNumSelected();
		return;
	}
	else
	{
		TargetMesh->ProcessMesh([&NumSelected, &Selection](const FDynamicMesh3& Mesh)
		{
			NumSelected = Selection.GetNumUniqueSelected(Mesh);
		});
	}
}



void UGeometryScriptLibrary_MeshSelectionFunctions::DebugPrintMeshSelection(
	FGeometryScriptMeshSelection Selection,
	bool bDisable)
{
	if (!bDisable)
	{
		Selection.DebugPrint();
	}
}


UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::CreateSelectAllMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& Selection,
	EGeometryScriptMeshSelectionType SelectionType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "CreateSelectAllMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		FGeometrySelection NewSelection;
		if (SelectionType == EGeometryScriptMeshSelectionType::Vertices)
		{
			NewSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
			for (int32 vid : ReadMesh.VertexIndicesItr())
			{
				NewSelection.Selection.Add(FGeoSelectionID::MeshVertex(vid).Encoded());
			}
		}
		else if (SelectionType == EGeometryScriptMeshSelectionType::Triangles)
		{
			NewSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
			for (int32 tid : ReadMesh.TriangleIndicesItr())
			{
				NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(tid).Encoded());
			}
		}
		else if (SelectionType == EGeometryScriptMeshSelectionType::Edges)
		{
			NewSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
			for (int32 TID : ReadMesh.TriangleIndicesItr())
			{
				NewSelection.Selection.Add(FMeshTriEdgeID(TID, 0).Encoded());
				NewSelection.Selection.Add(FMeshTriEdgeID(TID, 1).Encoded());
				NewSelection.Selection.Add(FMeshTriEdgeID(TID, 2).Encoded());
			}
		}
		else
		{
			NewSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);
			TSet<int32> UniqueGroupIDs;
			for (int32 tid : ReadMesh.TriangleIndicesItr())
			{
				int32 GroupID = ReadMesh.GetTriangleGroup(tid);
				if (UniqueGroupIDs.Contains(GroupID) == false)
				{
					NewSelection.Selection.Add(FGeoSelectionID::GroupFace(tid, GroupID).Encoded());
					UniqueGroupIDs.Add(GroupID);
				}
			}
		}
		Selection.SetSelection(MoveTemp(NewSelection));
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsByMaterialID(UDynamicMesh* TargetMesh,
	int MaterialID,
	FGeometryScriptMeshSelection& Selection,
	EGeometryScriptMeshSelectionType SelectionType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "CreateMeshSelectionByMaterialID: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([MaterialID, &Selection, SelectionType](const FDynamicMesh3& ReadMesh)
	{
		if (!ReadMesh.HasAttributes() || ReadMesh.Attributes()->GetMaterialID() == nullptr)
		{
			UE_LOGF(LogGeometry, Warning, "CreateMeshSelectionByMaterialID: Mesh does not have material IDs");
			return;
		}

		const FDynamicMeshMaterialAttribute* MaterialIDAttr = ReadMesh.Attributes()->GetMaterialID();

		UE::MeshSelectionLocals::SelectByTriangleAttribute(ReadMesh, Selection, SelectionType, [&MaterialIDAttr, MaterialID](int32 TID)
		{
			return MaterialIDAttr->GetValue(TID) == MaterialID;
		});
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsByPolygroup(
	UDynamicMesh* TargetMesh,
	FGeometryScriptGroupLayer GroupLayer,
	UPARAM(DisplayName = "PolyGroup ID") int PolygroupID,
	FGeometryScriptMeshSelection& Selection,
	EGeometryScriptMeshSelectionType SelectionType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "CreateMeshSelectionByPolygroup: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([GroupLayer, PolygroupID, &Selection, SelectionType](const FDynamicMesh3& ReadMesh)
	{
		if (GroupLayer.bDefaultLayer == true)
		{
			UE::MeshSelectionLocals::SelectByTriangleAttribute(ReadMesh, Selection, SelectionType, [&ReadMesh, PolygroupID](int32 TID)
			{
				return ReadMesh.GetTriangleGroup(TID) == PolygroupID;
			});
		}
		else
		{
			if (!ReadMesh.HasAttributes() || ReadMesh.Attributes()->NumPolygroupLayers() <= GroupLayer.ExtendedLayerIndex)
			{
				UE_LOGF(LogGeometry, Warning, "CreateMeshSelectionByPolygroup: Requested Polygroup Layer (%d) not found", GroupLayer.ExtendedLayerIndex);
				return;
			}

			const FDynamicMeshPolygroupAttribute* PolygroupAttr = ReadMesh.Attributes()->GetPolygroupLayer(GroupLayer.ExtendedLayerIndex);
			UE::MeshSelectionLocals::SelectByTriangleAttribute(ReadMesh, Selection, SelectionType, [&PolygroupAttr, PolygroupID](int32 TID)
			{
				return PolygroupAttr->GetValue(TID) == PolygroupID;
			});
		}
	});
	return TargetMesh;
}

void UGeometryScriptLibrary_MeshSelectionFunctions::CombineMeshSelections(
	FGeometryScriptMeshSelection SelectionA,
	FGeometryScriptMeshSelection SelectionB,
	FGeometryScriptMeshSelection& ResultSelectionOut,
	EGeometryScriptCombineSelectionMode CombineMode)
{
	if (SelectionA.GetSelectionType() != SelectionB.GetSelectionType())
	{
		UE_LOGF(LogGeometry, Warning, "CombineMeshSelections: Selections have different types, cannot combine");
	}
	ResultSelectionOut.SetSelection(SelectionA);
	ResultSelectionOut.CombineSelectionInPlace(SelectionB, CombineMode);
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection FromSelection,
	FGeometryScriptMeshSelection& ToSelection,
	EGeometryScriptMeshSelectionType NewType,
	bool bAllowPartialInclusion)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}
	if (FromSelection.GetSelectionType() == NewType)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMeshSelection: Trying to convert to same type");
		ToSelection.SetSelection(FromSelection);
		return TargetMesh;
	}

	if (NewType == EGeometryScriptMeshSelectionType::Vertices)
	{
		TargetMesh->ProcessMesh([&FromSelection, &ToSelection, bAllowPartialInclusion](const FDynamicMesh3& ReadMesh)
		{
			TSet<int32> CurElements, CurVertices;
			
			FGeometrySelection NewSelection;
			NewSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);

			if (bAllowPartialInclusion)
			{
				FromSelection.ProcessByVertexID(ReadMesh, [&NewSelection](int32 VertexID) { 
					NewSelection.Selection.Add(FGeoSelectionID::MeshVertex(VertexID).Encoded());
				});
			}
			else if(FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles
				|| FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Polygroups)
			{
				FromSelection.ProcessByTriangleID(ReadMesh, [&CurElements, &CurVertices, &ReadMesh](int32 TriangleID)
				{
					CurElements.Add(TriangleID);
					FIndex3i Vertices = ReadMesh.GetTriangle(TriangleID);
					CurVertices.Add(Vertices.A); CurVertices.Add(Vertices.B); CurVertices.Add(Vertices.C);
				});

				for (int32 VID : CurVertices)
				{
					bool bAllInSet = true;
					ReadMesh.EnumerateVertexTriangles(VID, [&bAllInSet, &CurElements](int32 TID) { bAllInSet = bAllInSet && CurElements.Contains(TID); });
					if (bAllInSet)
					{
						NewSelection.Selection.Add(FGeoSelectionID::MeshVertex(VID).Encoded());
					}
				}
			}
			else if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
			{
				FromSelection.ProcessByEdgeID(ReadMesh, [&CurElements, &CurVertices, &ReadMesh](int32 EdgeID)
				{
					CurElements.Add(EdgeID);
					FIndex2i Vertices = ReadMesh.GetEdgeV(EdgeID);
					CurVertices.Add(Vertices.A); CurVertices.Add(Vertices.B);
				});

				for (int32 VID : CurVertices)
				{
					bool bAllInSet = true;
					ReadMesh.EnumerateVertexEdges(VID, [&bAllInSet, &CurElements](int32 EID) { bAllInSet = bAllInSet && CurElements.Contains(EID); });
					if (bAllInSet)
					{
						NewSelection.Selection.Add(FGeoSelectionID::MeshVertex(VID).Encoded());
					}
				}
			}
			else
			{
				ensureMsgf(false, TEXT("Unhandled mesh selection type"));
			}

			ToSelection.SetSelection(MoveTemp(NewSelection));
		});

	}
	else if (NewType == EGeometryScriptMeshSelectionType::Edges)
	{
		FGeometrySelection NewSelection;
		NewSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);

		if (bAllowPartialInclusion)
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				FromSelection.ProcessByEdgeID(ReadMesh, [&ReadMesh, &NewSelection](int32 EdgeID)
				{
					ReadMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&NewSelection](FMeshTriEdgeID TriEdgeID)
					{
						NewSelection.Selection.Add(TriEdgeID.Encoded());
					});
				});
			});
		}
		else if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices) // select edges w/ both verts selected
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				TSet<int32> CurVertices;
				FromSelection.ProcessByVertexID(ReadMesh, [&CurVertices](int32 VertexID)
				{
					CurVertices.Add(VertexID);
				});
				FromSelection.ProcessByEdgeID(ReadMesh, [&CurVertices, &ReadMesh, &NewSelection](int32 EdgeID)
				{
					FIndex2i EdgeV = ReadMesh.GetEdgeV(EdgeID);
					if (CurVertices.Contains(EdgeV.A) && CurVertices.Contains(EdgeV.B))
					{
						ReadMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&NewSelection](FMeshTriEdgeID TriEdgeID)
						{
							NewSelection.Selection.Add(TriEdgeID.Encoded());
						});
					}
				});
			});
		}
		// select edges w/ all tris selected
		else if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles
			|| FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Polygroups)
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				TSet<int32> CurTriangles;
				FromSelection.ProcessByTriangleID(ReadMesh, [&CurTriangles](int32 TriangleID)
				{
					CurTriangles.Add(TriangleID);
				});
				FromSelection.ProcessByEdgeID(ReadMesh, [&CurTriangles, &ReadMesh, &NewSelection](int32 EdgeID)
				{
					FIndex2i EdgeT = ReadMesh.GetEdgeT(EdgeID);
					if (CurTriangles.Contains(EdgeT.A) && (EdgeT.B == FDynamicMesh3::InvalidID || CurTriangles.Contains(EdgeT.B)))
					{
						ReadMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&NewSelection](FMeshTriEdgeID TriEdgeID)
						{
							NewSelection.Selection.Add(TriEdgeID.Encoded());
						});
					}
				});
			});
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled mesh selection type"));
		}
		
		ToSelection.SetSelection(MoveTemp(NewSelection));
	}
	else if (NewType == EGeometryScriptMeshSelectionType::Triangles)
	{
		FGeometrySelection NewSelection;
		NewSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);

		if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Polygroups || bAllowPartialInclusion)
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				FromSelection.ProcessByTriangleID(ReadMesh, [&NewSelection](int32 TriangleID) {
					NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
				});
			});
		}
		else if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices) // vertex selection w/ no partial inclusion, ie only "full" triangles
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				// in this case FromSelection already has this set! but we do not have access to it...
				TSet<int32> CurVertices;
				FromSelection.ProcessByVertexID(ReadMesh, [&CurVertices](int32 VertexID) {
					CurVertices.Add(VertexID);
				});
				FromSelection.ProcessByTriangleID(ReadMesh, [&CurVertices, &NewSelection, &ReadMesh](int32 TriangleID) {
					FIndex3i Triangle = ReadMesh.GetTriangle(TriangleID);
					if (CurVertices.Contains(Triangle.A) && CurVertices.Contains(Triangle.B) && CurVertices.Contains(Triangle.C))
					{
						NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
					}
				});
			});
		}
		else if (FromSelection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
		{
			TargetMesh->ProcessMesh([&FromSelection, &NewSelection](const FDynamicMesh3& ReadMesh)
			{
				TSet<int32> CurEdges;
				FromSelection.ProcessByEdgeID(ReadMesh, [&CurEdges](int32 EdgeID) {
					CurEdges.Add(EdgeID);
				});
				FromSelection.ProcessByTriangleID(ReadMesh, [&ReadMesh, &CurEdges, &NewSelection](int32 TriangleID) {
					FIndex3i TriangleEdges = ReadMesh.GetTriEdges(TriangleID);
					if (CurEdges.Contains(TriangleEdges.A) && CurEdges.Contains(TriangleEdges.B) && CurEdges.Contains(TriangleEdges.C))
					{
						NewSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
					}
				});
			});
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled mesh selection type"));
		}

		ToSelection.SetSelection(MoveTemp(NewSelection));
	}
	else   // polygroups
	{
		if (bAllowPartialInclusion)
		{
			TSet<int32> UniqueGroupIDs;
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				FromSelection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) {
					UniqueGroupIDs.Add(ReadMesh.GetTriangleGroup(TriangleID));
				});
			});
			ConvertIndexSetToMeshSelection(TargetMesh, UniqueGroupIDs, EGeometryScriptMeshSelectionType::Polygroups, ToSelection);
		}
		else
		{
			TSet<int32> UniqueGroupIDs;
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				// Note: for vertex and edge selections, will include all 'touched' triangles
				// This is less strict than one might expect.
				// If the stricter selection conversion is desired, please consider how to do so without changing existing BP behavior.
				TSet<int32> AllTriangles;
				FromSelection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) {
					AllTriangles.Add(TriangleID);
					UniqueGroupIDs.Add(ReadMesh.GetTriangleGroup(TriangleID));
				});

				// if we have a non-selected triangle whose group is in our group set, that group is not fully selected
				TSet<int32> FailGroups;
				for (int32 TriangleID : ReadMesh.TriangleIndicesItr())
				{
					int32 GroupID = ReadMesh.GetTriangleGroup(TriangleID);
					if (UniqueGroupIDs.Contains(GroupID) && AllTriangles.Contains(TriangleID) == false)
					{
						FailGroups.Add(GroupID);
					}
				}
				for (int32 GroupID : FailGroups)
				{
					UniqueGroupIDs.Remove(GroupID);
				}
			});
			ConvertIndexSetToMeshSelection(TargetMesh, UniqueGroupIDs, EGeometryScriptMeshSelectionType::Polygroups, ToSelection);
		}
	}


	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertIndexArrayToMeshSelection(
	UDynamicMesh* TargetMesh,
	const TArray<int32>& IndexArray,
	EGeometryScriptMeshSelectionType SelectionType,
	FGeometryScriptMeshSelection& SelectionOut)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertIndexArrayToMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}

	FGeometrySelection GeoSelection;

	switch (SelectionType)
	{
	case EGeometryScriptMeshSelectionType::Triangles:
		for (int32 tid : IndexArray)
		{
			GeoSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
		break;

	case EGeometryScriptMeshSelectionType::Vertices:
		for (int32 vid : IndexArray)
		{
			GeoSelection.Selection.Add( FGeoSelectionID::MeshVertex(vid).Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
		break;

	case EGeometryScriptMeshSelectionType::Edges:
		GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
		TargetMesh->ProcessMesh([&GeoSelection, &IndexArray](const FDynamicMesh3& Mesh)
		{
			for (int32 EID : IndexArray)
			{
				Mesh.EnumerateTriEdgeIDsFromEdgeID(EID, [&GeoSelection](FMeshTriEdgeID TriEdgeID)
				{
					GeoSelection.Selection.Add(TriEdgeID.Encoded());
				});
			}
		});
		break;

	case EGeometryScriptMeshSelectionType::Polygroups:
		for (int32 gid : IndexArray)
		{
			FGeoSelectionID HackGroupFace;
			HackGroupFace.TopologyID = (uint32)gid;
			HackGroupFace.GeometryID = 0xFFFFFFFFu;
			GeoSelection.Selection.Add( HackGroupFace.Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);
		break;

	default:
		ensureMsgf(false, TEXT("Unhandled mesh selection type"));

	}

	SelectionOut.SetSelection(MoveTemp(GeoSelection));
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertIndexSetToMeshSelection(
	UDynamicMesh* TargetMesh,
	const TSet<int32>& IndexSet,
	EGeometryScriptMeshSelectionType SelectionType,
	FGeometryScriptMeshSelection& SelectionOut)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertIndexSetToMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}

	FGeometrySelection GeoSelection;

	switch (SelectionType)
	{
	case EGeometryScriptMeshSelectionType::Triangles:
		for (int32 tid : IndexSet)
		{
			GeoSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
		break;

	case EGeometryScriptMeshSelectionType::Vertices:
		for (int32 vid : IndexSet)
		{
			GeoSelection.Selection.Add( FGeoSelectionID::MeshVertex(vid).Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
		break;

	case EGeometryScriptMeshSelectionType::Edges:
		GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
		TargetMesh->ProcessMesh([&GeoSelection, &IndexSet](const FDynamicMesh3& Mesh)
		{
			for (int32 EID : IndexSet)
			{
				Mesh.EnumerateTriEdgeIDsFromEdgeID(EID, [&GeoSelection](FMeshTriEdgeID TriEdgeID)
				{
					GeoSelection.Selection.Add(TriEdgeID.Encoded());
				});
			}
		});
		break;

	case EGeometryScriptMeshSelectionType::Polygroups:
		for (int32 gid : IndexSet)
		{
			FGeoSelectionID HackGroupFace;
			HackGroupFace.TopologyID = (uint32)gid;
			HackGroupFace.GeometryID = 0xFFFFFFFFu;
			GeoSelection.Selection.Add( HackGroupFace.Encoded() );
		}
		GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);

		break;

	default:
		ensureMsgf(false, TEXT("Unhandled mesh selection type"));
	}

	SelectionOut.SetSelection(MoveTemp(GeoSelection));
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertMeshSelectionToIndexArray(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	TArray<int32>& IndexArray,
	EGeometryScriptMeshSelectionType& SelectionType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMeshSelectionToIndexArray: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		Selection.ConvertToMeshIndexArray(Mesh, IndexArray);
		SelectionType = Selection.GetSelectionType();
	});

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertIndexListToMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptIndexList IndexList,
	EGeometryScriptMeshSelectionType SelectionType,
	FGeometryScriptMeshSelection& SelectionOut)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertIndexListToMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}
	if (IndexList.IndexType == EGeometryScriptIndexType::Any)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertIndexListToMeshSelection: IndexList has type Any, cannot convert");
		return TargetMesh;
	}

	EGeometryScriptMeshSelectionType InitialType;
	if (FGeometryScriptMeshSelection::ConvertIndexTypeToSelectionType(IndexList.IndexType, InitialType))
	{
		if (SelectionType == InitialType)
		{
			ConvertIndexArrayToMeshSelection(TargetMesh, *IndexList.List, InitialType, SelectionOut);
		}
		else
		{
			FGeometryScriptMeshSelection TempSelection;
			ConvertIndexArrayToMeshSelection(TargetMesh, *IndexList.List, InitialType, TempSelection);
			ConvertMeshSelection(TargetMesh, TempSelection, SelectionOut, SelectionType, true);
		}
	}
	else if (IndexList.IndexType == EGeometryScriptIndexType::MaterialID)
	{
		TArray<int32> Triangles;
		const TArray<int32>& MaterialIDSelection = *IndexList.List;
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->HasMaterialID())
			{
				const FDynamicMeshMaterialAttribute* MaterialIDs = ReadMesh.Attributes()->GetMaterialID();
				for (int32 tid : ReadMesh.TriangleIndicesItr())
				{
					if (MaterialIDSelection.Contains(MaterialIDs->GetValue(tid)))
					{
						Triangles.Add(tid);
					}
				}
			}
		});

		if (SelectionType == EGeometryScriptMeshSelectionType::Triangles)
		{
			ConvertIndexArrayToMeshSelection(TargetMesh, Triangles, EGeometryScriptMeshSelectionType::Triangles, SelectionOut);
		}
		else
		{
			FGeometryScriptMeshSelection TempSelection;
			ConvertIndexArrayToMeshSelection(TargetMesh, Triangles, EGeometryScriptMeshSelectionType::Triangles, TempSelection);
			ConvertMeshSelection(TargetMesh, TempSelection, SelectionOut, SelectionType, true);
		}
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ConvertMeshSelectionToIndexList(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptIndexList& IndexList,
	EGeometryScriptIndexType& ResultType,
	EGeometryScriptIndexType ConvertToType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMeshSelectionToIndexList: TargetMesh is Null");
		return TargetMesh;
	}

	ResultType = EGeometryScriptIndexType::Any;
	TArray<int32> TempArray;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		ResultType = Selection.ConvertToMeshIndexArray(Mesh, TempArray, ConvertToType);
	});

	IndexList.Reset(ResultType);
	*IndexList.List = MoveTemp(TempArray);

	if (ResultType == EGeometryScriptIndexType::Any)
	{
		UE_LOGF(LogGeometry, Warning, "ConvertMeshSelectionToIndexList: Conversion is not currently supported");
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsInBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	FBox Box,
	EGeometryScriptMeshSelectionType SelectionType,
	bool bInvert,
	int MinNumTrianglePoints)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsInBox: TargetMesh is Null");
		return TargetMesh;
	}

	FAxisAlignedBox3d Container(Box);
	auto ContainsFunc = [&Container, bInvert](const FVector3d& Point, const FVector3d& Normal) 
	{ 
		return Container.Contains(Point) != bInvert; 
	};
	UE::MeshSelectionLocals::SelectMeshElementsWithContainmentTest(TargetMesh, ContainsFunc,
		SelectionOut, SelectionType, MinNumTrianglePoints, false);
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsInSphere(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	FVector SphereOrigin,
	double SphereRadius,
	EGeometryScriptMeshSelectionType SelectionType,
	bool bInvert,
	int MinNumTrianglePoints)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsInSphere: TargetMesh is Null");
		return TargetMesh;
	}

	SphereRadius = FMathd::Clamp(SphereRadius, FMathd::Epsilon, FMathd::Sqrt(TNumericLimits<double>::Max()));
	UE::Geometry::FSphere3d Container(SphereOrigin, SphereRadius);

	auto ContainsFunc = [&Container, bInvert](const FVector3d& Point, const FVector3d& Normal) 
	{ 
		return Container.Contains(Point) != bInvert; 
	};
	UE::MeshSelectionLocals::SelectMeshElementsWithContainmentTest(TargetMesh, ContainsFunc,
		SelectionOut, SelectionType, MinNumTrianglePoints, false);
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsWithPlane(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	FVector PlaneOrigin,
	FVector PlaneNormal,
	EGeometryScriptMeshSelectionType SelectionType,
	bool bInvert,
	int MinNumTrianglePoints)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsWithPlane: TargetMesh is Null");
		return TargetMesh;
	}

	PlaneNormal = Normalized(PlaneNormal);
	auto ContainsFunc = [&PlaneOrigin, &PlaneNormal, bInvert](const FVector3d& Point, const FVector3d& Normal) 
	{ 
		bool bContains = (Point - PlaneOrigin).Dot(PlaneNormal) >= 0;
		return bContains != bInvert;
	};
	UE::MeshSelectionLocals::SelectMeshElementsWithContainmentTest(TargetMesh, ContainsFunc,
		SelectionOut, SelectionType, MinNumTrianglePoints, false);
	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsByNormalAngle(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	FVector PlaneNormal,
	double MaxAngleDeg,
	EGeometryScriptMeshSelectionType SelectionType,
	bool bInvert,
	int MinNumTrianglePoints)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsByNormalAngle: TargetMesh is Null");
		return TargetMesh;
	}

	PlaneNormal = Normalized(PlaneNormal);
	double CosMaxAngle = FMathd::Cos(MaxAngleDeg * FMathd::DegToRad);
	auto ContainsFunc = [&PlaneNormal, &CosMaxAngle, bInvert](const FVector3d& Point, const FVector3d& Normal) 
	{
		bool bContains = PlaneNormal.Dot(Normal) >= CosMaxAngle;
		return bContains != bInvert;
	};
	UE::MeshSelectionLocals::SelectMeshElementsWithContainmentTest(TargetMesh, ContainsFunc,
		SelectionOut, SelectionType, MinNumTrianglePoints, true);
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshSharpEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	double MinAngleDeg)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshSharpEdges: TargetMesh is Null");
		return TargetMesh;
	}

	double CosThresh = FMath::Cos(FMathd::DegToRad * MinAngleDeg);
	TargetMesh->ProcessMesh([&SelectionOut, CosThresh](const FDynamicMesh3& Mesh)
	{
		UE::MeshSelectionLocals::SelectEdgesWithFilter(Mesh, SelectionOut, [CosThresh, &Mesh](int32 EID)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(EID);
			if (EdgeT.B == INDEX_NONE)
			{
				return false;
			}
			FVector3d NormalA = Mesh.GetTriNormal(EdgeT.A);
			FVector3d NormalB = Mesh.GetTriNormal(EdgeT.B);
			return NormalA.Dot(NormalB) <= CosThresh;
		});
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshSplitNormalEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshSplitNormalEdges: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&SelectionOut](const FDynamicMesh3& Mesh)
	{
		if (!Mesh.HasAttributes() || !Mesh.Attributes()->PrimaryNormals())
		{
			UE_LOGF(LogGeometry, Warning, "SelectMeshSplitNormalEdges: TargetMesh does has no normals attribute");
			return;
		}
		const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
		UE::MeshSelectionLocals::SelectEdgesWithFilter(Mesh, SelectionOut, [&Mesh, Normals](int32 EID)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(EID);
			// boundary edges are not split normal edges, but the overlay will consider them as seams, so handle them here
			if (EdgeT.B == INDEX_NONE)
			{
				return false;
			}
			return Normals->IsSeamEdge(EID);
		});
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshBoundaryEdges(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection& SelectionOut)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshBoundaryEdges: TargetMesh is Null");
		return TargetMesh;
	}

	FGeometrySelection GeoSelection;
	GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
	TargetMesh->ProcessMesh([&GeoSelection](const FDynamicMesh3& Mesh)
	{
		for (int32 EID : Mesh.EdgeIndicesItr())
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(EID);
			if (EdgeT.B == INDEX_NONE)
			{
				GeoSelection.Selection.Add(Mesh.GetTriEdgeIDFromEdgeID(EID).Encoded());
			}
		}
	});
	SelectionOut.SetSelection(MoveTemp(GeoSelection));
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectSelectionBoundaryEdges(
	UDynamicMesh* TargetMesh,
	const FGeometryScriptMeshSelection& RegionSelection,
	FGeometryScriptMeshSelection& SelectionOut,
	bool bExcludeMeshBoundaryEdges
)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectSelectionBoundaryEdges: TargetMesh is Null");
		return TargetMesh;
	}

	FGeometrySelection GeoSelection;
	GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
	TargetMesh->ProcessMesh([&GeoSelection, &RegionSelection, bExcludeMeshBoundaryEdges](const FDynamicMesh3& Mesh)
	{
		TSet<int32> TriSel;
		RegionSelection.ProcessByTriangleID(Mesh, [&TriSel](int32 TID) { TriSel.Add(TID); });
		for (int32 TID : TriSel)
		{
			FIndex3i TriEdges = Mesh.GetTriEdges(TID);
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(TID);
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				if (NbrTris[SubIdx] == INDEX_NONE)
				{
					if (!bExcludeMeshBoundaryEdges)
					{
						GeoSelection.Selection.Add(Mesh.GetTriEdgeIDFromEdgeID(TriEdges[SubIdx]).Encoded());
					}
				}
				else if (!TriSel.Contains(NbrTris[SubIdx]))
				{
					Mesh.EnumerateTriEdgeIDsFromEdgeID(TriEdges[SubIdx], [&GeoSelection](FMeshTriEdgeID TriEdgeID)
					{
						GeoSelection.Selection.Add(TriEdgeID.Encoded());
					});
				}
			}
		}
	});
	SelectionOut.SetSelection(MoveTemp(GeoSelection));
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshUVSeamEdges(
	UDynamicMesh* TargetMesh, FGeometryScriptMeshSelection& SelectionOut,
	int32 UVChannel, bool& bHaveValidUVs,
	bool bExcludeMeshBoundaryEdges
)
{
	bHaveValidUVs = false;
	SelectionOut.ClearSelection();
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshUVSeamEdges: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&SelectionOut, UVChannel, &bHaveValidUVs, bExcludeMeshBoundaryEdges](const FDynamicMesh3& Mesh)
	{
		if (!Mesh.HasAttributes() || !Mesh.Attributes()->GetUVLayer(UVChannel))
		{
			return;
		}
		bHaveValidUVs = true;
		const FDynamicMeshUVOverlay* UVLayer = Mesh.Attributes()->GetUVLayer(UVChannel);
		UE::MeshSelectionLocals::SelectEdgesWithFilter(Mesh, SelectionOut, [UVLayer](int32 EID)
		{
			return UVLayer->IsSeamEdge(EID);
		}, bExcludeMeshBoundaryEdges);
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshPolyGroupBoundaryEdges(
	UDynamicMesh* TargetMesh, FGeometryScriptMeshSelection& SelectionOut,
	FGeometryScriptGroupLayer GroupLayer,
	bool bExcludeMeshBoundaryEdges
)
{
	SelectionOut.ClearSelection();
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshPolyGroupBoundaryEdges: TargetMesh is Null");
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&SelectionOut, GroupLayer, bExcludeMeshBoundaryEdges](const FDynamicMesh3& Mesh)
	{
		if (GroupLayer.bDefaultLayer == true)
		{
			if (!Mesh.HasTriangleGroups())
			{
				UE_LOGF(LogGeometry, Warning, "SelectMeshPolyGroupBoundaryEdges: Mesh does not have PolyGroups enabled");
				return;
			}
			UE::MeshSelectionLocals::SelectEdgesWithFilter(Mesh, SelectionOut, [&Mesh](int32 EID)
			{
				FIndex2i EdgeT = Mesh.GetEdgeT(EID);
				return EdgeT.B == INDEX_NONE || Mesh.GetTriangleGroup(EdgeT.A) != Mesh.GetTriangleGroup(EdgeT.B);
			}, bExcludeMeshBoundaryEdges);
		}
		else
		{
			if (!Mesh.HasAttributes() || Mesh.Attributes()->NumPolygroupLayers() <= GroupLayer.ExtendedLayerIndex)
			{
				UE_LOGF(LogGeometry, Warning, "SelectMeshPolyGroupBoundaryEdges: Requested Polygroup Layer (%d) not found", GroupLayer.ExtendedLayerIndex);
				return;
			}

			const FDynamicMeshPolygroupAttribute* PolygroupAttr = Mesh.Attributes()->GetPolygroupLayer(GroupLayer.ExtendedLayerIndex);
			UE::MeshSelectionLocals::SelectEdgesWithFilter(Mesh, SelectionOut, [&Mesh, PolygroupAttr](int32 EID)
			{
				FIndex2i EdgeT = Mesh.GetEdgeT(EID);
				return EdgeT.B == INDEX_NONE || PolygroupAttr->GetValue(EdgeT.A) != PolygroupAttr->GetValue(EdgeT.B);
			}, bExcludeMeshBoundaryEdges);
		}
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsInsideMesh(
	UDynamicMesh* TargetMesh,
	UDynamicMesh* SelectionMesh,
	FGeometryScriptMeshSelection& SelectionOut,
	FTransform SelectionMeshTransform,
	EGeometryScriptMeshSelectionType SelectionType,
	bool bInvert,
	double ShellDistance,
	double WindingThreshold,
	/** Only include triangles with this many vertices inside the Selection Mesh */
	int MinNumTrianglePoints)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsInsideMesh: TargetMesh is Null");
		return TargetMesh;
	}
	if (SelectionMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsInsideMesh: SelectionMesh is Null");
		return TargetMesh;
	}
	if (SelectionMesh == TargetMesh)
	{
		UE_LOGF(LogGeometry, Warning, "SelectMeshElementsInsideMesh: SelectionMesh == TargetMesh, this is not supported");
		// TODO: could select-all here?
		return TargetMesh;
	}

	// todo: for small meshes it is possibly cheaper to make a copy?
	FTransform InvTransform = SelectionMeshTransform.Inverse();

	SelectionMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		FDynamicMeshAABBTree3 Spatial(&ReadMesh, true);
		TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial, true);

		auto ContainsFunc = [&Spatial, &FastWinding, InvTransform, WindingThreshold, ShellDistance, bInvert](const FVector3d& Point, const FVector3d& Normal)
		{
			FVector3d LocalPoint = InvTransform.TransformPosition(Point);
			bool bContains = FastWinding.IsInside(LocalPoint, WindingThreshold);
			if (!bContains && ShellDistance > 0)
			{
				double NearestDistSqr;
				int32 NearestTID = Spatial.FindNearestTriangle(LocalPoint, NearestDistSqr, IMeshSpatial::FQueryOptions(ShellDistance));
				if (NearestTID >= 0 && NearestDistSqr < ShellDistance * ShellDistance)
				{
					bContains = true;
				}
			}
			return bContains != bInvert;
		};
		UE::MeshSelectionLocals::SelectMeshElementsWithContainmentTest(TargetMesh, ContainsFunc,
			SelectionOut, SelectionType, MinNumTrianglePoints, true);
	});

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::InvertMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptMeshSelection& NewSelection,
	bool bOnlyToConnected)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "InvertMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}

	if (bOnlyToConnected)
	{
		ExpandMeshSelectionToConnected(TargetMesh, Selection, NewSelection, EGeometryScriptTopologyConnectionType::Geometric);
	}
	else
	{
		CreateSelectAllMeshSelection(TargetMesh, NewSelection, Selection.GetSelectionType());
	}
	NewSelection.CombineSelectionInPlace(Selection, EGeometryScriptCombineSelectionMode::Subtract);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ExpandMeshSelectionToConnected(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptMeshSelection& NewSelection,
	EGeometryScriptTopologyConnectionType ConnectionType)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ExpandMeshSelectionToConnected: TargetMesh is Null");
		return TargetMesh;
	}
	if (Selection.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "ExpandMeshSelectionToConnected: Initial Selection is Empty");
		return TargetMesh;
	}
	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
	{
		// todo: this...
		UE_LOGF(LogGeometry, Warning, "ExpandMeshSelectionToConnected: Vertex Selection currently not supported");
		NewSelection.SetSelection(Selection);
		return TargetMesh;
	}
	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
	{
		// todo: implement this
		UE_LOGF(LogGeometry, Warning, "ExpandMeshSelectionToConnected: Edge Selection currently not supported");
		NewSelection.SetSelection(Selection);
		return TargetMesh;
	}

	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Polygroups &&
		ConnectionType == EGeometryScriptTopologyConnectionType::Polygroup)
	{
		UE_LOGF(LogGeometry, Warning, "ExpandMeshSelectionToConnected: Expanding Polygroup Selection to Connected Polygroups will not change selection");
		NewSelection.SetSelection(Selection);	// this setup makes no sense
		return TargetMesh;
	}

	// collect up existing triangles and (optionally) polygroups
	TArray<int32> CurTriangles;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		Selection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) { CurTriangles.Add(TriangleID); });
	});

	TSet<int32> ResultTriangles;
	if (ConnectionType == EGeometryScriptTopologyConnectionType::Geometric)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			FMeshConnectedComponents::GrowToConnectedTriangles(&ReadMesh, CurTriangles, ResultTriangles, nullptr, [](int32, int32) { return true; });
		});
	}
	else if (ConnectionType == EGeometryScriptTopologyConnectionType::Polygroup)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			FMeshConnectedComponents::GrowToConnectedTriangles(&ReadMesh, CurTriangles, ResultTriangles, nullptr,
				[&](int32 FromTriID, int32 ToTriID) {
					return ReadMesh.GetTriangleGroup(FromTriID) == ReadMesh.GetTriangleGroup(ToTriID);
			});
		});
	}
	else if (ConnectionType == EGeometryScriptTopologyConnectionType::MaterialID)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->GetMaterialID() != nullptr)
			{
				const FDynamicMeshMaterialAttribute* MaterialID = ReadMesh.Attributes()->GetMaterialID();
				FMeshConnectedComponents::GrowToConnectedTriangles(&ReadMesh, CurTriangles, ResultTriangles, nullptr,
					[&](int32 FromTriID, int32 ToTriID) {
					return MaterialID->GetValue(FromTriID) == MaterialID->GetValue(ToTriID);
				});
			}
		});
	}

	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles)
	{
		ConvertIndexSetToMeshSelection(TargetMesh, ResultTriangles, EGeometryScriptMeshSelectionType::Triangles, NewSelection);
	}
	else
	{
		TSet<int32> ResultGroupIDs;
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			for (int32 tid : ResultTriangles)
			{
				ResultGroupIDs.Add(ReadMesh.GetTriangleGroup(tid));
			}
		});
		ConvertIndexSetToMeshSelection(TargetMesh, ResultGroupIDs, EGeometryScriptMeshSelectionType::Polygroups, NewSelection);
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSelectionFunctions::ExpandContractMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptMeshSelection& NewSelection,
	int32 Iterations,
	bool bContract,
	bool bOnlyExpandToFaceNeighbours)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "ExpandContractMeshSelection: TargetMesh is Null");
		return TargetMesh;
	}
	if (Selection.IsEmpty())
	{
		UE_LOGF(LogGeometry, Warning, "ExpandContractMeshSelection: Initial Selection is Empty");
		return TargetMesh;
	}
	if (Iterations <= 0)
	{
		NewSelection.SetSelection(Selection);
		return TargetMesh;
	}
	Iterations = FMath::Clamp(Iterations, 1, 100);

	// TODO: when doing multiple iterations w/ polygroups, we cannot rely on the code below because it is only
	// expanding/contracting by triangle rings. Need to expand to polygroups at each step which is currently not easy
	// to do with a FMeshFaceSelection, need to convert to FMeshConnectedComponents/etc. So for now we will just
	// recursively do it this way which is expensive...
	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Polygroups && Iterations > 1)
	{
		FGeometryScriptMeshSelection CurSelection = Selection;
		FGeometryScriptMeshSelection NextSelection;
		for (int32 k = 0; k < Iterations; ++k)
		{
			ExpandContractMeshSelection(TargetMesh, CurSelection, NextSelection, 1, bContract, bOnlyExpandToFaceNeighbours);
			CurSelection = NextSelection;
		}
		NewSelection = CurSelection;
		return TargetMesh;
	}


	FGeometrySelection NewGeoSelection;
	if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
	{
		NewGeoSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			FMeshVertexSelection VtxSelection(&ReadMesh);
			Selection.ProcessByVertexID(ReadMesh, [&](int32 VertexID) { VtxSelection.Select(VertexID); } );
			if (bContract)
			{
				VtxSelection.ContractByBorderVertices(Iterations);
			}
			else
			{
				VtxSelection.ExpandToOneRingNeighbours(Iterations);
			}
			for (int32 VertexID : VtxSelection)
			{
				NewGeoSelection.Selection.Add(FGeoSelectionID::MeshVertex(VertexID).Encoded());
			}
		});
	}
	else if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
	{
		NewGeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
		
		TargetMesh->ProcessMesh([&Selection, &NewGeoSelection, bContract, Iterations](const FDynamicMesh3& ReadMesh)
		{
			FMeshEdgeSelection EdgeSelection(&ReadMesh);
			Selection.ProcessByEdgeID(ReadMesh, [&EdgeSelection](int32 EdgeID) { EdgeSelection.Select(EdgeID); });
			if (bContract)
			{
				EdgeSelection.ContractByBorderEdges(Iterations);
			}
			else
			{
				EdgeSelection.ExpandToOneRingNeighbors(Iterations);
			}
			for (int32 EdgeID : EdgeSelection)
			{
				ReadMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&NewGeoSelection](FMeshTriEdgeID TriEdgeID)
				{
					NewGeoSelection.Selection.Add(TriEdgeID.Encoded());
				});
			}
		});
	}
	else 
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			FMeshFaceSelection TriSelection(&ReadMesh);
			Selection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) { TriSelection.Select(TriangleID); });
			if (bContract)
			{
				TriSelection.ContractBorderByOneRingNeighbours(Iterations, true);
			}
			else
			{
				if (bOnlyExpandToFaceNeighbours)
				{
					TriSelection.ExpandToFaceNeighbours(Iterations);
				}
				else
				{
					TriSelection.ExpandToOneRingNeighbours(Iterations);
				}
			}
			if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Triangles)
			{
				NewGeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
				for (int32 TriangleID : TriSelection)
				{
					NewGeoSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
				}
			}
			else
			{
				NewGeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);
				TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
				{
					TSet<int32> UniqueGroupIDs;
					for (int32 tid : TriSelection)
					{
						int32 GroupID = ReadMesh.GetTriangleGroup(tid);
						if (UniqueGroupIDs.Contains(GroupID) == false)
						{
							NewGeoSelection.Selection.Add(FGeoSelectionID::GroupFace(tid, GroupID).Encoded());
						}
					}
				});

			}
		});
	}

	NewSelection.SetSelection(MoveTemp(NewGeoSelection));

	return TargetMesh;
}





#undef LOCTEXT_NAMESPACE
