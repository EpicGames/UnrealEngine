// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSelectionQueryFunctions.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "Selections/GeometrySelection.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "BoxTypes.h"
#include "Selections/MeshConnectedComponents.h"
#include "MeshRegionBoundaryLoops.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSelectionQueryFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSelectionQueryFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshSelectionQueryFunctions::GetMeshSelectionBoundingBox(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FBox& SelectionBounds,
	bool& bIsEmpty,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshSelectionBoundingBox_InvalidInput", "GetMeshSelectionBoundingBox: TargetMesh is Null"));
		return TargetMesh;
	}


	bIsEmpty = true;
	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices)
		{
			Selection.ProcessByVertexID(ReadMesh, [&](int32 VertexID)
			{
				Bounds.Contain(ReadMesh.GetVertex(VertexID));
			});
		}
		else if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
		{
			Selection.ProcessByEdgeID(ReadMesh, [&Bounds, &ReadMesh](int32 EdgeID)
			{
				FIndex2i EdgeV = ReadMesh.GetEdgeV(EdgeID);
				Bounds.Contain(ReadMesh.GetVertex(EdgeV.A));
				Bounds.Contain(ReadMesh.GetVertex(EdgeV.B));
			});
		}
		else // EGeometryScriptMeshSelectionType::Triangles or EGeometryScriptMeshSelectionType::Polygroups
		{
			Selection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) {
				Bounds.Contain(ReadMesh.GetTriBounds(TriangleID));
			});
		}
	});
	
	if (Bounds != FAxisAlignedBox3d::Empty())
	{
		bIsEmpty = false;
		SelectionBounds = (FBox)Bounds;
	}
	else
	{
		SelectionBounds = FBox(EForceInit::ForceInitToZero);
	}
	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshSelectionQueryFunctions::GetMeshSelectionBoundaryLoops(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	TArray<FGeometryScriptIndexList>& IndexLoops,
	TArray<FGeometryScriptPolyPath>& PathLoops,
	int& NumLoops,
	bool& bFoundErrors,
	UGeometryScriptDebug* Debug)
{
	NumLoops = 0;
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshSelectionBoundaryLoops_InvalidInput", "GetMeshSelectionBoundaryLoops: TargetMesh is Null"));
		return TargetMesh;
	}

	bFoundErrors = true;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		TArray<int32> Triangles;
		Selection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) {
			Triangles.Add(TriangleID);
		});

		// TODO: if #Triangles == Mesh.TriangleCount, use MeshBoundaryLoops

		FMeshRegionBoundaryLoops Loops(&ReadMesh, Triangles, false);
		// Compute returns false if errors occurred
		bFoundErrors = !Loops.Compute();
		NumLoops = Loops.Num();
		IndexLoops.Reserve(NumLoops);
		PathLoops.Reserve(NumLoops);
		for (int32 li = 0; li < NumLoops; ++li)
		{
			FEdgeLoop& Loop = Loops.Loops[li];

			FGeometryScriptPolyPath LoopVertices;
			LoopVertices.Reset();
			LoopVertices.bClosedLoop = true;
			LoopVertices.Path->Reserve(Loop.Vertices.Num());
			for (int32 vid : Loop.Vertices)
			{
				LoopVertices.Path->Add(ReadMesh.GetVertex(vid));
			}

			FGeometryScriptIndexList LoopIndices;
			LoopIndices.Reset(EGeometryScriptIndexType::Vertex);
			*LoopIndices.List = MoveTemp(Loop.Vertices);

			PathLoops.Add(LoopVertices);
			IndexLoops.Add(LoopIndices);
		}

	});
	
	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE
