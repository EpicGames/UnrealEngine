// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSimplifyFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "ProjectionTargets.h"
#include "Polygroups/PolygroupSet.h"
#include "GroupTopology.h"
#include "Operations/PolygroupRemesh.h"
#include "Operations/MeshClusterSimplifier.h"
#include "ConstrainedDelaunay2.h"
#include "MeshSimplificationQuadrics.h"

#if WITH_EDITOR
// for UE Standard simplifier (editor only)
#include "CleaningOps/SimplifyMeshOp.h"
#include "StaticMeshAttributes.h"
#include "Modules/ModuleManager.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"
#endif

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSimplifyFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSimplifyFunctions"

UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPlanar(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPlanarSimplifyOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToPlanar_InvalidInput", "ApplySimplifyToPlanar: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FQEMSimplification Simplifier(&EditMesh);

		// todo: set up seam collapse etc?

		Simplifier.CollapseMode = FQEMSimplification::ESimplificationCollapseModes::AverageVertexPosition;
		Simplifier.SimplifyToMinimalPlanar( FMath::Max(0.00001, Options.AngleThreshold) );

		if (Options.bAutoCompact)
		{
			EditMesh.CompactInPlace();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPolygroupTopology(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPolygroupSimplifyOptions Options,
	FGeometryScriptGroupLayer GroupLayer,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToPolygroupTopology_InvalidInput", "ApplySimplifyToPolygroupTopology: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToPolygroupTopology_MissingGroups", "ApplySimplifyToPolygroupTopology: Target Polygroup Layer does not exist"));
			return;
		}

		TUniquePtr<FGroupTopology> Topo;
		if (GroupLayer.bDefaultLayer)
		{
			Topo = MakeUnique<FGroupTopology>(&EditMesh, true);
		}
		else
		{
			Topo = MakeUnique<FGroupTopology>(&EditMesh, EditMesh.Attributes()->GetPolygroupLayer(GroupLayer.ExtendedLayerIndex), true);
		}

		FPolygroupRemesh Simplifier(&EditMesh, Topo.Get(), ConstrainedDelaunayTriangulate<double>);
		Simplifier.SimplificationAngleTolerance = Options.AngleThreshold;
		Simplifier.Compute();

		if (Options.bAutoCompact)
		{
			EditMesh.CompactInPlace();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



namespace UE::PrivateSimplifyHelper
{
	bool UEStandardEditorSimplify(FDynamicMesh3& Mesh, bool bTargetIsTriCount, int32 TargetCount = 0.f)
	{
#if WITH_EDITOR
		IMeshReductionManagerModule* MeshReductionModule = FModuleManager::Get().LoadModulePtr<IMeshReductionManagerModule>("MeshReductionInterface");
		if (!MeshReductionModule)
		{
			UE_LOGF(LogGeometry, Warning, "Failed to load mesh reduction module; cannot simplify mesh");
			return false;
		}
		IMeshReduction* MeshReduction = MeshReductionModule->GetStaticMeshReductionInterface();
		if (!MeshReduction)
		{
			UE_LOGF(LogGeometry, Warning, "Failed to load mesh reduction interface; cannot simplify mesh");
			return false;
		}

		FMeshDescription SrcMeshDescription;
		FStaticMeshAttributes Attributes(SrcMeshDescription);
		Attributes.Register();

		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(&Mesh, SrcMeshDescription, true);
		float Total = (float)(bTargetIsTriCount ? Mesh.TriangleCount() : Mesh.VertexCount());
		float Percent = Total > 0 ? (float)TargetCount / Total : 0.f;
		return FSimplifyMeshOp::ComputeStandardSimplifier(MeshReduction, SrcMeshDescription, Mesh, Percent, bTargetIsTriCount, false, nullptr);
#else
		return false;
#endif
	}


	enum class ESimplifyTargetMode
	{
		TriangleCount,
		VertexCount,
		EdgeLength
	};

	template <typename SimplificationType>
	static void InitQuadricOptions(const FGeometryScriptSimplifyMeshOptions& Options, 
		typename SimplificationType::FQuadricOptions& QuadricOptions)
	{
	}

	template <>
	void InitQuadricOptions<TMeshSimplification<FAttrBasedQuadricErrorV2d>>(const FGeometryScriptSimplifyMeshOptions& Options, 
		typename TMeshSimplification<FAttrBasedQuadricErrorV2d>::FQuadricOptions& QuadricOptions )
	{
		if (Options.QuadricVariant == EGeometryScriptMeshSimplificationQuadricVariant::PlaneQuadric)
		{
			QuadricOptions.QuadricVariant = FAttrBasedQuadricErrorV2d::EQuadricVariant::PlaneQuadric;
		}
		else
		{
			QuadricOptions.QuadricVariant = FAttrBasedQuadricErrorV2d::EQuadricVariant::TriangleQuadric;
		}
		QuadricOptions.NormalAttributeWeight    = Options.NormalAttributeWeight;
		QuadricOptions.TangentAttributeWeight   = Options.TangentAttributeWeight;
		QuadricOptions.ColorAttributeWeight     = Options.ColorAttributeWeight;
		QuadricOptions.TexCoordAttributeWeight  = Options.TexCoordAttributeWeight;
		QuadricOptions.ScaleCorrection          = Options.ScaleCorrection;
	}
	
	template<typename SimplificationType>
	static void DoSimplifyMesh(
		FDynamicMesh3& EditMesh,
		FGeometryScriptSimplifyMeshOptions Options,
		ESimplifyTargetMode SimplifyTargetMode,
		int32 TargetCount,
		double TargetMinEdgeLength = 0.,
		FMeshProjectionTarget* ProjectionTarget = nullptr,
		double GeometricTolerance = -1 // Note: Values < 0 will skip using GeometricTolerance
	)
	{
		SimplificationType Simplifier(&EditMesh);

		InitQuadricOptions<SimplificationType>(Options, Simplifier.QuadricOptions);

		Simplifier.ProjectionMode = SimplificationType::ETargetProjectionMode::NoProjection;
		if (ProjectionTarget != nullptr)
		{
			Simplifier.SetProjectionTarget(ProjectionTarget);
		}
		
		Simplifier.DEBUG_CHECK_LEVEL = 0;
		Simplifier.bRetainQuadricMemory = Options.bRetainQuadricMemory;
		Simplifier.bAllowSeamCollapse = Options.bAllowSeamCollapse;
		if (Options.bAllowSeamCollapse)
		{
			Simplifier.SetEdgeFlipTolerance(1.e-5);
			if (EditMesh.HasAttributes())
			{
				EditMesh.Attributes()->SplitAllBowties();	// eliminate any bowties that might have formed on attribute seams.
			}
		}

		// do these flags matter here since we are not flipping??
		EEdgeRefineFlags MeshBoundaryConstraints = EEdgeRefineFlags::NoFlip;
		EEdgeRefineFlags GroupBorderConstraints = EEdgeRefineFlags::NoConstraint;
		EEdgeRefineFlags MaterialBorderConstraints = EEdgeRefineFlags::NoConstraint;

		FMeshConstraints Constraints;
		FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, EditMesh,
			MeshBoundaryConstraints, GroupBorderConstraints, MaterialBorderConstraints,
			Options.bAllowSeamSplits, Options.bAllowSeamSmoothing, Options.bAllowSeamCollapse);
		Simplifier.SetExternalConstraints(MoveTemp(Constraints));

		if (Options.bPreserveVertexPositions)
		{
			Simplifier.CollapseMode = SimplificationType::ESimplificationCollapseModes::MinimalExistingVertexError;
		}

		TFunction<double(const FDynamicMesh3& Mesh, int VertexA, int VertexB)> EdgeLengthScaleF = nullptr;
		TFunction<double(const FDynamicMesh3& Mesh, int VertexA, int VertexB)> GeometricErrorScale = nullptr;
		TFunction<double(const FDynamicMesh3& Mesh, int Vertex)> QEMErrorScale = nullptr;
		if (EditMesh.HasAttributes())
		{
			auto CreateWeightMapScaleF = [&EditMesh](int32 WeightMapAttributeLayerIndex, float RelativeDensity) -> TFunction<double(const FDynamicMesh3 & Mesh, int VertexA, int VertexB)>
			{
				TFunction<double(const FDynamicMesh3& Mesh, int VertexA, int VertexB)> ToRet = nullptr;
				if (WeightMapAttributeLayerIndex >= 0 && WeightMapAttributeLayerIndex < EditMesh.Attributes()->NumWeightLayers())
				{
					const FDynamicMeshWeightAttribute* const AttributeLayer = EditMesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex);
					ToRet = [AttributeLayer, RelativeDensity](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
					{
						float WeightValueA;
						AttributeLayer->GetValue(VertexA, &WeightValueA);
						WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

						float WeightValueB;
						AttributeLayer->GetValue(VertexB, &WeightValueB);
						WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

						// choose the weight that will lead to a larger measure, so we choose the more conservative of the two weights
						const float UseWeight = RelativeDensity < 0 ? FMath::Max(WeightValueA, WeightValueB) : FMath::Min(WeightValueA, WeightValueB);

						return FMath::Pow(0.5, -RelativeDensity * (double)UseWeight);
					};
				}
				return ToRet;
			};
			EdgeLengthScaleF = CreateWeightMapScaleF(Options.EdgeLengthWeightMap.Handle.WeightMapAttributeLayerIndex, Options.EdgeLengthWeightMap.RelativeDensity);
			GeometricErrorScale = CreateWeightMapScaleF(Options.GeometricToleranceWeightMap.Handle.WeightMapAttributeLayerIndex, Options.GeometricToleranceWeightMap.RelativeDensity);
			if (Options.QuadricErrorWeightMap.Handle.IsValid() && Options.QuadricErrorWeightMap.Handle.WeightMapAttributeLayerIndex < EditMesh.Attributes()->NumWeightLayers())
			{
				const FDynamicMeshWeightAttribute* const AttributeLayer = EditMesh.Attributes()->GetWeightLayer(Options.QuadricErrorWeightMap.Handle.WeightMapAttributeLayerIndex);
				QEMErrorScale = [AttributeLayer, RelativeDensity = Options.QuadricErrorWeightMap.RelativeDensity](const FDynamicMesh3& Mesh, int Vertex) -> double
				{
					float WeightValue;
					AttributeLayer->GetValue(Vertex, &WeightValue);
					WeightValue = FMath::Clamp(WeightValue, 0.0f, 1.0f);

					// We use a squared error metric for the quadric error scale to get a similar scale in effect as the tolerance/edge-len scales, since it is a squared error term
					return FMath::Pow(0.5, -2. * RelativeDensity * (double)WeightValue);
				};
			}	
		}

		if (ProjectionTarget != nullptr && GeometricTolerance >= 0)
		{
			Simplifier.CustomGeometricErrorScaleF = GeometricErrorScale;
			Simplifier.GeometricErrorConstraint = SimplificationType::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = GeometricTolerance;
		}
		Simplifier.CustomQuadricErrorScaleF = QEMErrorScale;
		Simplifier.RegularizeWeight = Options.RegularizeWeight;

		if (SimplifyTargetMode == ESimplifyTargetMode::TriangleCount)
		{
			Simplifier.SimplifyToTriangleCount(FMath::Max(1, TargetCount));
		}
		else if (SimplifyTargetMode == ESimplifyTargetMode::VertexCount)
		{
			Simplifier.SimplifyToVertexCount(FMath::Max(1, TargetCount));
		}
		else // SimplifyTargetMode == ESimplifyTargetMode::EdgeLength
		{
			Simplifier.CustomEdgeLengthScaleF = EdgeLengthScaleF;
			Simplifier.SimplifyToEdgeLength(TargetMinEdgeLength);
		}


		if (Options.bAutoCompact)
		{
			EditMesh.CompactInPlace();
		}
	}

	static void DoSimplifyMesh(
		EGeometryScriptRemoveMeshSimplificationType SimplificationType,
		FDynamicMesh3& EditMesh,
		FGeometryScriptSimplifyMeshOptions Options,
		ESimplifyTargetMode SimplifyTargetMode,
		int32 TargetCount,
		double TargetMinEdgeLength = 0.,
		FMeshProjectionTarget* ProjectionTarget = nullptr,
		double GeometricTolerance = -1 // Note: Values < 0 will skip using GeometricTolerance
	)
	{
		if (Options.Method == EGeometryScriptRemoveMeshSimplificationType::AttributeAware)
		{
			DoSimplifyMesh<FAttrMeshSimplification>(EditMesh, Options, SimplifyTargetMode, TargetCount, TargetMinEdgeLength, ProjectionTarget, GeometricTolerance);
		}
		else if (Options.Method == EGeometryScriptRemoveMeshSimplificationType::VolumePreserving)
		{
			DoSimplifyMesh<FVolPresMeshSimplification>(EditMesh, Options, SimplifyTargetMode, TargetCount, TargetMinEdgeLength, ProjectionTarget, GeometricTolerance);
		}
		else if (Options.Method == EGeometryScriptRemoveMeshSimplificationType::AttributeAwareV2)
		{
			DoSimplifyMesh<TMeshSimplification<FAttrBasedQuadricErrorV2d>>(EditMesh, Options, SimplifyTargetMode, TargetCount, TargetMinEdgeLength, ProjectionTarget, GeometricTolerance);
		}
		else // EGeometryScriptRemoveMeshSimplificationType::StandardQEM
		{
			DoSimplifyMesh<FQEMSimplification>(EditMesh, Options, SimplifyTargetMode, TargetCount, TargetMinEdgeLength, ProjectionTarget, GeometricTolerance);
		}
	}
}

UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplyEditorSimplifyToTriangleCount(
	UDynamicMesh* TargetMesh,
	int32 TriangleCount,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyEditorSimplifyToTriangleCount_InvalidInput", "ApplyEditorSimplifyToTriangleCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		UE::PrivateSimplifyHelper::UEStandardEditorSimplify(EditMesh, true, TriangleCount);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplyEditorSimplifyToVertexCount(
	UDynamicMesh* TargetMesh,
	int32 VertexCount,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyEditorSimplifyToVertexCount_InvalidInput", "ApplySimplifyToVertexCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		UE::PrivateSimplifyHelper::UEStandardEditorSimplify(EditMesh, false, VertexCount);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToTriangleCount(
	UDynamicMesh* TargetMesh,
	int32 TriangleCount,
	FGeometryScriptSimplifyMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToTriangleCount_InvalidInput", "ApplySimplifyToTriangleCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		using namespace UE::PrivateSimplifyHelper;
		DoSimplifyMesh(Options.Method, EditMesh, Options, ESimplifyTargetMode::TriangleCount, TriangleCount);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToVertexCount(
	UDynamicMesh* TargetMesh,
	int32 VertexCount,
	FGeometryScriptSimplifyMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToVertexCount_InvalidInput", "ApplySimplifyToVertexCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		using namespace UE::PrivateSimplifyHelper;
		DoSimplifyMesh(Options.Method, EditMesh, Options, ESimplifyTargetMode::VertexCount, VertexCount);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToTolerance(
	UDynamicMesh* TargetMesh,
	float Tolerance,
	FGeometryScriptSimplifyMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToTolerance_InvalidInput", "ApplySimplifyToTolerance: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FDynamicMesh3 TempCopy;
		TempCopy.Copy(EditMesh, false, false, false, false);
		FDynamicMeshAABBTree3 Spatial(&TempCopy, true);
		FMeshProjectionTarget ProjTarget(&TempCopy, &Spatial);
		float UseTolerance = FMath::Max(0.0, Tolerance);

		using namespace UE::PrivateSimplifyHelper;
		DoSimplifyMesh(Options.Method, EditMesh, Options, ESimplifyTargetMode::TriangleCount, 1, -1., &ProjTarget, UseTolerance);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToEdgeLength(
	UDynamicMesh* TargetMesh,
	double EdgeLength,
	FGeometryScriptSimplifyMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToVertexCount_InvalidInput", "ApplySimplifyToVertexCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&EdgeLength, &Options](FDynamicMesh3& EditMesh) 
	{
		// Clamp target length to non-negative
		double UseEdgeLength = FMath::Max(0.f, EdgeLength);

		using namespace UE::PrivateSimplifyHelper;
		DoSimplifyMesh(Options.Method, EditMesh, Options, ESimplifyTargetMode::EdgeLength, 1, UseEdgeLength);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSimplifyFunctions::ApplyClusterSimplifyToEdgeLength(
	UDynamicMesh* TargetMesh,
	double EdgeLength,
	FGeometryScriptClusterSimplifyMeshOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplySimplifyToVertexCount_InvalidInput", "ApplySimplifyToVertexCount: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&EdgeLength, &Options](FDynamicMesh3& EditMesh) 
	{
		// Clamp target length to non-negative
		double UseEdgeLength = FMath::Max(0.f, EdgeLength);

		MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
		SimplifyOptions.TargetEdgeLength = UseEdgeLength;
		SimplifyOptions.FixBoundaryAngleTolerance = Options.FixBoundaryAngleTolerance;

		auto ConvertConstraintLevel = [](EGeometryScriptClusterSimplifyConstraintLevel ConstraintLevel) -> MeshClusterSimplify::FSimplifyOptions::EConstraintLevel
			{
				switch (ConstraintLevel)
				{
				default:
					ensure(false);
				case EGeometryScriptClusterSimplifyConstraintLevel::Free:
					return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
				case EGeometryScriptClusterSimplifyConstraintLevel::Constrained:
					return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Constrained;
				case EGeometryScriptClusterSimplifyConstraintLevel::Fixed:
					return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Fixed;
				}
			};
		SimplifyOptions.bTransferGroups = true;

		SimplifyOptions.PreserveEdges.Boundary = ConvertConstraintLevel(Options.EdgeConstraints.MeshBoundary);
		SimplifyOptions.PreserveEdges.PolyGroup = ConvertConstraintLevel(Options.EdgeConstraints.GroupBoundary);
		SimplifyOptions.PreserveEdges.Material = ConvertConstraintLevel(Options.EdgeConstraints.MaterialBoundary);
		SimplifyOptions.PreserveEdges.UVSeam = ConvertConstraintLevel(Options.EdgeConstraints.UVSeam);
		SimplifyOptions.PreserveEdges.ColorSeam = ConvertConstraintLevel(Options.EdgeConstraints.ColorSeam);
		SimplifyOptions.PreserveEdges.NormalSeam = ConvertConstraintLevel(Options.EdgeConstraints.NormalSeam);

		// Note: Currently not exposing tangent seam constraints; they usually should be left free
		SimplifyOptions.PreserveEdges.TangentSeam = MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;

		if (Options.bDiscardAttributes)
		{
			// if discarding attributes, also discard constraints from the attribute layer
			SimplifyOptions.PreserveEdges.SetSeamConstraints(MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free);
			SimplifyOptions.PreserveEdges.Material = MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
			SimplifyOptions.bTransferAttributes = false;
		}

		if (EditMesh.HasAttributes() && Options.WeightMapHandle.IsValid() && Options.WeightMapHandle.WeightMapAttributeLayerIndex < EditMesh.Attributes()->NumWeightLayers())
		{
			const FDynamicMeshWeightAttribute* const AttributeLayer = EditMesh.Attributes()->GetWeightLayer(Options.WeightMapHandle.WeightMapAttributeLayerIndex);
			TFunction<double(int Vertex)> ClusterEdgeTargetLengthScale =
				[AttributeLayer,
				RelativeDensity = Options.RelativeDensity](int Vertex) -> double
				{
					float WeightValue;
					AttributeLayer->GetValue(Vertex, &WeightValue);
					WeightValue = FMath::Clamp(WeightValue, 0.0f, 1.0f);

					// Note: We use positive RelativeDensity here instead of negative, b/c cluster simplifier scales target length rather measured length
					return FMath::Pow(0.5, RelativeDensity * WeightValue);
				};
			SimplifyOptions.OptionalTargetEdgeLengthScale = ClusterEdgeTargetLengthScale;
		}
		
		FDynamicMesh3 ResultMesh;
		MeshClusterSimplify::Simplify(EditMesh, ResultMesh, SimplifyOptions);
		EditMesh = ResultMesh;
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




#undef LOCTEXT_NAMESPACE
