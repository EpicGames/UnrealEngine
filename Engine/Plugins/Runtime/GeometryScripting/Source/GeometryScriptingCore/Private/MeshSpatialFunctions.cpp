// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSpatialFunctions.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "MeshQueries.h"
#include "Spatial/FastWinding.h"
#include "Selections/GeometrySelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSpatialFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSpatial"


void UGeometryScriptLibrary_MeshSpatial::ResetBVH(FGeometryScriptDynamicMeshBVH& ResetBVH)
{
	ResetBVH.Spatial.Reset();
	ResetBVH.FWNTree.Reset();
}

UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::BuildBVHForMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptDynamicMeshBVH& OutputBVH,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BuildBVHForMesh_InvalidInput", "BuildBVHForMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	if (OutputBVH.Spatial.IsValid() == false)
	{
		OutputBVH.Spatial = MakeShared<FDynamicMeshAABBTree3>();
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		OutputBVH.Spatial->SetMesh(&ReadMesh, true);
		OutputBVH.FWNTree = MakeShared<TFastWindingTree<FDynamicMesh3>>(OutputBVH.Spatial.Get(), true);
	});

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::IsBVHValidForMesh(
	UDynamicMesh* TargetMesh,
	const FGeometryScriptDynamicMeshBVH& TestBVH,
	bool& bIsValid,
	UGeometryScriptDebug* Debug)
{
	bIsValid = false;
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsBVHValidForMesh_InvalidInput", "IsBVHValidForMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	if (TestBVH.Spatial.IsValid())
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			if (TestBVH.Spatial->GetMesh() == &ReadMesh)
			{
				bIsValid = TestBVH.Spatial->IsValid(false);
			}
		});
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::RebuildBVHForMesh(
	UDynamicMesh* TargetMesh,
	FGeometryScriptDynamicMeshBVH& UpdateBVH,
	bool bOnlyIfInvalid,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RebuildBVHForMesh_InvalidInput", "RebuildBVHForMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	if (UpdateBVH.Spatial.IsValid() == false)
	{
		UpdateBVH.Spatial = MakeShared<FDynamicMeshAABBTree3>();
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (bOnlyIfInvalid == false || UpdateBVH.Spatial->GetMesh() != &ReadMesh || UpdateBVH.Spatial->IsValid(false) == false)
		{
			UpdateBVH.Spatial->SetMesh(&ReadMesh, false);
			UpdateBVH.Spatial->Build();
			// FWNTree does not support rebuilding like this?
			UpdateBVH.FWNTree = MakeShared<TFastWindingTree<FDynamicMesh3>>(UpdateBVH.Spatial.Get(), true);
		}
	});

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::FindNearestPointOnMesh(
	UDynamicMesh* TargetMesh,
	const FGeometryScriptDynamicMeshBVH& QueryBVH,
	FVector QueryPoint,
	FGeometryScriptSpatialQueryOptions Options,
	FGeometryScriptTrianglePoint& NearestResult,
	EGeometryScriptSearchOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptSearchOutcomePins::NotFound;
	NearestResult.bValid = false;
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindNearestPointOnMesh_InvalidInput", "FindNearestPointOnMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHaveValidBVH = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (QueryBVH.Spatial.IsValid())
		{
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				if (QueryBVH.Spatial->GetMesh() == &ReadMesh && QueryBVH.Spatial->IsValid(Options.bAllowUnsafeModifiedQueries))
				{
					bHaveValidBVH = true;
					IMeshSpatial::FQueryOptions QueryOptions;
					QueryOptions.MaxDistance = (Options.MaxDistance == 0) ? TNumericLimits<float>::Max() : Options.MaxDistance;
					double NearDistSqr = 0;
					int NearestTID = QueryBVH.Spatial->FindNearestTriangle( (FVector3d)QueryPoint, NearDistSqr, QueryOptions);
					if (NearestTID >= 0)
					{
						Outcome = EGeometryScriptSearchOutcomePins::Found;
						FDistPoint3Triangle3d Distance = TMeshQueries<FDynamicMesh3>::TriangleDistance(ReadMesh, NearestTID, (FVector3d)QueryPoint);
						NearestResult.bValid = true;
						NearestResult.TriangleID = NearestTID;
						NearestResult.Position = (FVector)Distance.ClosestTrianglePoint;
						NearestResult.BaryCoords = (FVector)Distance.TriangleBaryCoords;
					}
				}
			});
		}
	});

	if (!bHaveValidBVH)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindNearestPointOnMesh_InvalidBVH", "FindNearestPointOnMesh: QueryBVH is Invalid for this TargetMesh"));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::FindNearestRayIntersectionWithMesh(
	UDynamicMesh* TargetMesh,
	UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
	FVector RayOrigin,
	FVector RayDirection,
	FGeometryScriptSpatialQueryOptions Options,
	FGeometryScriptRayHitResult& HitResult,
	EGeometryScriptSearchOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptSearchOutcomePins::NotFound;
	HitResult.bHit = false;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindNearestRayIntersectionWithMesh_InvalidInput", "FindNearestRayIntersectionWithMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHaveValidBVH = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (QueryBVH.Spatial.IsValid())
		{
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				if (QueryBVH.Spatial->GetMesh() == &ReadMesh && QueryBVH.Spatial->IsValid(Options.bAllowUnsafeModifiedQueries))
				{
					bHaveValidBVH = true;
					IMeshSpatial::FQueryOptions QueryOptions;
					QueryOptions.MaxDistance = (Options.MaxDistance == 0) ? TNumericLimits<float>::Max() : Options.MaxDistance;
					FRay3d Ray((FVector3d)RayOrigin, Normalized((FVector3d)RayDirection));
					int HitTID = QueryBVH.Spatial->FindNearestHitTriangle(Ray, QueryOptions);
					if (HitTID >= 0)
					{
						Outcome = EGeometryScriptSearchOutcomePins::Found;
						FIntrRay3Triangle3d Intersection = TMeshQueries<FDynamicMesh3>::TriangleIntersection(ReadMesh, HitTID, Ray);
						HitResult.RayParameter = Intersection.RayParameter;
						HitResult.bHit = true;
						HitResult.HitTriangleID = HitTID;
						HitResult.HitPosition = Ray.PointAt(Intersection.RayParameter);
						HitResult.HitBaryCoords = (FVector)Intersection.TriangleBaryCoords;
					}
				}
			});
		}
	});

	if (!bHaveValidBVH)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindNearestRayIntersectionWithMesh_InvalidBVH", "FindNearestRayIntersectionWithMesh: QueryBVH is Invalid for this TargetMesh"));
	}

	return TargetMesh;
}






UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::IsPointInsideMesh(
	UDynamicMesh* TargetMesh,
	UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
	FVector QueryPoint,
	FGeometryScriptSpatialQueryOptions Options,
	bool& bIsInside,
	EGeometryScriptContainmentOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptContainmentOutcomePins::Outside;
	bIsInside = false;

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsPointInsideMesh_InvalidInput", "IsPointInsideMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHaveValidBVH = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (QueryBVH.Spatial.IsValid() && QueryBVH.FWNTree.IsValid())
		{
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				if (QueryBVH.Spatial->GetMesh() == &ReadMesh && QueryBVH.Spatial->IsValid(Options.bAllowUnsafeModifiedQueries))
				{
					bHaveValidBVH = true;
					bIsInside = QueryBVH.FWNTree->IsInside((FVector3d)QueryPoint, Options.WindingIsoThreshold);
					if (bIsInside)
					{
						Outcome = EGeometryScriptContainmentOutcomePins::Inside;
					}
				}
			});
		}
	});

	if (!bHaveValidBVH)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsPointInsideMesh_InvalidBVH", "IsPointInsideMesh: QueryBVH is Invalid for this TargetMesh"));
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshSpatial::SelectMeshElementsInBoxWithBVH(
	UDynamicMesh* TargetMesh,
	UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
	FBox QueryBox,
	FGeometryScriptSpatialQueryOptions Options,
	FGeometryScriptMeshSelection& SelectionOut,
	EGeometryScriptMeshSelectionType SelectionType,
	int MinNumTrianglePoints,
	UGeometryScriptDebug* Debug)
{
	MinNumTrianglePoints = FMath::Clamp(MinNumTrianglePoints, 1, 3);

	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SelectMeshElementsInBoxWithBVH_InvalidInput", "SelectMeshElementsInBoxWithBVH: TargetMesh is Null"));
		return TargetMesh;
	}

	FAxisAlignedBox3d QueryBounds(QueryBox);
	IMeshSpatial::FQueryOptions QueryOptions;
	QueryOptions.MaxDistance = (Options.MaxDistance == 0) ? TNumericLimits<float>::Max() : Options.MaxDistance;

	FGeometrySelection GeoSelection;

	auto TriangleIntersectionTest = [](const FDynamicMesh3& Mesh, int32 TriangleID, const FAxisAlignedBox3d& Box, int MinNumTrianglePoints) -> bool
	{
		FIndex3i Tri = Mesh.GetTriangle(TriangleID);
		int NumContained =
			(Box.Contains(Mesh.GetVertex(Tri.A)) ? 1 : 0) +
			(Box.Contains(Mesh.GetVertex(Tri.B)) ? 1 : 0) +
			(Box.Contains(Mesh.GetVertex(Tri.C)) ? 1 : 0);
		return (NumContained >= MinNumTrianglePoints);
	};

	bool bHaveValidBVH = false;
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (QueryBVH.Spatial.IsValid())
		{
			TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				if (QueryBVH.Spatial->GetMesh() == &ReadMesh && QueryBVH.Spatial->IsValid(Options.bAllowUnsafeModifiedQueries))
				{
					if (SelectionType == EGeometryScriptMeshSelectionType::Vertices)
					{
						GeoSelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);

						FDynamicMeshAABBTree3::FTreeTraversal Traversal;
						Traversal.NextBoxF = [QueryBounds](const FAxisAlignedBox3d& Box, int Depth) { return Box.Intersects(QueryBounds); };
						Traversal.NextTriangleF = [&ReadMesh,&GeoSelection,QueryBounds](int TriangleID) 
						{
							FIndex3i Tri = ReadMesh.GetTriangle(TriangleID);
							for (int32 k = 0; k < 3; ++k)
							{
								if ( QueryBounds.Contains( ReadMesh.GetVertex(Tri[k])) )
								{
									GeoSelection.Selection.Add(FGeoSelectionID::MeshVertex(Tri[k]).Encoded());
								}
							}
						};
						QueryBVH.Spatial->DoTraversal(Traversal, QueryOptions);
					}
					else if (SelectionType == EGeometryScriptMeshSelectionType::Triangles)
					{
						GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
						FDynamicMeshAABBTree3::FTreeTraversal Traversal;
						Traversal.NextBoxF = [QueryBounds](const FAxisAlignedBox3d& Box, int Depth) { return Box.Intersects(QueryBounds); };
						Traversal.NextTriangleF = [&](int TriangleID) 
						{
							if (TriangleIntersectionTest(ReadMesh, TriangleID, QueryBounds, MinNumTrianglePoints))
							{
								GeoSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
							}
						};
						QueryBVH.Spatial->DoTraversal(Traversal, QueryOptions);
					}
					else if (SelectionType == EGeometryScriptMeshSelectionType::Edges)
					{
						int32 UseMinNumEdgePoints = FMath::Clamp(MinNumTrianglePoints, 1, 2);
						GeoSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
						FDynamicMeshAABBTree3::FTreeTraversal Traversal;
						Traversal.NextBoxF = [QueryBounds](const FAxisAlignedBox3d& Box, int Depth) { return Box.Intersects(QueryBounds); };
						Traversal.NextTriangleF = [&ReadMesh, &GeoSelection, QueryBounds, UseMinNumEdgePoints](int TriangleID)
						{
							FIndex3i Tri = ReadMesh.GetTriangle(TriangleID);
							bool ContainsV[3]{};
							for (int32 k = 0; k < 3; ++k)
							{
								ContainsV[k] = QueryBounds.Contains(ReadMesh.GetVertex(Tri[k]));
							}
							for (int32 Idx = 0, PrevIdx = 2; Idx < 3; PrevIdx = Idx++)
							{
								int32 ContainCount = (int32)ContainsV[Idx] + (int32)ContainsV[PrevIdx];
								if (ContainCount >= UseMinNumEdgePoints)
								{
									GeoSelection.Selection.Add(FMeshTriEdgeID(TriangleID, PrevIdx).Encoded());
								}
							}
						};
						QueryBVH.Spatial->DoTraversal(Traversal, QueryOptions);
					}
					else if (SelectionType == EGeometryScriptMeshSelectionType::Polygroups)
					{
						TArray<int32> UniqueGroups;
						TArray<FIndex2i> GroupTriPairs;
						FDynamicMeshAABBTree3::FTreeTraversal Traversal;
						Traversal.NextBoxF = [QueryBounds](const FAxisAlignedBox3d& Box, int Depth) { return Box.Intersects(QueryBounds); };
						Traversal.NextTriangleF = [&](int TriangleID) 
						{
							if (TriangleIntersectionTest(ReadMesh, TriangleID, QueryBounds, MinNumTrianglePoints))
							{
								int32 GroupID = ReadMesh.GetTriangleGroup(TriangleID);
								if (UniqueGroups.Contains(GroupID) == false)
								{
									UniqueGroups.Add(GroupID);
									GroupTriPairs.Add(FIndex2i(TriangleID, GroupID));
								}
							}
						};
						QueryBVH.Spatial->DoTraversal(Traversal, QueryOptions);

						GeoSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Polygroup);
						for (FIndex2i Pair : GroupTriPairs)
						{
							GeoSelection.Selection.Add(FGeoSelectionID::GroupFace(Pair.A, Pair.B).Encoded());
						}
					}
				}
			});
		}

	});

	if (!bHaveValidBVH)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("IsPointInsideMesh_InvalidBVH", "IsPointInsideMesh: QueryBVH is Invalid for this TargetMesh"));
	}

	SelectionOut.SetSelection(MoveTemp(GeoSelection));


	return TargetMesh;
}



#undef LOCTEXT_NAMESPACE

