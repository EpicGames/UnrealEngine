// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshSubdivideFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "Operations/PNTriangles.h"
#include "Operations/UniformTessellate.h"
#include "Operations/SelectiveTessellate.h"
#include "UDynamicMesh.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSubdivideFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshSubdivideFunctions"

UDynamicMesh* UGeometryScriptLibrary_MeshSubdivideFunctions::ApplyPNTessellation(
	UDynamicMesh* TargetMesh,
	FGeometryScriptPNTessellateOptions Options,
	int TessellationLevel,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPNTessellation_InvalidInput", "ApplyPNTessellation: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TessellationLevel <= 0)
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FPNTriangles Tessellator(&EditMesh);
		Tessellator.TessellationLevel = TessellationLevel;
		Tessellator.bRecalculateNormals = Options.bRecomputeNormals;
		
		if (Tessellator.Validate() != EOperationValidationResult::Ok)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPNTessellation_Error", "ApplyPNTessellation: The inputs are invalid"));
			return;
		}

		if (Tessellator.Compute() == false) 
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyPNTessellation_Failed", "ApplyPNTessellation: Tessellation failed"));
			return;
		}
		
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshSubdivideFunctions::ApplyUniformTessellation(
	UDynamicMesh* TargetMesh,
	int TessellationLevel,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyUniformTessellation_InvalidInput", "ApplyUniformTessellation: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TessellationLevel <= 0)
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FUniformTessellate Tessellator(&EditMesh);
		Tessellator.TessellationNum = TessellationLevel;
		
		if (Tessellator.Validate() != EOperationValidationResult::Ok)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyUniformTessellation_Error", "ApplyUniformTessellation: The inputs are invalid"));
			return;
		} 

		if (Tessellator.Compute() == false) 
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyUniformTessellation_Failed", "ApplyUniformTessellation: Tessellation failed")); 
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshSubdivideFunctions::ApplySelectiveTessellation(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FGeometryScriptSelectiveTessellateOptions Options,
	int TessellationLevel,
	ESelectiveTessellatePatternType PatternType,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdapativeTessellation_InvalidInput", "ApplyAdapativeTessellation: TargetMesh is Null"));
		return TargetMesh;
	}
	if (TessellationLevel <= 0)
	{
		return TargetMesh;
	}
	if (PatternType != ESelectiveTessellatePatternType::ConcentricRings)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdapativeTessellation_InvalidPatternTypeNew", "ApplyAdapativeTessellation: Only ConcentricRings pattern is currently supported"));
		return TargetMesh;
	} 
	if (Selection.IsEmpty() && Options.EmptyBehavior != EGeometryScriptEmptySelectionBehavior::FullMeshSelection )
	{
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{	
		FDynamicMesh3 TessellatedMesh;
		FSelectiveTessellate Tessellator(&EditMesh, &TessellatedMesh);

		TArray<int32> Triangles;
		Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
		if (Triangles.Num() == 0 && Options.EmptyBehavior != EGeometryScriptEmptySelectionBehavior::FullMeshSelection)
		{
			return;
		}

		TUniquePtr<FTessellationPattern> Pattern; 
		if (PatternType == ESelectiveTessellatePatternType::ConcentricRings)
		{
			Pattern = (Triangles.Num() > 0) ?
				FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&EditMesh, TessellationLevel, Triangles)
				: FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&EditMesh, TessellationLevel);
		}
		else
		{
			checkSlow(false);
			return;
		}

		Tessellator.SetPattern(Pattern.Get());
		Tessellator.bUseParallel = Options.bEnableMultithreading;
		
		if (Tessellator.Validate() != EOperationValidationResult::Ok)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyAdapativeTessellation_Error", "ApplyAdapativeTessellation: The inputs are invalid"));
			return;
		} 

		if (Tessellator.Compute() == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyAdapativeTessellate_Failed", "ApplyAdapativeTessellate: Tessellation failed")); 
		}
		else 
		{
			EditMesh = MoveTemp(TessellatedMesh);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

#undef LOCTEXT_NAMESPACE

