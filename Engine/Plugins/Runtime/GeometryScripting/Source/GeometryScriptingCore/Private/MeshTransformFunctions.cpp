// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshTransformFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Async/ParallelFor.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTransformFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshTransformFunctions"


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::TransformMesh(
	UDynamicMesh* TargetMesh,
	FTransform Transform,
	bool bFixOrientationForNegativeScale,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformMesh_InvalidInput", "TransformMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		MeshTransforms::ApplyTransform(EditMesh, (FTransformSRT3d)Transform, bFixOrientationForNegativeScale);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::InverseTransformMesh(
	UDynamicMesh* TargetMesh,
	FTransform Transform,
	bool bFixOrientationForNegativeScale,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("InverseTransformMesh_InvalidInput", "InverseTransformMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		MeshTransforms::ApplyTransformInverse(EditMesh, (FTransformSRT3d)Transform, bFixOrientationForNegativeScale);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::TranslateMesh(
	UDynamicMesh* TargetMesh,
	FVector Translation,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TranslateMesh_InvalidInput", "TranslateMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		MeshTransforms::Translate(EditMesh, (FVector3d)Translation);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::RotateMesh(
	UDynamicMesh* TargetMesh,
	FRotator Rotation,
	FVector RotationOrigin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RotateMesh_InvalidInput", "RotateMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		MeshTransforms::Rotate(EditMesh, Rotation, RotationOrigin);
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::ScaleMesh(
	UDynamicMesh* TargetMesh,
	FVector Scale,
	FVector ScaleOrigin,
	bool bFixOrientationForNegativeScale,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ScaleMesh_InvalidInput", "ScaleMesh: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		MeshTransforms::Scale(EditMesh, (FVector3d)Scale, (FVector3d)ScaleOrigin, bFixOrientationForNegativeScale);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}







UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::TransformMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FTransform Transform,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransformMeshSelection_InvalidInput", "TransformMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TArray<int32> Vertices;
		Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
		for ( int32 vid : Vertices)
		{
			if (EditMesh.IsVertex(vid))
			{
				EditMesh.SetVertex(vid, Transform.TransformPosition( EditMesh.GetVertex(vid) ) );
			}
		}

		// normals?

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::InverseTransformMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FTransform Transform,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("InverseTransformMeshSelection_InvalidInput", "InverseTransformMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		TArray<int32> Vertices;
		Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
		for ( int32 vid : Vertices)
		{
			if (EditMesh.IsVertex(vid))
			{
				EditMesh.SetVertex(vid, Transform.InverseTransformPosition( EditMesh.GetVertex(vid) ) );
			}
		}

		// normals?

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::TranslateMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FVector Translation,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TranslateMeshSelection_InvalidInput", "TranslateMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if ( ! Selection.IsEmpty() )
		{
			TArray<int32> Vertices;
			Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
			for ( int32 vid : Vertices)
			{
				if (EditMesh.IsVertex(vid))
				{
					EditMesh.SetVertex(vid, EditMesh.GetVertex(vid) + Translation);
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::RotateMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FRotator Rotation,
	FVector RotationOrigin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RotateMeshSelection_InvalidInput", "RotateMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if ( ! Selection.IsEmpty() )
		{
			TArray<int32> Vertices;
			Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
			for ( int32 vid : Vertices)
			{
				if (EditMesh.IsVertex(vid))
				{
					EditMesh.SetVertex(vid, Rotation.RotateVector( (EditMesh.GetVertex(vid) - RotationOrigin) ) + RotationOrigin);
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshTransformFunctions::ScaleMeshSelection(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FVector Scale,
	FVector ScaleOrigin,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ScaleMeshSelection_InvalidInput", "ScaleMeshSelection: TargetMesh is Null"));
		return TargetMesh;
	}

	// todo: publish correct change types
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if ( ! Selection.IsEmpty() )
		{
			TArray<int32> Vertices;
			Selection.ConvertToMeshIndexArray(EditMesh, Vertices, EGeometryScriptIndexType::Vertex);
			for ( int32 vid : Vertices)
			{
				if (EditMesh.IsVertex(vid))
				{
					EditMesh.SetVertex(vid, (EditMesh.GetVertex(vid) - ScaleOrigin) * Scale + ScaleOrigin);
				}
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}





#undef LOCTEXT_NAMESPACE

