// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshUVFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Polygroups/PolygroupSet.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/PatchBasedMeshUVGenerator.h"
#include "DynamicMesh/MeshNormals.h"
#include "Parameterization/MeshLocalParam.h"
#include "ParameterizationOps/UVLayoutOp.h"
#include "TriangleTypes.h"
#include "XAtlasWrapper.h"
#include "ParameterizationOps/TexelDensityOp.h"

#include "Async/ParallelFor.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshUVFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshUVFunctions"



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetNumUVSets(
	UDynamicMesh* TargetMesh,
	int NumUVSets,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetNumUVSets_InvalidInput", "SetNumUVSets: TargetMesh is Null"));
		return TargetMesh;
	}
	if (NumUVSets > 8)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetNumUVSets_InvalidNumUVSets", "SetNumUVSets: Maximum of 8 UV Sets are supported"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
		}
		if (NumUVSets != EditMesh.Attributes()->NumUVLayers())
		{
			EditMesh.Attributes()->SetNumUVLayers(NumUVSets);
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::ClearUVChannel(
	UDynamicMesh* TargetMesh,
	int UVChannel,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ClearUVChannel_InvalidInput", "ClearUVChannel: TargetMesh is Null"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([UVChannel, Debug](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshUVOverlay* UVOverlay = nullptr;
		if (!EditMesh.HasAttributes() || UVChannel < 0 || UVChannel >= EditMesh.Attributes()->NumUVLayers())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ClearUVChannel_NoChannel", "ClearUVChannel: Mesh did not have the requested UV channel, so it could not be cleared."));
			return;
		}
		else
		{
			UVOverlay = EditMesh.Attributes()->GetUVLayer(UVChannel);
			UVOverlay->ClearElements();
		}
	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::UVs, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::CopyUVSet(
	UDynamicMesh* TargetMesh,
	int FromUVSet,
	int ToUVSet,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyUVSet_InvalidInput", "CopyUVSet: TargetMesh is Null"));
		return TargetMesh;
	}
	if (FromUVSet == ToUVSet)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyUVSet_SameSet", "CopyUVSet: From and To UV Sets have the same Index"));
		return TargetMesh;
	}
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshUVOverlay* FromUVOverlay = nullptr, *ToUVOverlay = nullptr;
		if (EditMesh.HasAttributes())
		{
			FromUVOverlay = (FromUVSet < EditMesh.Attributes()->NumUVLayers()) ? EditMesh.Attributes()->GetUVLayer(FromUVSet) : nullptr;
			ToUVOverlay = (ToUVSet < EditMesh.Attributes()->NumUVLayers()) ? EditMesh.Attributes()->GetUVLayer(ToUVSet) : nullptr;
		}
		if (FromUVOverlay == nullptr || ToUVOverlay == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetNumUVSets_CopyUVSet", "CopyUVSet: From or To UV Set does not Exist"));
			return;
		}
		FDynamicMeshUVEditor UVEditor(&EditMesh, ToUVOverlay);
		UVEditor.CopyUVLayer(FromUVOverlay);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshTriangleUVs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	int TriangleID, 
	FGeometryScriptUVTriangle UVs,
	bool& bIsValidTriangle,
	bool bDeferChangeNotifications)
{
	bIsValidTriangle = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsTriangle(TriangleID) && EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr)
				{
					bIsValidTriangle = true;
					int32 Elem0 = UVOverlay->AppendElement((FVector2f)UVs.UV0);
					int32 Elem1 = UVOverlay->AppendElement((FVector2f)UVs.UV1);
					int32 Elem2 = UVOverlay->AppendElement((FVector2f)UVs.UV2);
					UVOverlay->SetTriangle(TriangleID, FIndex3i(Elem0, Elem1, Elem2), true);
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::AddUVElementToMesh(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FVector2D NewUVPosition,
	int& NewUVElementID,
	bool& bIsValidUVSet,
	bool bDeferChangeNotifications)
{
	bIsValidUVSet = false;
	NewUVElementID = -1;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr)
				{
					bIsValidUVSet = true;
					NewUVElementID = UVOverlay->AppendElement((FVector2f)NewUVPosition);
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshTriangleUVElementIDs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	int TriangleID, 
	FIntVector TriangleUVElements,
	bool& bIsValidTriangle,
	bool bDeferChangeNotifications)
{
	bIsValidTriangle = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsTriangle(TriangleID) && EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr)
				{
					// sanity check here because SetTriangle does not
					FIndex3i MeshTri = EditMesh.GetTriangle(TriangleID);
					for (int32 j = 0; j < 3; ++j)
					{
						int32 VertexID = MeshTri[j];
						int32 ElemID = TriangleUVElements[j];
						int32 ParentVertexID = UVOverlay->GetParentVertex(ElemID);
						if (ParentVertexID != IndexConstants::InvalidID && ParentVertexID != VertexID)
						{
							return;		// would create broken topology
						}
						
					}

					if (UVOverlay->SetTriangle(TriangleID, (FIndex3i)TriangleUVElements, false) == EMeshResult::Ok)
					{
						bIsValidTriangle = true;
					}
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::GetMeshTriangleUVElementIDs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	int TriangleID,
	FIntVector& TriangleUVElements,
	bool& bHaveValidUVs)
{
	bHaveValidUVs = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
		{
			if (EditMesh.IsTriangle(TriangleID) && EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				const FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr && UVOverlay->IsSetTriangle(TriangleID))
				{
					bHaveValidUVs = true;
					TriangleUVElements = UVOverlay->GetTriangle(TriangleID);
				}
			}
		});
	}
	return TargetMesh;	
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::GetMeshUVElementPosition(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	int ElementID,
	FVector2D& UVPosition,
	bool& bIsValidElementID)
{
	bIsValidElementID = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				const FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr && UVOverlay->IsElement(ElementID))
				{
					bIsValidElementID = true;
					UVPosition = (FVector2D)UVOverlay->GetElement(ElementID);
				}
			}
		});
	}
	return TargetMesh;	
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshUVElementPosition(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	int ElementID,
	FVector2D NewUVPosition,
	bool& bIsValidElementID,
	bool bDeferChangeNotifications)
{
	bIsValidElementID = false;
	if (TargetMesh)
	{
		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (EditMesh.HasAttributes() && UVSetIndex < EditMesh.Attributes()->NumUVLayers() )
			{
				FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
				if (UVOverlay != nullptr && UVOverlay->IsElement(ElementID))
				{
					bIsValidElementID = true;
					UVOverlay->SetElement(ElementID, (FVector2f)NewUVPosition);
				}
			}
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
	}
	return TargetMesh;	
}


void ApplyMeshUVEditorOperation(UDynamicMesh* TargetMesh, int32 UVSetIndex, bool& bHasUVSet, UGeometryScriptDebug* Debug,
	TFunctionRef<void(FDynamicMesh3& Mesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)> EditFunc, bool bDeferChangeNotifications = false)
{
	bHasUVSet = false;
	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() == false
			|| UVSetIndex >= EditMesh.Attributes()->NumUVLayers()
			|| EditMesh.Attributes()->GetUVLayer(UVSetIndex) == nullptr)
		{
			return;
		}

		bHasUVSet = true;
		FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVSetIndex);
		FDynamicMeshUVEditor Editor(&EditMesh, UVOverlay);
		EditFunc(EditMesh, UVOverlay, Editor);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChangeNotifications);
}


namespace UELocal
{
void ApplyUVTransform(
	FDynamicMesh3& EditMesh,
	FDynamicMeshUVOverlay& UVOverlay,
	FGeometryScriptMeshSelection& Selection,
	TFunctionRef<FVector2f(FVector2f)> UVTransformFunc
)
{
	if (Selection.IsEmpty())
	{
		for (int32 ElementID : UVOverlay.ElementIndicesItr())
		{
			FVector2f UV = UVOverlay.GetElement(ElementID);
			UVOverlay.SetElement(ElementID, UVTransformFunc(UV));
		}
	}
	else
	{
		TSet<int32> ElementSet;
		Selection.ProcessByTriangleID(EditMesh, [&](int32 TriangleID)
		{
			if (UVOverlay.IsSetTriangle(TriangleID))
			{
				FIndex3i TriElems = UVOverlay.GetTriangle(TriangleID);
				ElementSet.Add(TriElems.A); ElementSet.Add(TriElems.B); ElementSet.Add(TriElems.C);
			}
		});
		for (int32 ElementID : ElementSet)
		{
			FVector2f UV = UVOverlay.GetElement(ElementID);
			UVOverlay.SetElement(ElementID, UVTransformFunc(UV));
		}
	}
}
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetUVSeamsAlongSelectedEdges(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FGeometryScriptMeshSelection Selection,
	bool bInsertSeams,
	bool bDeferChangeNotifications,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetUVSeamsAlongSelectedEdges_InvalidInput", "SetUVSeamsAlongSelectedEdges: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&Selection, bInsertSeams](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TArray<int32> EdgeArr;
		Selection.ConvertToMeshIndexArray(EditMesh, EdgeArr, EGeometryScriptIndexType::Edge);
		TSet<int32> EdgeSet(EdgeArr);
		if (bInsertSeams)
		{
			UVEditor.CreateSeamsAtEdges(EdgeSet);
		}
		else
		{
			UVEditor.RemoveSeamsAtEdges(EdgeSet);
		}
	}, bDeferChangeNotifications);
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetUVSeamsAlongSelectedEdges_InvalidUVSet", "SetUVSeamsAlongSelectedEdges: UV Channel does not exist on TargetMesh"));
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::TranslateMeshUVs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FVector2D Translation,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TranslateMeshUVs_InvalidInput", "TranslateMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		UELocal::ApplyUVTransform(EditMesh, *UVOverlay, Selection,
			[&](FVector2f UV) { return UV + (FVector2f)Translation; });
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TranslateMeshUVs_InvalidUVSet", "TranslateMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::ScaleMeshUVs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FVector2D Scale,
	FVector2D ScaleOrigin,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ScaleMeshUVs_InvalidInput", "ScaleMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	FVector2f UseScale = FVector2f(Scale);
	if (UseScale.Length() < 0.0001)
	{
		UseScale = FVector2f::One();
	}
	FVector2f UseOrigin = (FVector2f)ScaleOrigin;

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		UELocal::ApplyUVTransform(EditMesh, *UVOverlay, Selection,
			[&](FVector2f UV) { return (UV - UseOrigin) * UseScale + UseOrigin; });
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ScaleMeshUVs_InvalidUVSet", "ScaleMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::RotateMeshUVs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	float RotationAngle,
	FVector2D RotationOrigin,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RotateMeshUVs_InvalidInput", "RotateMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	FMatrix2f RotationMat = FMatrix2f::RotationDeg(RotationAngle);
	FVector2f UseOrigin = (FVector2f)RotationOrigin;

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		UELocal::ApplyUVTransform(EditMesh, *UVOverlay, Selection,
			[&](FVector2f UV) { return RotationMat * (UV - UseOrigin) + UseOrigin; });
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RotateMeshUVs_InvalidUVSet", "RotateMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshUVsFromPlanarProjection(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FTransform PlaneTransform,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromPlanarProjection_InvalidInput", "SetMeshUVsFromPlanarProjection: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TArray<int32> TriangleROI;
		Selection.ProcessByTriangleID(EditMesh, [&](int32 TriangleID) { TriangleROI.Add(TriangleID); }, true);

		FFrame3d ProjectionFrame(PlaneTransform);
		FVector Scale = PlaneTransform.GetScale3D();
		FVector2d Dimensions(Scale.X, Scale.Y);

		UVEditor.SetTriangleUVsFromPlanarProjection(TriangleROI, [&](const FVector3d& Pos) { return Pos; },
			ProjectionFrame, Dimensions);
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromPlanarProjection_InvalidUVSet", "SetMeshUVsFromPlanarProjection: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshUVsFromBoxProjection(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FTransform PlaneTransform,
	FGeometryScriptMeshSelection Selection,
	int MinIslandTriCount,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromBoxProjection_InvalidInput", "SetMeshUVsFromBoxProjection: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TArray<int32> TriangleROI;
		Selection.ProcessByTriangleID(EditMesh, [&](int32 TriangleID) { TriangleROI.Add(TriangleID); }, true);

		FFrame3d ProjectionFrame(PlaneTransform);
		FVector Scale = PlaneTransform.GetScale3D();
		FVector3d Dimensions = (FVector)Scale;
		UVEditor.SetTriangleUVsFromBoxProjection(TriangleROI, [&](const FVector3d& Pos) { return Pos; },
			ProjectionFrame, Dimensions, MinIslandTriCount);
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromBoxProjection_InvalidUVSet", "SetMeshUVsFromBoxProjection: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::SetMeshUVsFromCylinderProjection(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FTransform CylinderTransform,
	FGeometryScriptMeshSelection Selection,
	float SplitAngle,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromCylinderProjection_InvalidInput", "SetMeshUVsFromCylinderProjection: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TArray<int32> TriangleROI;
		Selection.ProcessByTriangleID(EditMesh, [&](int32 TriangleID) { TriangleROI.Add(TriangleID); }, true);

		FFrame3d ProjectionFrame(CylinderTransform);
		FVector Scale = CylinderTransform.GetScale3D();
		FVector3d Dimensions = (FVector)Scale;
		UVEditor.SetTriangleUVsFromCylinderProjection(TriangleROI, [&](const FVector3d& Pos) { return Pos; },
			ProjectionFrame, Dimensions, SplitAngle);
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromCylinderProjection_InvalidUVSet", "SetMeshUVsFromCylinderProjection: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::TransferMeshUVsByProjection(
	UDynamicMesh* TargetMesh,
	int TargetUVChannel,
	FGeometryScriptMeshSelection TargetSelection,
	FTransform TargetTransform,
	const UDynamicMesh* SourceMesh,
	FGeometryScriptDynamicMeshBVH SourceMeshOptionalBVH,
	int SourceUVChannel,
	FGeometryScriptMeshSelection SourceSelection,
	FTransform SourceTransform,
	FGeometryScriptMeshProjectionSettings Settings,
	FVector ProjectionDirection,
	double ProjectionOffset,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromMeshProjection_InvalidInput_Target", "SetMeshUVsFromMeshProjection: TargetMesh is Null"));
		return TargetMesh;
	}

	if (SourceMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromMeshProjection_InvalidInput_Source", "SetMeshUVsFromMeshProjection: SourceMesh is Null"));
		return TargetMesh;
	}

	const FDynamicMeshUVOverlay* SourceUVLayer = nullptr;
	SourceMesh->ProcessMesh([&SourceUVLayer, SourceUVChannel](const FDynamicMesh3& SourceMesh)
	{
		if (const FDynamicMeshAttributeSet* AttribSet = SourceMesh.Attributes())
		{
			SourceUVLayer = AttribSet->GetUVLayer(SourceUVChannel);
		}
	});
	if (!SourceUVLayer)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromMeshProjection_InvalidSourceUVSet", "SetMeshUVsFromMeshProjection: Source UV Channel does not exist on SourceMesh"));
		return TargetMesh;
	}

	if (TargetSelection.IsEmpty() && !Settings.bProcessAllIfEmptySelection)
	{
		return TargetMesh;
	}
	if (SourceSelection.IsEmpty() && !Settings.bProcessAllIfEmptySelection)
	{
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, TargetUVChannel, bHasUVSet, Debug,
		[SourceUVChannel, &SourceMesh, &SourceMeshOptionalBVH, &TargetSelection, &SourceSelection,
		ProjectionDirection, ProjectionOffset, &Settings, TargetTransform, SourceTransform]
		(FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TArray<int32> TriangleROI;
		TargetSelection.ProcessByTriangleID(EditMesh, [&TriangleROI](int32 TriangleID) { TriangleROI.Add(TriangleID); }, true);
		FDynamicMeshUVEditor::FTransferFromMeshViaProjectionSettings ProjSettings;
		ProjSettings.bResetUVsForUnmatched = Settings.bResetUVsForUnmatched;
		ProjSettings.MaxDistance = Settings.ProjectionRangeMax;
		ProjSettings.MinDistance = Settings.ProjectionRangeMin;
		TSet<int32> SourceTriROI;
		if (!SourceSelection.IsEmpty())
		{
			SourceSelection.ProcessByTriangleID(SourceMesh->GetMeshRef(), [&SourceTriROI](int32 TID) { SourceTriROI.Add(TID); });
			ProjSettings.SourceMeshTriFilter = [&SourceTriROI](int32 TID)
				{
					return SourceTriROI.Contains(TID);
				};
		}
		FDynamicMeshAABBTree3 SourceMeshSpatial(SourceMesh->GetMeshPtr(), false);
		FDynamicMeshAABBTree3* UseSpatial = &SourceMeshSpatial;
		if (SourceMeshOptionalBVH.Spatial.IsValid() && SourceMeshOptionalBVH.Spatial->GetMesh() == SourceMesh->GetMeshPtr())
		{
			UseSpatial = SourceMeshOptionalBVH.Spatial.Get();
		}
		else
		{
			SourceMeshSpatial.Build();
		}
		UVEditor.TransferTriangleUVsFromMeshViaDirectionProjection(TriangleROI, [TargetTransform, SourceTransform](const FVector3d& Pos) -> FVector3d
			{
				return SourceTransform.InverseTransformPosition(TargetTransform.TransformPosition(Pos));
			},
			ProjectionDirection, ProjectionOffset, [SourceTransform](const FVector3d& Vec) -> FVector3d
			{
				return SourceTransform.InverseTransformVector(Vec);
			}, *UseSpatial, SourceUVChannel,
			ProjSettings);
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshUVsFromMeshProjection_InvalidUVSet", "SetMeshUVsFromMeshProjection: Target UV Channel does not exist on TargetMesh"));
	}

	return TargetMesh;
}

UDynamicMesh*
UGeometryScriptLibrary_MeshUVFunctions::ApplyTexelDensityUVScaling(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FGeometryScriptUVTexelDensityOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyTexelDensityUVScaling_InvalidInput", "ApplyTexelDensityUVScaling: TargetMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{

		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(EditMesh));

		FUVEditorTexelDensityOp Op;
		
		Op.OriginalMesh = SourceMesh;

		switch (Options.TexelDensityMode)
		{
		case EGeometryScriptTexelDensityMode::ApplyToIslands:
			Op.TexelDensityMode = EUVTexelDensityOpModes::ScaleIslands;
			break;
		case EGeometryScriptTexelDensityMode::ApplyToWhole:
			Op.TexelDensityMode = EUVTexelDensityOpModes::ScaleGlobal;
			break;
		case EGeometryScriptTexelDensityMode::Normalize:
			Op.TexelDensityMode = EUVTexelDensityOpModes::Normalize;
			break;
		default:
			ensure(false);
			break;
		}


		TArray<int32> TriangleSelection;
		Selection.ConvertToMeshIndexArray(*SourceMesh, TriangleSelection, EGeometryScriptIndexType::Triangle);

		Op.TextureResolution = Options.TextureResolution;
		Op.TargetWorldSpaceMeasurement = Options.TargetWorldUnits;
		Op.TargetPixelCountMeasurement = Options.TargetPixelCount;

		Op.UVLayerIndex = UVSetIndex;
		Op.TextureResolution = Options.TextureResolution;
		Op.SetTransform(FTransformSRT3d::Identity());
		Op.bMaintainOriginatingUDIM = Options.bEnableUDIMLayout;
		if (TriangleSelection.Num() > 0)
		{
			Op.Selection = TSet<int32>(TriangleSelection);
		}
		if (Options.UDIMResolutions.Num() > 0)
		{
			Op.TextureResolutionPerUDIM = Options.UDIMResolutions;
		}

		Op.CalculateResult(nullptr);
		if (Op.GetResultInfo().Result == EGeometryResultType::Success)
		{
			TUniquePtr<FDynamicMesh3> ResultMesh = Op.ExtractResult();
			EditMesh = MoveTemp(*ResultMesh);
		}
		else
		{
			EditMesh = MoveTemp(*SourceMesh);
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyTexelDensityUVScaling_ComputeError", "ApplyTexelDensityUVScaling: Error computing result, returning input mesh"));
		}

	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::UVs, false);

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::RecomputeMeshUVs( 
	UDynamicMesh* TargetMesh, 
	int UVSetIndex,
	FGeometryScriptRecomputeUVsOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeMeshUVs_InvalidInput", "RecomputeMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		TUniquePtr<FPolygroupSet> IslandSourceGroups;
		if (Options.IslandSource == EGeometryScriptUVIslandSource::PolyGroups && Selection.IsEmpty())
		{
			FPolygroupLayer InputGroupLayer{ Options.GroupLayer.bDefaultLayer, Options.GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh))
			{

				IslandSourceGroups = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);
			}
			else
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("RecomputeMeshUVs_MissingGroups", "RecomputeMeshUVs: Requested Polygroup Layer does not exist"));
				return;
			}
		}

		// find group-connected-components
		FMeshConnectedComponents ConnectedComponents(&EditMesh);
		if (Selection.IsEmpty())
		{
			if (Options.IslandSource == EGeometryScriptUVIslandSource::PolyGroups)
			{
				ConnectedComponents.FindConnectedTriangles([&](int32 CurTri, int32 NbrTri) {
					return IslandSourceGroups->GetTriangleGroup(CurTri) == IslandSourceGroups->GetTriangleGroup(NbrTri);
				});
			}
			else
			{
				ConnectedComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
					return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
				});
			}
		}
		else
		{
			TArray<int32> TriangleROI;
			Selection.ConvertToMeshIndexArray(EditMesh, TriangleROI, EGeometryScriptIndexType::Triangle);
			ConnectedComponents.FindConnectedTriangles(TriangleROI, [&](int32 Triangle0, int32 Triangle1) {
				return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
			});
		}

		int32 NumComponents = ConnectedComponents.Num();
		TArray<bool> bComponentSolved;
		bComponentSolved.Init(false, NumComponents);
		int32 SuccessCount = 0;
		for (int32 k = 0; k < NumComponents; ++k)
		{
			const TArray<int32>& ComponentTris = ConnectedComponents[k].Indices;
			bComponentSolved[k] = false;
			switch (Options.Method)
			{
				case EGeometryScriptUVFlattenMethod::ExpMap:
				{
					FDynamicMeshUVEditor::FExpMapOptions ExpMapOptions;
					ExpMapOptions.NormalSmoothingRounds = Options.ExpMapOptions.NormalSmoothingRounds;
					ExpMapOptions.NormalSmoothingAlpha = Options.ExpMapOptions.NormalSmoothingAlpha;
					bComponentSolved[k] = UVEditor.SetTriangleUVsFromExpMap(ComponentTris, ExpMapOptions);
					break;
				}
				case EGeometryScriptUVFlattenMethod::Conformal:
				{
					bComponentSolved[k] = UVEditor.SetTriangleUVsFromFreeBoundaryConformal(ComponentTris);
					if ( bComponentSolved[k] )
					{
						UVEditor.ScaleUVAreaTo3DArea(ComponentTris, true);
					}
					break;
				}
				case EGeometryScriptUVFlattenMethod::SpectralConformal:
				{
					bComponentSolved[k] = UVEditor.SetTriangleUVsFromFreeBoundarySpectralConformal(ComponentTris, false, Options.SpectralConformalOptions.bPreserveIrregularity);
					if ( bComponentSolved[k] )
					{
						UVEditor.ScaleUVAreaTo3DArea(ComponentTris, true);
					}
					break;
				}
			}
		}

		if (Options.bAutoAlignIslandsWithAxes)
		{
			ParallelFor(NumComponents, [&](int32 k)
			{
				if (bComponentSolved[k])
				{
					const TArray<int32>& ComponentTris = ConnectedComponents[k].Indices;
					UVEditor.AutoOrientUVArea(ComponentTris);
				};
			});
		}

	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RecomputeMeshUVs_InvalidUVSet", "RecomputeMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::RepackMeshUVs( 
	UDynamicMesh* TargetMesh, 
	int UVSetIndex,
	FGeometryScriptRepackUVsOptions RepackOptions,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RepackMeshUVs_InvalidInput", "RepackMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		if (RepackOptions.bOptimizeIslandRotation)
		{
			FMeshConnectedComponents UVComponents(&EditMesh);
			UVComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
				return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
			});

			ParallelFor(UVComponents.Num(), [&](int32 k)
			{
				UVEditor.AutoOrientUVArea(UVComponents[k].Indices);
			});
		}

		UVEditor.QuickPack(FMath::Max(16, RepackOptions.TargetImageWidth));
	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RepackMeshUVs_InvalidUVSet", "RepackMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::LayoutMeshUVs(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FGeometryScriptLayoutUVsOptions LayoutOptions,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("LayoutMeshUVs_InvalidInput", "LayoutMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}


	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{

			TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(EditMesh));

			FUVLayoutOp Op;

			Op.OriginalMesh = SourceMesh;

			switch (LayoutOptions.LayoutType)
			{
			case EGeometryScriptUVLayoutType::Normalize:
				Op.UVLayoutMode = EUVLayoutOpLayoutModes::Normalize;
				break;
			case EGeometryScriptUVLayoutType::Repack:
				Op.UVLayoutMode = EUVLayoutOpLayoutModes::RepackToUnitRect;
				break;
			case EGeometryScriptUVLayoutType::Stack:
				Op.UVLayoutMode = EUVLayoutOpLayoutModes::StackInUnitRect;
				break;
			case EGeometryScriptUVLayoutType::Transform:
				Op.UVLayoutMode = EUVLayoutOpLayoutModes::TransformOnly;
				break;
			default:
				ensure(false);
				break;
			}


			TArray<int32> TriangleSelection;
			Selection.ConvertToMeshIndexArray(*SourceMesh, TriangleSelection, EGeometryScriptIndexType::Triangle);

			Op.TextureResolution = LayoutOptions.TextureResolution;
			Op.bPreserveScale = LayoutOptions.bPreserveScale;
			Op.bPreserveRotation = LayoutOptions.bPreserveRotation;
			Op.bAllowFlips = LayoutOptions.bAllowFlips;
			Op.UVScaleFactor = LayoutOptions.Scale;
			Op.UVTranslation = FVector2f(LayoutOptions.Translation);
			Op.bMaintainOriginatingUDIM = LayoutOptions.bEnableUDIMLayout;					
			Op.UVLayerIndex = UVSetIndex;
			Op.TextureResolution = LayoutOptions.TextureResolution;
			Op.SetTransform(FTransformSRT3d::Identity());
			Op.bMaintainOriginatingUDIM = LayoutOptions.bEnableUDIMLayout;
			if (TriangleSelection.Num() > 0)
			{
				Op.Selection = TSet<int32>(TriangleSelection);
			}
			if (LayoutOptions.UDIMResolutions.Num() > 0)
			{
				Op.TextureResolutionPerUDIM = LayoutOptions.UDIMResolutions;
			}

			Op.CalculateResult(nullptr);
			if (Op.GetResultInfo().Result == EGeometryResultType::Success)
			{
				TUniquePtr<FDynamicMesh3> ResultMesh = Op.ExtractResult();
				EditMesh = MoveTemp(*ResultMesh);
			}
			else
			{
				EditMesh = MoveTemp(*SourceMesh);
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("LayoutMeshUVs_ComputeError", "LayoutMeshUVs: Error computing result, returning input mesh"));
			}

		}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::UVs, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::AutoGeneratePatchBuilderMeshUVs( 
	UDynamicMesh* TargetMesh, 
	int UVSetIndex,
	FGeometryScriptPatchBuilderOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGeneratePatchBuilderMeshUVs_InvalidInput", "AutoGeneratePatchBuilderMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		FPatchBasedMeshUVGenerator UVGenerator;

		TUniquePtr<FPolygroupSet> PolygroupConstraint;
		if (Options.bRespectInputGroups)
		{
			FPolygroupLayer InputGroupLayer{ Options.GroupLayer.bDefaultLayer, Options.GroupLayer.ExtendedLayerIndex };
			if (InputGroupLayer.CheckExists(&EditMesh))
			{

				PolygroupConstraint = MakeUnique<FPolygroupSet>(&EditMesh, InputGroupLayer);
				UVGenerator.GroupConstraint = PolygroupConstraint.Get();
			}
			else
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGeneratePatchBuilderMeshUVs_MissingGruops", "AutoGeneratePatchBuilderMeshUVs: Requested Polygroup Layer does not exist"));
			}
		}

		UVGenerator.TargetPatchCount = FMath::Max(1,Options.InitialPatchCount);
		UVGenerator.bNormalWeightedPatches = true;
		UVGenerator.PatchNormalWeight = FMath::Clamp(Options.PatchCurvatureAlignmentWeight, 0.0, 999999.0);
		UVGenerator.MinPatchSize = FMath::Max(1,Options.MinPatchSize);

		UVGenerator.MergingThreshold = FMath::Clamp(Options.PatchMergingMetricThresh, 0.001, 9999.0);
		UVGenerator.MaxNormalDeviationDeg = FMath::Clamp(Options.PatchMergingAngleThresh, 0.0, 180.0);

		UVGenerator.NormalSmoothingRounds = FMath::Clamp(Options.ExpMapOptions.NormalSmoothingRounds, 0, 9999);
		UVGenerator.NormalSmoothingAlpha = FMath::Clamp(Options.ExpMapOptions.NormalSmoothingAlpha, 0.0, 1.0);

		UVGenerator.bAutoPack = Options.bAutoPack;
		if (Options.bAutoPack)
		{
			UVGenerator.bAutoAlignPatches = Options.PackingOptions.bOptimizeIslandRotation;
			UVGenerator.PackingTextureResolution = FMath::Clamp(Options.PackingOptions.TargetImageWidth, 16, 4096);
			UVGenerator.PackingGutterWidth = 1.0;
		}
		FGeometryResult Result = UVGenerator.AutoComputeUVs(*UVEditor.GetMesh(), *UVEditor.GetOverlay(), nullptr);

		if (Result.HasFailed())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AutoGeneratePatchBuilderMeshUVs_Failed", "AutoGeneratePatchBuilderMeshUVs: UV Generation Failed"));
		}

	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGeneratePatchBuilderMeshUVs_InvalidUVSet", "AutoGeneratePatchBuilderMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::AutoGenerateXAtlasMeshUVs( 
	UDynamicMesh* TargetMesh, 
	int UVSetIndex,
	FGeometryScriptXAtlasOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGenerateXAtlasMeshUVs_InvalidInput", "AutoGenerateXAtlasMeshUVs: TargetMesh is Null"));
		return TargetMesh;
	}

	bool bHasUVSet = false;
	ApplyMeshUVEditorOperation(TargetMesh, UVSetIndex, bHasUVSet, Debug,
		[&](FDynamicMesh3& EditMesh, FDynamicMeshUVOverlay* UVOverlay, FDynamicMeshUVEditor& UVEditor)
	{
		if (EditMesh.IsCompact() == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGenerateXAtlasMeshUVs_NonCompact", "AutoGenerateXAtlasMeshUVs: TargetMesh is non-Compact, XAtlas cannot be run. Try calling CompactMesh to update TargetMesh."));
			return;
		}

		const bool bFixOrientation = false;
		//const bool bFixOrientation = true;
		//FDynamicMesh3 FlippedMesh(EMeshComponents::FaceGroups);
		//FlippedMesh.Copy(Mesh, false, false, false, false);
		//if (bFixOrientation)
		//{
		//	FlippedMesh.ReverseOrientation(false);
		//}

		int32 NumVertices = EditMesh.VertexCount();
		TArray<FVector3f> VertexBuffer;
		VertexBuffer.SetNum(NumVertices);
		for (int32 k = 0; k < NumVertices; ++k)
		{
			VertexBuffer[k] = (FVector3f)EditMesh.GetVertex(k);
		}

		TArray<int32> IndexBuffer;
		IndexBuffer.Reserve(EditMesh.TriangleCount()*3);
		for (FIndex3i Triangle : EditMesh.TrianglesItr())
		{
			IndexBuffer.Add(Triangle.A);
			IndexBuffer.Add(Triangle.B);
			IndexBuffer.Add(Triangle.C);
		}

		TArray<FVector2D> UVVertexBuffer;
		TArray<int32>     UVIndexBuffer;
		TArray<int32>     VertexRemapArray; // This maps the UV vertices to the original position vertices.  Note multiple UV vertices might share the same positional vertex (due to UV boundaries)
		XAtlasWrapper::XAtlasChartOptions ChartOptions;
		ChartOptions.MaxIterations = Options.MaxIterations;
		XAtlasWrapper::XAtlasPackOptions PackOptions;
		bool bSuccess = XAtlasWrapper::ComputeUVs(IndexBuffer, VertexBuffer, ChartOptions, PackOptions,
			UVVertexBuffer, UVIndexBuffer, VertexRemapArray);
		if (bSuccess == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("AutoGenerateXAtlasMeshUVs_Failed", "AutoGenerateXAtlasMeshUVs: UV Generation Failed"));
			return;
		}

		UVOverlay->ClearElements();

		int32 NumUVs = UVVertexBuffer.Num();
		TArray<int32> UVOffsetToElID;  UVOffsetToElID.Reserve(NumUVs);
		for (int32 i = 0; i < NumUVs; ++i)
		{
			FVector2D UV = UVVertexBuffer[i];
			const int32 VertOffset = VertexRemapArray[i];		// The associated VertID in the dynamic mesh
			const int32 NewID = UVOverlay->AppendElement(FVector2f(UV));		// add the UV to the mesh overlay
			UVOffsetToElID.Add(NewID);
		}

		int32 NumUVTris = UVIndexBuffer.Num() / 3;
		for (int32 i = 0; i < NumUVTris; ++i)
		{
			int32 t = i * 3;
			FIndex3i UVTri(UVIndexBuffer[t], UVIndexBuffer[t + 1], UVIndexBuffer[t + 2]);	// The triangle in UV space
			FIndex3i TriVertIDs;				// the triangle in terms of the VertIDs in the DynamicMesh
			for (int c = 0; c < 3; ++c)
			{
				int32 Offset = VertexRemapArray[UVTri[c]];		// the offset for this vertex in the LinearMesh
				TriVertIDs[c] = Offset;
			}

			// NB: this could be slow.. 
			int32 TriID = EditMesh.FindTriangle(TriVertIDs[0], TriVertIDs[1], TriVertIDs[2]);
			if (TriID != IndexConstants::InvalidID)
			{
				FIndex3i ElTri = (bFixOrientation) ?
					FIndex3i(UVOffsetToElID[UVTri[1]], UVOffsetToElID[UVTri[0]], UVOffsetToElID[UVTri[2]])
					: FIndex3i(UVOffsetToElID[UVTri[0]], UVOffsetToElID[UVTri[1]], UVOffsetToElID[UVTri[2]]);
				UVOverlay->SetTriangle(TriID, ElTri);
			}
		}

	});
	if (bHasUVSet == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("AutoGenerateXAtlasMeshUVs_InvalidUVSet", "AutoGenerateXAtlasMeshUVs: UVSetIndex does not exist on TargetMesh"));
	}

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::GetMeshUVSizeInfo(
	UDynamicMesh* TargetMesh,
	int UVSetIndex,
	FGeometryScriptMeshSelection Selection,
	double& MeshArea,
	double& UVArea,
	FBox& MeshBounds,
	FBox2D& UVBounds,
	bool& bIsValidUVSet,
	bool& bFoundUnsetUVs,
	bool bOnlyIncludeValidUVTris,
	UGeometryScriptDebug* Debug)
{
	MeshArea = UVArea = 0;
	bIsValidUVSet = false;
	bFoundUnsetUVs = false;
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshUVSizeInfo_InvalidInput", "GetMeshUVSizeInfo: TargetMesh is Null"));
		return TargetMesh;
	}
	FAxisAlignedBox3d MeshBoundsTmp = FAxisAlignedBox3d::Empty();
	FAxisAlignedBox2f UVBoundsTmp = FAxisAlignedBox2f::Empty();
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		const FDynamicMeshUVOverlay* UVOverlay = nullptr;
		if (ReadMesh.HasAttributes())
		{
			UVOverlay = (UVSetIndex >= 0 && UVSetIndex < ReadMesh.Attributes()->NumUVLayers()) ? ReadMesh.Attributes()->GetUVLayer(UVSetIndex) : nullptr;
		}
		if (UVOverlay == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshUVSizeInfo_InvalidUVSet", "GetMeshUVSizeInfo: UV Set does not Exist"));
			return;
		}
		bIsValidUVSet = true;

		Selection.ProcessByTriangleID(ReadMesh, [&](int32 TriangleID) { 
		
			bool bTriangleHasUVs = (UVOverlay->IsSetTriangle(TriangleID));
			if (!bTriangleHasUVs)
			{
				bFoundUnsetUVs = true;
			}

			FVector3d Vertices[3];
			FVector2f UVs[3];
			if (bTriangleHasUVs || bOnlyIncludeValidUVTris == false)
			{
				ReadMesh.GetTriVertices(TriangleID, Vertices[0], Vertices[1], Vertices[2]);
				MeshArea += VectorUtil::Area(Vertices[0], Vertices[1], Vertices[2]);
				MeshBoundsTmp.Contain(Vertices[0]); MeshBoundsTmp.Contain(Vertices[1]); MeshBoundsTmp.Contain(Vertices[2]);
			}
			if (bTriangleHasUVs)
			{
				UVOverlay->GetTriElements(TriangleID, UVs[0], UVs[1], UVs[2]);
				UVArea += (double)VectorUtil::Area(UVs[0], UVs[1], UVs[2]);
				UVBoundsTmp.Contain(UVs[0]); UVBoundsTmp.Contain(UVs[1]); UVBoundsTmp.Contain(UVs[2]);
			}
		
		}, true);
	});

	MeshBounds = (FBox)MeshBoundsTmp;
	UVBounds = (FBox2D)UVBoundsTmp;
	
	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::GetMeshPerVertexUVs(
	UDynamicMesh* TargetMesh, 
	int UVSetIndex,
	FGeometryScriptUVList& UVList, 
	bool& bIsValidUVSet,
	bool& bHasVertexIDGaps,
	bool& bHasSplitUVs,
	UGeometryScriptDebug* Debug)
{
	UVList.Reset();
	TArray<FVector2D>& UVs = *UVList.List;
	bHasVertexIDGaps = false;
	bIsValidUVSet = false;
	bHasSplitUVs = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			const FDynamicMeshUVOverlay* UVOverlay = (ReadMesh.HasAttributes() && UVSetIndex < ReadMesh.Attributes()->NumUVLayers()) ?
				ReadMesh.Attributes()->GetUVLayer(UVSetIndex) : nullptr;
			if (UVOverlay == nullptr)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshPerVertexUVs_InvalidUVSet", "GetMeshPerVertexUVs: UVSetIndex does not exist on TargetMesh"));
				return;
			}

			bHasVertexIDGaps = ! ReadMesh.IsCompactV();

			UVs.Init(FVector2D::Zero(), ReadMesh.MaxVertexID());
			TArray<int32> ElemIndex;		// set to elementID of first element seen at each vertex, if we see a second element ID, it is a split vertex
			ElemIndex.Init(-1, UVs.Num());

			for (int32 tid : ReadMesh.TriangleIndicesItr())
			{
				if (UVOverlay->IsSetTriangle(tid))
				{
					FIndex3i TriV = ReadMesh.GetTriangle(tid);
					FIndex3i TriE = UVOverlay->GetTriangle(tid);
					for (int j = 0; j < 3; ++j)
					{
						if (ElemIndex[TriV[j]] == -1)
						{
							UVs[TriV[j]] = (FVector2D)UVOverlay->GetElement(TriE[j]);
							ElemIndex[TriV[j]] = TriE[j];
						}
						else if (ElemIndex[TriV[j]] != TriE[j])
						{
							bHasSplitUVs = true;
						}
					}
				}
			}

			bIsValidUVSet = true;
		});
	}

	return TargetMesh;
}





UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::CopyMeshUVLayerToMesh(
	UDynamicMesh* CopyFromMesh,
	int UVSetIndex,
	UPARAM(DisplayName = "Copy To UV Mesh", ref) UDynamicMesh* CopyToUVMesh,
	UPARAM(DisplayName = "Copy To UV Mesh") UDynamicMesh*& CopyToUVMeshOut,
	bool& bInvalidTopology,
	bool& bIsValidUVSet,
	UGeometryScriptDebug* Debug)
{
	if (CopyFromMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshUVLayerToMesh_InvalidInput", "CopyMeshUVLayerToMesh: CopyFromMesh is Null"));
		return CopyFromMesh;
	}
	if (CopyToUVMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshUVLayerToMesh_InvalidInput2", "CopyMeshUVLayerToMesh: CopyToUVMesh is Null"));
		return CopyFromMesh;
	}
	if (CopyFromMesh == CopyToUVMesh)
	{
		// TODO: can actually support this but complicates the code below...
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToUVMesh_SameMeshes", "CopyMeshUVLayerToMesh: CopyFromMesh and CopyToUVMesh are the same mesh, this is not supported"));
		return CopyFromMesh;
	}

	FDynamicMesh3 UVMesh;
	bIsValidUVSet = false;
	bInvalidTopology = false;
	CopyFromMesh->ProcessMesh([&](const FDynamicMesh3& FromMesh)
	{
		const FDynamicMeshUVOverlay* UVOverlay = (FromMesh.HasAttributes() && UVSetIndex < FromMesh.Attributes()->NumUVLayers()) ?
			FromMesh.Attributes()->GetUVLayer(UVSetIndex) : nullptr;
		if (UVOverlay == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToUVMesh_InvalidUVSet", "CopyMeshUVLayerToMesh: UVSetIndex does not exist on CopyFromMesh"));
			return;
		}
		bIsValidUVSet = true;

		UVMesh.EnableTriangleGroups();
		UVMesh.EnableAttributes();
		UVMesh.Attributes()->SetNumUVLayers(0);

		const FDynamicMeshMaterialAttribute* FromMaterialID = (FromMesh.HasAttributes() && FromMesh.Attributes()->HasMaterialID()) ?
			FromMesh.Attributes()->GetMaterialID() : nullptr;
		FDynamicMeshMaterialAttribute* ToMaterialID = nullptr;
		if (FromMaterialID)
		{
			UVMesh.Attributes()->EnableMaterialID();
			ToMaterialID = UVMesh.Attributes()->GetMaterialID();
		}

		UVMesh.BeginUnsafeVerticesInsert();
		for (int32 elemid : UVOverlay->ElementIndicesItr())
		{
			FVector2f UV = UVOverlay->GetElement(elemid);
			UVMesh.InsertVertex(elemid, FVector3d(UV.X, UV.Y, 0), true);
		}
		UVMesh.EndUnsafeVerticesInsert();
		UVMesh.BeginUnsafeTrianglesInsert();
		for (int32 tid : FromMesh.TriangleIndicesItr())
		{
			FIndex3i UVTri = UVOverlay->GetTriangle(tid);
			int32 GroupID = FromMesh.GetTriangleGroup(tid);
			EMeshResult Result = UVMesh.InsertTriangle(tid, UVTri, GroupID, true);
			if (Result != EMeshResult::Ok)
			{
				bInvalidTopology = true;
			}
			else
			{
				if (FromMaterialID)
				{
					ToMaterialID->SetValue(tid, FromMaterialID->GetValue(tid));		// could we use Copy() here ?
				}
			}
		}
		UVMesh.EndUnsafeTrianglesInsert();
	});

	FMeshNormals::InitializeOverlayToPerVertexNormals(UVMesh.Attributes()->PrimaryNormals());

	CopyToUVMesh->SetMesh(MoveTemp(UVMesh));
	CopyToUVMeshOut = CopyToUVMesh;

	return CopyFromMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::CopyMeshToMeshUVLayer(
	UDynamicMesh* CopyFromUVMesh,
	int ToUVSetIndex,
	UDynamicMesh* CopyToMesh,
	UDynamicMesh*& CopyToMeshOut,
	bool& bFoundTopologyErrors,
	bool& bIsValidUVSet,
	bool bOnlyUVPositions,
	UGeometryScriptDebug* Debug)
{
	if (CopyFromUVMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMeshUVLayer_InvalidInput", "CopyMeshToMeshUVLayer: CopyFromUVMesh is Null"));
		return CopyFromUVMesh;
	}
	if (CopyToMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMeshUVLayer_InvalidInput2", "CopyMeshToMeshUVLayer: CopyToUVMesh is Null"));
		return CopyFromUVMesh;
	}
	if (CopyFromUVMesh == CopyToMesh)
	{
		// TODO: can actually support this but complicates the code below...
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMeshUVLayer_SameMeshes", "CopyMeshToMeshUVLayer: CopyFromUVMesh and CopyToMesh are the same mesh, this is not supported"));
		return CopyFromUVMesh;
	}

	bFoundTopologyErrors = false;
	bIsValidUVSet = false;
	CopyToMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshUVOverlay* UVOverlay = (EditMesh.HasAttributes() && ToUVSetIndex < EditMesh.Attributes()->NumUVLayers()) ?
			EditMesh.Attributes()->GetUVLayer(ToUVSetIndex) : nullptr;
		if (UVOverlay == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshToMeshUVLayer_InvalidUVSet", "CopyMeshToMeshUVLayer: ToUVSetIndex does not exist on CopyFromMesh"));
			return;
		}
		bIsValidUVSet = true;

		CopyFromUVMesh->ProcessMesh([&](const FDynamicMesh3& UVMesh)
		{
			if (bOnlyUVPositions)
			{
				if ( UVMesh.MaxVertexID() <= UVOverlay->MaxElementID() )
				{
					for (int32 vid : UVMesh.VertexIndicesItr())
					{
						if (UVOverlay->IsElement(vid))
						{
							FVector3d Pos = UVMesh.GetVertex(vid);
							UVOverlay->SetElement(vid, FVector2f((float)Pos.X, (float)Pos.Y));
						}
						else
						{
							bFoundTopologyErrors = true;
						}
					}
				}
				else
				{
					bFoundTopologyErrors = true;
				}
			}
			else
			{
				if (UVMesh.MaxTriangleID() <= EditMesh.MaxTriangleID())
				{
					UVOverlay->ClearElements();
					UVOverlay->BeginUnsafeElementsInsert();
					for (int32 vid : UVMesh.VertexIndicesItr())
					{
						FVector3d Pos = UVMesh.GetVertex(vid);
						FVector2f UV((float)Pos.X, (float)Pos.Y);
						UVOverlay->InsertElement(vid, &UV.X, true);
					}
					UVOverlay->EndUnsafeElementsInsert();
					for (int32 tid : UVMesh.TriangleIndicesItr())
					{
						if (EditMesh.IsTriangle(tid))
						{
							FIndex3i Tri = UVMesh.GetTriangle(tid);
							UVOverlay->SetTriangle(tid, Tri);
						}
						else
						{
							bFoundTopologyErrors = true;
						}
					}
				}
				else
				{
					bFoundTopologyErrors = true;
				}
			}
		});
	});


	CopyToMeshOut = CopyToMesh;
	return CopyFromUVMesh;
}



UDynamicMesh* UGeometryScriptLibrary_MeshUVFunctions::ComputeMeshLocalUVParam(
	UDynamicMesh* TargetMesh,
	FVector CenterPoint,
	int32 CenterPointTriangleID,
	TArray<int>& VertexIDs,
	TArray<FVector2D>& VertexUVs,
	double Radius,
	bool bUseInterpolatedNormal,
	FVector TangentYDirection,
	double UVRotationDeg,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshLocalUVParam_InvalidInput", "ComputeMeshLocalUVParam: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		if (Mesh.IsTriangle(CenterPointTriangleID) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ComputeMeshLocalUVParam_InvalidTriangle", "ComputeMeshLocalUVParam: CenterPointTriangleID is not a valid triangle"));
			return;
		}
		FIndex3i TriVertices = Mesh.GetTriangle(CenterPointTriangleID);
		FFrame3d SeedFrame(Mesh.GetTriCentroid(CenterPointTriangleID), Mesh.GetTriNormal(CenterPointTriangleID));
		SeedFrame.Origin = CenterPoint;

		if (bUseInterpolatedNormal && Mesh.HasAttributes())
		{
			if (const FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals())
			{
				if (Normals->IsSetTriangle(CenterPointTriangleID))
				{
					FTriangle3d Triangle(Mesh.GetVertex(TriVertices.A), Mesh.GetVertex(TriVertices.B), Mesh.GetVertex(TriVertices.C));
					FVector3d BaryCoords = Triangle.GetBarycentricCoords(SeedFrame.Origin);
					FVector3d InterpNormal;
					Mesh.Attributes()->PrimaryNormals()->GetTriBaryInterpolate<double>(CenterPointTriangleID, &BaryCoords.X, &InterpNormal.X);
					SeedFrame.AlignAxis(2, InterpNormal);
				}
			}
		}

		if (TangentYDirection.SquaredLength() > 0.1)
		{
			SeedFrame.ConstrainedAlignAxis(1, TangentYDirection, SeedFrame.Z());
		}
		else
		{
			SeedFrame.ConstrainedAlignAxis(1, FVector3d::UnitZ(), SeedFrame.Z());
		}

		if (UVRotationDeg != 0)
		{
			SeedFrame.Rotate(FQuaterniond(SeedFrame.Z(), UVRotationDeg, true));
		}

		TMeshLocalParam<FDynamicMesh3> Param(&Mesh);
		Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;

		Param.bEnableExternalNormals = true;
		Param.ExternalNormalFunc = [&](int32 VertexID) { return FMeshNormals::ComputeVertexNormal(Mesh, VertexID); };

		Param.ComputeToMaxDistance(SeedFrame, TriVertices, Radius * FMathd::Sqrt2);

		Param.GetAllComputedUVs(VertexIDs, VertexUVs, Radius, Radius * 2.0);
	});

	return TargetMesh;
}


bool UGeometryScriptLibrary_MeshUVFunctions::IntersectsUVBox2D(FBox2D A, FBox2D B, bool bWrappedToUnitRange)
{
	if (!A.bIsValid || !B.bIsValid)
	{
		return false;
	}
	if (!bWrappedToUnitRange)
	{
		return A.Intersect(B);
	}
	else
	{
		// Wrap the min value to [0,1] and shift the max value to have the same offset (without wrapping)
		auto WrapMin = [](double& MinV, double& MaxV) -> void
		{
			double WrapMinV = FMath::Wrap(MinV, 0., 1.);
			// Offset MinV and MaxV by (WrapMinV-MinV), so MinV becomes wrapped and MaxV keeps the same distance
			MaxV += WrapMinV - MinV;
			MinV = WrapMinV;
		};
		// Test for overlap in a single dimension, with wrapping
		auto Overlaps1DWrapped = [](double AMin, double AMax, double BMin, double BMax) -> bool
		{
			// either value covers the full range, must overlap
			if (AMax-AMin >= 1 || BMax - BMin >= 1)
			{
				return true;
			}
			if (BMax > 1)
			{
				if (AMax > 1)
				{
					// both wrap
					return true;
				}
				// only range B wraps, convert to the only-range-A-wraps case
				Swap(AMin, BMin);
				Swap(AMax, BMax);
			}
			if (AMax > 1)
			{
				AMax -= 1.;
				return BMin <= AMax || BMax >= AMin;
			}
			else
			{
				return !(AMax < BMin || AMin > BMax);
			}
		};
		WrapMin(A.Min.X, A.Max.X);
		WrapMin(B.Min.X, B.Max.X);
		if (Overlaps1DWrapped(A.Min.X, A.Max.X, B.Min.X, B.Max.X))
		{
			WrapMin(A.Min.Y, A.Max.Y);
			WrapMin(B.Min.Y, B.Max.Y);
			return Overlaps1DWrapped(A.Min.Y, A.Max.Y, B.Min.Y, B.Max.Y);
		}
		return false;
	}
}

#undef LOCTEXT_NAMESPACE