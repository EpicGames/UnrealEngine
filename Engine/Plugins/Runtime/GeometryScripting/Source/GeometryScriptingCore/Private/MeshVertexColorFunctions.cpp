// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshVertexColorFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Util/ColorConstants.h"
#include "Operations/SmoothDynamicMeshAttributes.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "Operations/TransferDynamicMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexColorFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshVertexColorFunctions"


static FLinearColor CombineColors(FLinearColor ExistingColor, FLinearColor NewColor, FGeometryScriptColorFlags Flags)
{
	ExistingColor.R = (Flags.bRed) ? NewColor.R : ExistingColor.R;
	ExistingColor.G = (Flags.bGreen) ? NewColor.G : ExistingColor.G;
	ExistingColor.B = (Flags.bBlue) ? NewColor.B : ExistingColor.B;
	ExistingColor.A = (Flags.bAlpha) ? NewColor.A : ExistingColor.A;
	return ExistingColor;
}

UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::SetMeshConstantVertexColor(
	UDynamicMesh* TargetMesh,
	FLinearColor Color,
	FGeometryScriptColorFlags Flags,
	bool bClearExisting,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshConstantVertexColor_InvalidInput", "SetMeshConstantVertexColor: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		bool bCreated = false;
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
			bCreated = true;
		}
		if (EditMesh.Attributes()->HasPrimaryColors() == false)
		{
			EditMesh.Attributes()->EnablePrimaryColors();
			bCreated = true;
		}
		FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
		if (bClearExisting && bCreated == false)
		{
			Colors->ClearElements();
		}
		if (Colors->ElementCount() == 0)
		{
			TArray<int32> ElemIDs;
			ElemIDs.SetNum(EditMesh.MaxVertexID());
			for (int32 VertexID : EditMesh.VertexIndicesItr())
			{
				ElemIDs[VertexID] = Colors->AppendElement(FVector4f(Color.R, Color.G, Color.B, Color.A));
			}
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
				Colors->SetTriangle(TriangleID, FIndex3i(ElemIDs[Triangle.A], ElemIDs[Triangle.B], ElemIDs[Triangle.C]) );
			}
		}
		else
		{
			for (int32 ElementID : Colors->ElementIndicesItr())
			{
				FLinearColor Existing = ToLinearColor(Colors->GetElement(ElementID));
				FLinearColor NewColor = CombineColors(Existing, Color, Flags);
				Colors->SetElement(ElementID, ToVector4<float>(NewColor));
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::SetMeshSelectionVertexColor(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	FLinearColor Color,
	FGeometryScriptColorFlags Flags,
	bool bCreateColorSeam,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshSelectionVertexColor_InvalidInput", "SetMeshSelectionVertexColor: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		bool bCreated = false;
		if (EditMesh.HasAttributes() == false)
		{
			EditMesh.EnableAttributes();
			bCreated = true;
		}
		if (EditMesh.Attributes()->HasPrimaryColors() == false)
		{
			EditMesh.Attributes()->EnablePrimaryColors();
			bCreated = true;
		}
		FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();

		// If we created a new color overlay, we are going to initialize the vertex colors to white.
		if (bCreated)
		{
			TArray<int32> ElemIDs;
			ElemIDs.SetNum(EditMesh.MaxVertexID());
			for (int32 VertexID : EditMesh.VertexIndicesItr())
			{
				ElemIDs[VertexID] = Colors->AppendElement(FVector4f::One());
			}
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
				Colors->SetTriangle(TriangleID, FIndex3i(ElemIDs[Triangle.A], ElemIDs[Triangle.B], ElemIDs[Triangle.C]));
			}
		}

		if (bCreateColorSeam)
		{
			FLinearColor NewColor = CombineColors(FLinearColor::Black, Color, Flags);

			TArray<int32> Triangles;
			Selection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);
			TMap<int32, int32> VerticesToElements;
			for (int32 TriangleID : Triangles)
			{
				FIndex3i TriVerts = EditMesh.GetTriangle(TriangleID);
				for (int32 j = 0; j < 3; ++j)
				{
					const int32* FoundElemID = VerticesToElements.Find(TriVerts[j]);
					if (FoundElemID == nullptr)
					{
						int32 NewElemID = Colors->AppendElement(FVector4f(NewColor.R, NewColor.G, NewColor.B, NewColor.A));
						VerticesToElements.Add(TriVerts[j], NewElemID);
					}
				}
			}

			for (int32 TriangleID : Triangles)
			{
				FIndex3i TriVerts = EditMesh.GetTriangle(TriangleID);
				TriVerts[0] = VerticesToElements[TriVerts[0]];
				TriVerts[1] = VerticesToElements[TriVerts[1]];
				TriVerts[2] = VerticesToElements[TriVerts[2]];
				Colors->SetTriangle(TriangleID, TriVerts);
			}
		}
		else if (Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Vertices || Selection.GetSelectionType() == EGeometryScriptMeshSelectionType::Edges)
		{
			Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID)
			{
				EditMesh.EnumerateVertexTriangles(VertexID, [&](int32 TriangleID)
				{
					if (Colors->IsSetTriangle(TriangleID))
					{
						FIndex3i TriElems = Colors->GetTriangle(TriangleID);
						for (int32 j = 0; j < 3; ++j)
						{
							if (Colors->GetParentVertex(TriElems[j]) == VertexID)
							{
								FLinearColor Existing = ToLinearColor(Colors->GetElement(TriElems[j]));
								FLinearColor NewColor = CombineColors(Existing, Color, Flags);
								Colors->SetElement(TriElems[j], ToVector4<float>(NewColor));
							}
						}
					}
				});
			});
		}
		else
		{
			Selection.ProcessByTriangleID(EditMesh, [&](int32 TriangleID)
			{
				if (Colors->IsSetTriangle(TriangleID))
				{
					FIndex3i TriElems = Colors->GetTriangle(TriangleID);
					for (int32 j = 0; j < 3; ++j)
					{
						FLinearColor Existing = ToLinearColor(Colors->GetElement(TriElems[j]));
						FLinearColor NewColor = CombineColors(Existing, Color, Flags);
						Colors->SetElement(TriElems[j], ToVector4<float>(NewColor));
					}
				}
			});
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::SetMeshPerVertexColors(
	UDynamicMesh* TargetMesh,
	FGeometryScriptColorList VertexColorList,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexColors_InvalidMesh", "SetMeshPerVertexColors: TargetMesh is Null"));
		return TargetMesh;
	}
	if (VertexColorList.List.IsValid() == false || VertexColorList.List->Num() == 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexColors_InvalidList", "SetMeshPerVertexColors: List is empty"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		const TArray<FLinearColor>& VertexColors = *VertexColorList.List;
		if (VertexColors.Num() < EditMesh.MaxVertexID())
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshPerVertexColors_IncorrectCount", "SetMeshPerVertexColors: size of provided VertexColorList is smaller than MaxVertexID of Mesh"));
		}
		else
		{
			if (EditMesh.HasAttributes() == false)
			{
				EditMesh.EnableAttributes();
			}
			if (EditMesh.Attributes()->HasPrimaryColors() == false)
			{
				EditMesh.Attributes()->EnablePrimaryColors();
			}
			FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
			Colors->ClearElements();
			TArray<int32> ElemIDs;
			ElemIDs.SetNum(EditMesh.MaxVertexID());
			for (int32 VertexID : EditMesh.VertexIndicesItr())
			{
				//const FLinearColor& FromColor = VertexColors[VertexID];
				//FColor SRGBFColor = FromColor.ToFColorSRGB();
				//FLinearColor Color = SRGBFColor.ReinterpretAsLinear();
				const FLinearColor& Color = VertexColors[VertexID];
				ElemIDs[VertexID] = Colors->AppendElement(FVector4f(Color.R, Color.G, Color.B, Color.A));
			}
			for (int32 TriangleID : EditMesh.TriangleIndicesItr())
			{
				FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
				Colors->SetTriangle(TriangleID, FIndex3i(ElemIDs[Triangle.A], ElemIDs[Triangle.B], ElemIDs[Triangle.C]) );
			}
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}








UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::GetMeshPerVertexColors(
	UDynamicMesh* TargetMesh, 
	FGeometryScriptColorList& ColorList, 
	bool& bIsValidColorSet,
	bool& bHasVertexIDGaps,
	bool bBlendSplitVertexValues)
{
	ColorList.Reset();
	TArray<FLinearColor>& Colors = *ColorList.List;
	bHasVertexIDGaps = false;
	bIsValidColorSet = false;
	if (TargetMesh)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			Colors.Init(FLinearColor::Transparent, ReadMesh.MaxVertexID());
			bHasVertexIDGaps = ! ReadMesh.IsCompactV();

			if (ReadMesh.HasAttributes() && ReadMesh.Attributes()->HasPrimaryColors() )
			{
				const FDynamicMeshColorOverlay* ColorOverlay = ReadMesh.Attributes()->PrimaryColors();

				if (bBlendSplitVertexValues)
				{
					TArray<int32> ColorCounts;
					ColorCounts.Init(0, ReadMesh.MaxVertexID());
					for (int32 tid : ReadMesh.TriangleIndicesItr())
					{
						if (ColorOverlay->IsSetTriangle(tid))
						{
							FIndex3i TriV = ReadMesh.GetTriangle(tid);
							FVector4f A, B, C;
							ColorOverlay->GetTriElements(tid, A, B, C);
							Colors[TriV.A] += ToLinearColor(A);
							ColorCounts[TriV.A]++;
							Colors[TriV.B] += ToLinearColor(B);
							ColorCounts[TriV.B]++;
							Colors[TriV.C] += ToLinearColor(C);
							ColorCounts[TriV.C]++;
						}
					}

					for (int32 k = 0; k < ColorCounts.Num(); ++k)
					{
						if (ColorCounts[k] > 1)
						{
							Colors[k] *= 1.0f / (float)ColorCounts[k];
						}
					}
				}
				else
				{
					for (int32 tid : ReadMesh.TriangleIndicesItr())
					{
						if (ColorOverlay->IsSetTriangle(tid))
						{
							FIndex3i TriV = ReadMesh.GetTriangle(tid);
							FVector4f A, B, C;
							ColorOverlay->GetTriElements(tid, A, B, C);
							Colors[TriV.A] = ToLinearColor(A);
							Colors[TriV.B] = ToLinearColor(B);
							Colors[TriV.C] = ToLinearColor(C);
						}
					}
				}
				
				bIsValidColorSet = true;
			}
		});
	}

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::ConvertMeshVertexColorsSRGBToLinear(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertMeshVertexColorsSRGBToLinear_InvalidInput", "ConvertMeshVertexColorsSRGBToLinear: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || EditMesh.Attributes()->HasPrimaryColors() == false)
		{
			return;
		}

		FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
		if (Colors->ElementCount() > 0)
		{
			for (int32 ElementID : Colors->ElementIndicesItr())
			{
				FVector4f Existing = Colors->GetElement(ElementID);
				LinearColors::SRGBToLinear(Existing);
				Colors->SetElement(ElementID, Existing);
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::ConvertMeshVertexColorsLinearToSRGB(
	UDynamicMesh* TargetMesh,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ConvertMeshVertexColorsLinearToSRGB_InvalidInput", "ConvertMeshVertexColorsLinearToSRGB: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || EditMesh.Attributes()->HasPrimaryColors() == false)
		{
			return;
		}

		FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
		if (Colors->ElementCount() > 0)
		{
			for (int32 ElementID : Colors->ElementIndicesItr())
			{
				FVector4f Existing = Colors->GetElement(ElementID);
				LinearColors::LinearToSRGB(Existing);
				Colors->SetElement(ElementID, Existing);
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::BlurMeshVertexColors(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMeshSelection Selection,
	int NumIterations,
	double Strength,
	EGeometryScriptBlurColorMode BlurMode,
	FGeometryScriptBlurMeshVertexColorsOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BlurMeshVertexColors_InvalidInput", "BlurMeshVertexColors: TargetMesh is Null."));
		return TargetMesh;
	}

	if (NumIterations < 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BlurMeshVertexColors_InvalidIterationNumber", "BlurMeshVertexColors: Number of iterations must be non-negative."));
		return TargetMesh;
	}

	if (Strength < 0)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("BlurMeshVertexColors_InvalidStrength", "BlurMeshVertexColors: Blur strength must be non-negative."));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		if (EditMesh.HasAttributes() == false || EditMesh.Attributes()->HasPrimaryColors() == false)
		{
			return;
		}

		FDynamicMeshColorOverlay* Colors = EditMesh.Attributes()->PrimaryColors();
		if (Colors->ElementCount() > 0)
		{
			FSmoothDynamicMeshAttributes BlurOp(EditMesh);
			BlurOp.bUseParallel = true;
			BlurOp.NumIterations = NumIterations;
			BlurOp.Strength = Strength;
			BlurOp.EdgeWeightMethod = static_cast<FSmoothDynamicMeshAttributes::EEdgeWeights>(BlurMode);

			TArray<bool> ColorsToSmooth;
			ColorsToSmooth.Reserve(4);
			ColorsToSmooth.Add(Options.Red);
			ColorsToSmooth.Add(Options.Green);
			ColorsToSmooth.Add(Options.Blue);
			ColorsToSmooth.Add(Options.Alpha);

			if (!Selection.IsEmpty())
			{
				Selection.ProcessByVertexID(EditMesh, [&](int32 VertexID)
				{
					BlurOp.Selection.Add(VertexID);
				});
			}

			if (!BlurOp.SmoothOverlay(Colors, ColorsToSmooth))
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("BlurMeshVertexColors_BlurFailed", "BlurMeshVertexColors: Failed to blur the colors."));
				return;
			}
		}
	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshVertexColorFunctions::TransferVertexColorsFromMesh(
	UDynamicMesh* SourceMesh,
	UDynamicMesh* TargetMesh,
	FGeometryScriptTransferMeshVertexColorsOptions Options,
	FGeometryScriptMeshSelection Selection,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (SourceMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferVertexColorsFromMesh_InvalidSourceMesh", "TransferVertexColorsFromMesh: Source Mesh is Null"));
		return TargetMesh;
	}
	if (TargetMesh == nullptr)
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferVertexColorsFromMesh_InvalidTargetMesh", "TransferVertexColorsFromMesh: Target Mesh is Null"));
		return TargetMesh;
	}

	SourceMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		if (!ReadMesh.HasAttributes() || !ReadMesh.Attributes()->HasPrimaryColors())
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("TransferVertexColorsFromMesh_NoColor", "Source Mesh has no vertex color attribute"));
			return;
		}

		FTransferVertexColorAttribute TransferColors(&ReadMesh);
		TransferColors.TransferMethod = static_cast<FTransferVertexColorAttribute::ETransferMethod>(Options.TransferMethod);
		TransferColors.bUseParallel = true;
		TransferColors.bHardEdges = Options.bHardEdges;
		TransferColors.BiasRatio = Options.BiasRatio;

		TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (!Selection.IsEmpty())
			{
				Selection.ConvertToMeshIndexArray(EditMesh, TransferColors.TargetVerticesSubset, EGeometryScriptIndexType::Vertex);
			}
			
			if (!EditMesh.HasAttributes())
			{
				EditMesh.EnableAttributes();
			}
			
			if (Options.TransferMethod == ETransferVertexColorMethod::Inpaint)
			{
				TransferColors.NormalThreshold = FMathd::DegToRad * Options.NormalThreshold;
				TransferColors.SearchRadius = Options.RadiusPercentage * EditMesh.GetBounds().DiagonalLength();
				TransferColors.NumSmoothingIterations = Options.NumSmoothingIterations;
				TransferColors.SmoothingStrength = Options.SmoothingStrength;
				TransferColors.LayeredMeshSupport = Options.LayeredMeshSupport;
			}

			if (TransferColors.Validate() != EOperationValidationResult::Ok)
			{
				AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("TransferVertexColorsFromMesh_ValidationFailed", "TransferVertexColorsFromMesh: Invalid parameters were set for the transfer colors operator"));
				return;
			}
			
			if (!TransferColors.TransferColorsToMesh(EditMesh))
			{
				AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("TransferVertexColorsFromMesh_TransferFailed", "TransferVertexColorsFromMesh: Failed to transfer the colors"));
			}
			
		}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	});

	return TargetMesh;
}



#undef LOCTEXT_NAMESPACE