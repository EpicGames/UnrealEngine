// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshVoxelFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "Implicit/Solidify.h"
#include "Implicit/Morphology.h"
#include "Operations/ExtrudeMesh.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVoxelFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshVoxelFunctions"


static TAutoConsoleVariable<bool> CVarGeometryScriptUnclampedGridDimensions(
	TEXT("geometry.GeometryScript.AllowUnclampedGridDimensions"),
	false,
	TEXT("Flag to disable grid dimension clamping in GeometryScript grid-based operations"));



static int32 GetClampedGridResolution(int32 TargetResolution, FString ToolName, UGeometryScriptDebug* Debug)
{
	int ClampResolution = FMath::Max(4, TargetResolution);
	if (ClampResolution > 256 && CVarGeometryScriptUnclampedGridDimensions.GetValueOnGameThread() == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs,
			FText::Format(LOCTEXT("ClampedGridResolution", "{0}: Requested GridResolution clamped to 256 to avoid significant memory/computation. Use CVar geometry.GeometryScript.AllowUnclampedGridDimensions to disable this limit."), FText::FromString(ToolName)));
		ClampResolution = 256;
	}
	return ClampResolution;
}

static double GetClampedCellSize(double TargetCellSize, FAxisAlignedBox3d BoundingBox, FString ToolName, UGeometryScriptDebug* Debug)
{
	double UseCellSize = FMath::Max(0.001, TargetCellSize);
	int MaxGridDimEstimate = (int32)(BoundingBox.MaxDim() / UseCellSize);
	if (MaxGridDimEstimate > 256 && CVarGeometryScriptUnclampedGridDimensions.GetValueOnGameThread() == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, 
			FText::Format(LOCTEXT("ClampedCellSize", "{0}: Requested GridCellSize will require significant memory/computation, clamping to 256 cells. Use CVar geometry.GeometryScript.AllowUnclampedGridDimensions to disable this limit."), FText::FromString(ToolName)));
		UseCellSize = (float)BoundingBox.MaxDim() / 256.0;
	}
	return UseCellSize;
}







UDynamicMesh* UGeometryScriptLibrary_MeshVoxelFunctions::ApplyMeshSolidify(
	UDynamicMesh* TargetMesh,
	FGeometryScriptSolidifyOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshSolidify_InvalidInput", "ApplyMeshSolidify: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FDynamicMesh3& UseSourceMesh = EditMesh;

		const bool bUseCustomBounds = (bool)Options.CustomBounds.IsValid;
		FAxisAlignedBox3d UseBounds = bUseCustomBounds ?
			FAxisAlignedBox3d(Options.CustomBounds) : EditMesh.GetBounds(true);

		if (bUseCustomBounds && UseBounds.Diagonal().GetMin() <= 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshSolidify_EmptyCustomBounds", "ApplyMeshSolidify: Empty custom bounds"));
			EditMesh.Clear();
			return;
		}

		FDynamicMesh3 ThickenedMesh;
		if (Options.bThickenShells)
		{
			// thickness should be at least a cell wide so we don't end up deleting a bunch of the input surface
			double CellSize = (Options.GridParameters.SizeMethod == EGeometryScriptGridSizingMethod::GridResolution) ?
				(UseBounds.MaxDim() / Options.GridParameters.GridResolution) : Options.GridParameters.GridCellSize;
			double SafeThickness = FMathd::Max(CellSize * 2, Options.ShellThickness);
			ThickenedMesh.Copy(UseSourceMesh, false, false, false, false);
			FMeshNormals::QuickComputeVertexNormals(ThickenedMesh);
			FExtrudeMesh Extrude(&ThickenedMesh);
			Extrude.bSkipClosedComponents = true;
			Extrude.DefaultExtrudeDistance = -SafeThickness;
			Extrude.IsPositiveOffset = false;
			Extrude.Apply();
			UseSourceMesh = ThickenedMesh;
		}

		FDynamicMeshAABBTree3 Spatial(&UseSourceMesh);
		TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial);

		TImplicitSolidify<FDynamicMesh3> Solidify(&UseSourceMesh, &Spatial, &FastWinding);
		Solidify.bUseCustomBounds = bUseCustomBounds;
		Solidify.CustomBounds = UseBounds;

		if (Options.GridParameters.SizeMethod == EGeometryScriptGridSizingMethod::GridResolution)
		{
			int ClampResolution = GetClampedGridResolution(Options.GridParameters.GridResolution, TEXT("ApplyMeshSolidify"), Debug);
			Solidify.SetCellSizeAndExtendBounds(UseBounds, Options.ExtendBounds, ClampResolution);
		}
		else
		{
			double UseCellSize = GetClampedCellSize(Options.GridParameters.GridCellSize, UseBounds, TEXT("ApplyMeshSolidify"), Debug);
			Solidify.ExtendBounds = Options.ExtendBounds;
			Solidify.MeshCellSize = UseCellSize;
		}

		Solidify.WindingThreshold = Options.WindingThreshold;
		Solidify.SurfaceSearchSteps = Options.SurfaceSearchSteps;
		Solidify.bSolidAtBoundaries = Options.bSolidAtBoundaries;
		Solidify.ExtendBounds = Options.ExtendBounds;
		EditMesh.Copy(&Solidify.Generate());

		if (EditMesh.TriangleCount() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyMeshSolidify_EmptyMesh", "ApplyMeshSolidify: Solidify operation output an empty mesh"));
		}
		else
		{
			EditMesh.EnableAttributes();
			FMeshNormals::InitializeOverlayToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), false);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}







UDynamicMesh* UGeometryScriptLibrary_MeshVoxelFunctions::ApplyMeshMorphology(
	UDynamicMesh* TargetMesh,
	FGeometryScriptMorphologyOptions Options,
	UGeometryScriptDebug* Debug)
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshMorphology_InvalidInput", "ApplyMeshMorphology: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh) 
	{
		FDynamicMeshAABBTree3 Spatial(&EditMesh);
		TImplicitMorphology<FDynamicMesh3> Morphology;
		Morphology.Source = &EditMesh;
		Morphology.SourceSpatial = &Spatial;
		
		const bool bUseCustomBounds = (bool)Options.CustomBounds.IsValid;
		FAxisAlignedBox3d UseBounds = bUseCustomBounds ?
			FAxisAlignedBox3d(Options.CustomBounds) : Spatial.GetBoundingBox();

		if (bUseCustomBounds && UseBounds.Diagonal().GetMin() <= 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyMeshMorphology_EmptyCustomBounds", "ApplyMeshMorphology: Empty custom bounds"));
			EditMesh.Clear();
			return;
		}
		
		Morphology.MorphologyOp = static_cast<TImplicitMorphology<FDynamicMesh3>::EMorphologyOp>(Options.Operation);
		Morphology.bUseCustomBounds = bUseCustomBounds;
		Morphology.CustomBounds = UseBounds;
		Morphology.Distance = Options.Distance;
		if (Morphology.Distance == 0)
		{
			// TImplicitMorphology does not like Distance=0, but in this function we want to fallback to basically just computing SDF
			Morphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Dilate;
			Morphology.Distance = 0.00001;   
		}

		FGeometryScript3DGridParameters SDFGridParameters = Options.SDFGridParameters;
		if (SDFGridParameters.SizeMethod == EGeometryScriptGridSizingMethod::GridResolution)
		{
			int ClampResolution = GetClampedGridResolution(SDFGridParameters.GridResolution, TEXT("ApplyMeshMorphology"), Debug);
			Morphology.SetGridCellSize(UseBounds, Options.Distance, ClampResolution);
		}
		else
		{
			double UseCellSize = GetClampedCellSize(SDFGridParameters.GridCellSize, UseBounds, TEXT("ApplyMeshMorphology"), Debug);
			Morphology.GridCellSize = UseCellSize;
		}


		FGeometryScript3DGridParameters MeshGridParameters = Options.bUseSeparateMeshGrid ? Options.MeshGridParameters : SDFGridParameters;
		if (MeshGridParameters.SizeMethod == EGeometryScriptGridSizingMethod::GridResolution)
		{
			int ClampResolution = GetClampedGridResolution(MeshGridParameters.GridResolution, TEXT("ApplyMeshMorphology"), Debug);
			Morphology.SetMeshCellSize(UseBounds, Options.Distance, ClampResolution);
		}
		else
		{
			double UseCellSize = GetClampedCellSize(MeshGridParameters.GridCellSize, UseBounds, TEXT("ApplyMeshMorphology"), Debug);
			Morphology.MeshCellSize = UseCellSize;
		}

		EditMesh.Copy(&Morphology.Generate());

		if (EditMesh.TriangleCount() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyMeshMorphology_EmptyMesh", "ApplyMeshMorphology: Morphology operation output an empty mesh"));
		}
		else
		{
			EditMesh.EnableAttributes();
			FMeshNormals::InitializeOverlayToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), false);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	return TargetMesh;
}







#undef LOCTEXT_NAMESPACE



