// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/MeshWeightMapFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWeightMapFunctions)

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_MeshWeightMapFunctions"


namespace UE::MeshWeightMapFunctionLocals
{
	using namespace UE::Geometry;

	int32 FindWeightMapLayerIndex(const FDynamicMesh3& Mesh, const FName& Name)
	{
		if (Mesh.HasAttributes())
		{
			for (int32 LayerIndex = 0; LayerIndex < Mesh.Attributes()->NumWeightLayers(); ++LayerIndex)
			{
				if (Mesh.Attributes()->GetWeightLayer(LayerIndex)->GetName() == Name)
				{
					return LayerIndex;
				}
			}
		}
		return INDEX_NONE;
	}

	const FDynamicMeshWeightAttribute* GetWeightsFromHandle(const FDynamicMesh3& Mesh, const FGeometryScriptWeightMapHandle& WeightMapHandle)
	{
		if (WeightMapHandle.IsValid() && Mesh.HasAttributes() && WeightMapHandle.WeightMapAttributeLayerIndex < Mesh.Attributes()->NumWeightLayers())
		{
			return Mesh.Attributes()->GetWeightLayer(WeightMapHandle.WeightMapAttributeLayerIndex);
		}
		return nullptr;
	}
	FDynamicMeshWeightAttribute* GetWeightsFromHandle(FDynamicMesh3& Mesh, const FGeometryScriptWeightMapHandle& WeightMapHandle)
	{
		if (WeightMapHandle.IsValid() && Mesh.HasAttributes() && WeightMapHandle.WeightMapAttributeLayerIndex < Mesh.Attributes()->NumWeightLayers())
		{
			return Mesh.Attributes()->GetWeightLayer(WeightMapHandle.WeightMapAttributeLayerIndex);
		}
		return nullptr;
	}
}


UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::FindOrAddMeshWeightMap(
	UDynamicMesh* TargetMesh, FName Name, FGeometryScriptWeightMapHandle& WeightMapHandle,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	// init to invalid / not-found
	WeightMapHandle = FGeometryScriptWeightMapHandle();
	
	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("FindOrAddWeightMap_NullMesh", "FindOrAddWeightMap: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->ProcessMesh([&Name, &WeightMapHandle](const FDynamicMesh3& Mesh)
	{
		WeightMapHandle.WeightMapAttributeLayerIndex = UE::MeshWeightMapFunctionLocals::FindWeightMapLayerIndex(Mesh, Name);
	});

	if (WeightMapHandle.WeightMapAttributeLayerIndex == INDEX_NONE)
	{
		TargetMesh->EditMesh([&Name, &WeightMapHandle](FDynamicMesh3& Mesh)
		{
			Mesh.EnableAttributes();
			WeightMapHandle.WeightMapAttributeLayerIndex = Mesh.Attributes()->NumWeightLayers();
			Mesh.Attributes()->SetNumWeightLayers(WeightMapHandle.WeightMapAttributeLayerIndex + 1);
			Mesh.Attributes()->GetWeightLayer(WeightMapHandle.WeightMapAttributeLayerIndex)->SetName(Name);
		});
	}

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::RemoveMeshWeightMap(
	UDynamicMesh* TargetMesh, FGeometryScriptWeightMapHandle Handle,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("RemoveMeshWeightMap_NullMesh", "RemoveMeshWeightMap: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&Handle](FDynamicMesh3& Mesh)
	{
		if (Mesh.HasAttributes() && Handle.IsValid() && Handle.WeightMapAttributeLayerIndex < Mesh.Attributes()->NumWeightLayers())
		{
			Mesh.Attributes()->RemoveWeightLayer(Handle.WeightMapAttributeLayerIndex);
		}
	});
	
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::GetMeshWeightMapValues(
	UDynamicMesh* TargetMesh, FGeometryScriptScalarList& WeightValues, FGeometryScriptWeightMapHandle Handle, bool bSkipGaps, bool& bHasVertexIDGaps, UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	WeightValues.Reset();
	TArray<double>& Weights = *WeightValues.List;
	bHasVertexIDGaps = false;
	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshWeightMapValues_NullMesh", "GetMeshWeightMapValues: TargetMesh is Null"));
		return TargetMesh;
	}
	TargetMesh->ProcessMesh([&Weights, &Handle, bSkipGaps, &bHasVertexIDGaps, Debug](const FDynamicMesh3& Mesh)
	{
		const FDynamicMeshWeightAttribute* WeightAttrib = UE::MeshWeightMapFunctionLocals::GetWeightsFromHandle(Mesh, Handle);
		if (!WeightAttrib)
		{
			Weights.SetNumZeroed(bSkipGaps ? Mesh.VertexCount() : Mesh.MaxVertexID());
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("GetMeshWeightMapValues_WeightsNotFound", "GetMeshWeightMapValues: Weight Layer not found"));
			return;
		}
		if (bSkipGaps)
		{
			Weights.Reserve(Mesh.VertexCount());
			for (int32 VID : Mesh.VertexIndicesItr())
			{
				float Wt;
				WeightAttrib->GetValue(VID, &Wt);
				Weights.Add((double)Wt);
			}
			bHasVertexIDGaps = false;
		}
		else
		{
			Weights.SetNumZeroed(Mesh.MaxVertexID());
			for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
			{
				if (Mesh.IsVertex(VID))
				{
					float Wt;
					WeightAttrib->GetValue(VID, &Wt);
					Weights[VID] = (double)Wt;
				}
			}
			bHasVertexIDGaps = !Mesh.IsCompactV();
		}
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::SetMeshWeightMapValues(
	UDynamicMesh* TargetMesh, FGeometryScriptScalarList WeightValues, FGeometryScriptWeightMapHandle Handle, bool bSkipGaps, UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshWeightMapValues_NullMesh", "SetMeshWeightMapValues: TargetMesh is Null"));
		return TargetMesh;
	}
	if (!WeightValues.List)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshWeightMapValues_NullWeights", "SetMeshWeightMapValues: Weight List is not set"));
		return TargetMesh;
	}
	const TArray<double>& Weights = *WeightValues.List;

	TargetMesh->EditMesh([&Weights, &Handle, bSkipGaps, Debug](FDynamicMesh3& Mesh)
	{
		FDynamicMeshWeightAttribute* WeightAttrib = UE::MeshWeightMapFunctionLocals::GetWeightsFromHandle(Mesh, Handle);
		if (!WeightAttrib)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshWeightMapValues_WeightsNotFound", "SetMeshWeightMapValues: Weight Layer not found"));
			return;
		}
		if (Weights.Num() != (bSkipGaps ? Mesh.VertexCount() : Mesh.MaxVertexID()))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshWeightMapValues_IncorrectWeightNum", "SetMeshWeightMapValues: Weight List size mismatched with Target Mesh"));
			return;
		}
		if (bSkipGaps)
		{
			int32 CopyIdx = 0;
			for (int32 VID : Mesh.VertexIndicesItr())
			{
				WeightAttrib->SetScalarValue(VID, (float)Weights[CopyIdx++]);
			}
		}
		else
		{
			for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
			{
				if (Mesh.IsVertex(VID))
				{
					WeightAttrib->SetScalarValue(VID, (float)Weights[VID]);
				}
			}
		}
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::SetMeshConstantWeightMapValue(
	UDynamicMesh* TargetMesh,
	FGeometryScriptWeightMapHandle Handle,
	float ConstantWeight,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshConstantWeightMapValue_NullMesh", "SetMeshConstantWeightMapValue: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&Handle, ConstantWeight, Debug](FDynamicMesh3& Mesh)
	{
		FDynamicMeshWeightAttribute* WeightAttrib = UE::MeshWeightMapFunctionLocals::GetWeightsFromHandle(Mesh, Handle);
		if (!WeightAttrib)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshConstantWeightMapValue_WeightsNotFound", "SetMeshConstantWeightMapValue: Weight Layer not found"));
			return;
		}
		for (int32 VID : Mesh.VertexIndicesItr())
		{
			WeightAttrib->SetScalarValue(VID, ConstantWeight);
		}
	});
	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_MeshWeightMapFunctions::SetMeshSelectionWeightMapValue(
	UDynamicMesh* TargetMesh,
	FGeometryScriptWeightMapHandle Handle,
	FGeometryScriptMeshSelection Selection,
	float ConstantWeight,
	UGeometryScriptDebug* Debug)
{
	using namespace UE::Geometry;

	if (!TargetMesh)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshSelectionWeightMapValue_NullMesh", "SetMeshSelectionWeightMapValue: TargetMesh is Null"));
		return TargetMesh;
	}

	TargetMesh->EditMesh([&Handle, &Selection, ConstantWeight, Debug](FDynamicMesh3& Mesh)
	{
		FDynamicMeshWeightAttribute* WeightAttrib = UE::MeshWeightMapFunctionLocals::GetWeightsFromHandle(Mesh, Handle);
		if (!WeightAttrib)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetMeshSelectionWeightMapValue_WeightsNotFound", "SetMeshSelectionWeightMapValue: Weight Layer not found"));
			return;
		}
		Selection.ProcessByVertexID(Mesh, [&WeightAttrib, ConstantWeight](int32 VID)
		{
			WeightAttrib->SetScalarValue(VID, ConstantWeight);
		});
	});
	return TargetMesh;
}


#undef LOCTEXT_NAMESPACE