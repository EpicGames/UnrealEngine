// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/PointSetFunctions.h"

#include "Clustering/KMeans.h"
#include "Spatial/PriorityOrderPoints.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PointSetFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_PointSetFunctions"

namespace UE::Private::PointSetFunctionLocals
{
	static FClusterKMeans GetClustering(const TArray<FVector>& Points, const FGeometryScriptPointClusteringOptions& Options)
	{
		FClusterKMeans Clustering;
		Clustering.RandomSeed = Options.RandomSeed;
		Clustering.MaxIterations = Options.MaxIterations;
		const TArray<FVector>* UseInitialCenters = &Options.InitialClusterCenters;
		TArray<FVector> LocalInitialCenters;
		if (Options.InitialClusterCenters.IsEmpty())
		{
			UseInitialCenters = &LocalInitialCenters;
			if (Options.InitializeMethod == EGeometryScriptInitKMeansMethod::UniformSpacing)
			{
				Clustering.GetUniformSpacedInitialCenters<FVector>(Points, Options.TargetNumClusters, LocalInitialCenters);
			}
		}
		Clustering.ComputeClusters<FVector>(Points, Options.TargetNumClusters, *UseInitialCenters);
		return Clustering;
	}

	// Remap axes for 3D -> 2D conversion, dropping the chosen axis
	inline static FIndex2i FlattenedAxesMap(EGeometryScriptAxis DropAxis)
	{
		return FIndex2i(
			int32(DropAxis == EGeometryScriptAxis::X),
			1 + int32(DropAxis != EGeometryScriptAxis::Z));
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::KMeansClusterToIDs(
	const TArray<FVector>& Points,
	const FGeometryScriptPointClusteringOptions& Options,
	TArray<int32>& PointClusterIndices)
{
	FClusterKMeans Clustering = UE::Private::PointSetFunctionLocals::GetClustering(Points, Options);
	PointClusterIndices = MoveTemp(Clustering.ClusterIDs);
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::KMeansClusterToArrays(
	const TArray<FVector>& Points,
	const FGeometryScriptPointClusteringOptions& Options,
	TArray<FGeometryScriptIndexList>& ClusterIDToLists)
{
	FClusterKMeans Clustering = UE::Private::PointSetFunctionLocals::GetClustering(Points, Options);
	ClusterIDToLists.SetNum(Clustering.ClusterSizes.Num());
	for (int32 ClusterIdx = 0; ClusterIdx < ClusterIDToLists.Num(); ++ClusterIdx)
	{
		ClusterIDToLists[ClusterIdx].Reset(EGeometryScriptIndexType::Any);
		ClusterIDToLists[ClusterIdx].List->Reserve(Clustering.ClusterSizes[ClusterIdx]);
	}
	if (ClusterIDToLists.IsEmpty())
	{
		return;
	}

	for (int32 PointIdx = 0; PointIdx < Clustering.ClusterIDs.Num(); ++PointIdx)
	{
		ClusterIDToLists[Clustering.ClusterIDs[PointIdx]].List->Add(PointIdx);
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::TransformsToPoints(
	const TArray<FTransform>& Transforms,
	TArray<FVector>& Points
)
{
	Points.SetNum(Transforms.Num());
	for (int32 Idx = 0; Idx < Transforms.Num(); ++Idx)
	{
		Points[Idx] = Transforms[Idx].GetLocation();
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::DownsamplePoints(
	const TArray<FVector>& Points,
	const FGeometryScriptPointPriorityOptions& Options,
	FGeometryScriptIndexList& DownsampledIndices,
	int32 KeepNumPoints,
	UGeometryScriptDebug* Debug)
{
	DownsampledIndices.Reset(EGeometryScriptIndexType::Any);
	int32 NumWeights = Options.OptionalPriorityWeights.Num();
	bool bHasImportanceWeights = NumWeights >= Points.Num();
	if (NumWeights > 0 && NumWeights < Points.Num())
	{
		UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, 
			FText::Format(LOCTEXT("DownsamplePointsMissingWeights", "DownsamplePoints: Found {0} points but only {1} PriorityWeights"), Points.Num(), NumWeights));
	}
	KeepNumPoints = FMath::Min(Points.Num(), KeepNumPoints);
	if (Options.bUniformSpacing || bHasImportanceWeights)
	{
		FPriorityOrderPoints Order;
		if (Options.bUniformSpacing)
		{
			Order.ComputeUniformSpaced(Points, bHasImportanceWeights ? Options.OptionalPriorityWeights : TArrayView<const float>(), KeepNumPoints);
		}
		else // bHasImportanceWeights
		{
			Order.ComputeDescendingImportance(Options.OptionalPriorityWeights);
		}
		Order.Order.SetNum(KeepNumPoints);
		(*DownsampledIndices.List) = MoveTemp(Order.Order);
		
	}
	else // no criteria to prioritize points; just take the first indices
	{
		DownsampledIndices.List->SetNumUninitialized(KeepNumPoints);
		for (int32 Idx = 0; Idx < KeepNumPoints; ++Idx)
		{
			(*DownsampledIndices.List)[Idx] = Idx;
		}
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::GetPointsFromIndexList(
	const TArray<FVector>& AllPoints,
	const FGeometryScriptIndexList& Indices,
	TArray<FVector>& SelectedPoints
)
{
	if (!Indices.List.IsValid())
	{
		SelectedPoints.Reset();
		return;
	}
	SelectedPoints.Reset(Indices.List->Num());
	for (int32 Idx : *Indices.List)
	{
		SelectedPoints.Add(AllPoints[Idx]);
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::OffsetTransforms(
	TArray<FTransform>& Transforms,
	double Offset,
	FVector Direction,
	EGeometryScriptCoordinateSpace Space
)
{
	Direction.Normalize();
	for (FTransform& Transform : Transforms)
	{
		FVector OffsetVec = Direction * Offset;
		if (Space == EGeometryScriptCoordinateSpace::Local)
		{
			OffsetVec = Transform.TransformVector(OffsetVec);
		}
		Transform.SetLocation(OffsetVec + Transform.GetLocation());
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::FlattenPoints(
	const TArray<FVector>& PointsIn3D,
	TArray<FVector2D>& PointsIn2D,
	const FGeometryScriptPointFlatteningOptions& Options
)
{
	PointsIn2D.SetNumUninitialized(PointsIn3D.Num());

	FIndex2i AxesMap = UE::Private::PointSetFunctionLocals::FlattenedAxesMap(Options.DropAxis);
	for (int32 Idx = 0; Idx < PointsIn3D.Num(); ++Idx)
	{
		FVector RelPoint = Options.Frame.InverseTransformPosition(PointsIn3D[Idx]);
		PointsIn2D[Idx].X = RelPoint[AxesMap[0]];
		PointsIn2D[Idx].Y = RelPoint[AxesMap[1]];
	}
}

void UGeometryScriptLibrary_PointSetSamplingFunctions::UnflattenPoints(
	const TArray<FVector2D>& PointsIn2D,
	TArray<FVector>& PointsIn3D,
	const FGeometryScriptPointFlatteningOptions& Options,
	double Height
)
{
	PointsIn3D.SetNumUninitialized(PointsIn2D.Num());

	FIndex2i AxesMap = UE::Private::PointSetFunctionLocals::FlattenedAxesMap(Options.DropAxis);
	for (int32 Idx = 0; Idx < PointsIn2D.Num(); ++Idx)
	{
		FVector RelPoint;
		RelPoint[AxesMap[0]] = PointsIn2D[Idx].X;
		RelPoint[AxesMap[1]] = PointsIn2D[Idx].Y;
		RelPoint[(int32)Options.DropAxis] = Height;
		PointsIn3D[Idx] = Options.Frame.TransformPosition(RelPoint);
	}
}


FBox UGeometryScriptLibrary_PointSetSamplingFunctions::MakeBoundingBoxFromPoints(const TArray<FVector>& Points, double ExpandBy)
{
	FBox Bounds(Points);
	return Bounds.ExpandBy(ExpandBy);
}



#undef LOCTEXT_NAMESPACE