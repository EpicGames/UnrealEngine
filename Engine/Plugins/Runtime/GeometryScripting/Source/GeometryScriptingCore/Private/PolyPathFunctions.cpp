// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/PolyPathFunctions.h"

#include "Curve/CurveUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyPathFunctions)

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_PolyPathFunctions"


int UGeometryScriptLibrary_PolyPathFunctions::GetPolyPathNumVertices(FGeometryScriptPolyPath PolyPath)
{
	return (PolyPath.Path.IsValid()) ? PolyPath.Path->Num() : 0;
}

int UGeometryScriptLibrary_PolyPathFunctions::GetPolyPathLastIndex(FGeometryScriptPolyPath PolyPath)
{
	return (PolyPath.Path.IsValid()) ? FMath::Max(PolyPath.Path->Num()-1,0) : 0;
}

FVector UGeometryScriptLibrary_PolyPathFunctions::GetPolyPathVertex(FGeometryScriptPolyPath PolyPath, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (PolyPath.Path.IsValid() && Index >= 0 && Index < PolyPath.Path->Num())
	{
		bIsValidIndex = true;
		return (*PolyPath.Path)[Index];
	}
	return FVector::ZeroVector;
}

FVector UGeometryScriptLibrary_PolyPathFunctions::GetPolyPathTangent(FGeometryScriptPolyPath PolyPath, int Index, bool& bIsValidIndex)
{
	bIsValidIndex = false;
	if (PolyPath.Path.IsValid() && Index >= 0 && Index < PolyPath.Path->Num())
	{
		bIsValidIndex = true;
		FVector Tangent = UE::Geometry::CurveUtil::Tangent<double, FVector>(*PolyPath.Path, Index, PolyPath.bClosedLoop);
		return Tangent;
	}
	return FVector::ZeroVector;
}

double UGeometryScriptLibrary_PolyPathFunctions::GetPolyPathArcLength(FGeometryScriptPolyPath PolyPath)
{
	if (PolyPath.Path.IsValid())
	{
		return UE::Geometry::CurveUtil::ArcLength<double, FVector>(*PolyPath.Path, PolyPath.bClosedLoop);
	}
	return 0;
}

int32 UGeometryScriptLibrary_PolyPathFunctions::GetNearestVertexIndex(FGeometryScriptPolyPath PolyPath, FVector Point)
{
	if (PolyPath.Path.IsValid())
	{
		return UE::Geometry::CurveUtil::FindNearestIndex<double, FVector>(*PolyPath.Path, Point);
	}
	return -1;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::FlattenTo2DOnAxis(FGeometryScriptPolyPath PolyPath, EGeometryScriptAxis DropAxis)
{
	FGeometryScriptPolyPath ToReturn;
	ToReturn.Reset();
	if (PolyPath.Path.IsValid())
	{
		ToReturn.bClosedLoop = PolyPath.bClosedLoop;
		ToReturn.Path = PolyPath.Path;

		int32 Keep0 = int32(DropAxis == EGeometryScriptAxis::X);
		int32 Keep1 = 1 + int32(DropAxis != EGeometryScriptAxis::Z);
		for (FVector& V : *ToReturn.Path)
		{
			V[0] = V[Keep0];
			V[1] = V[Keep1];
			V.Z = 0;
		}
	}
	return ToReturn;
}

namespace PolyPathInternal
{
	
TArray<FVector> CreateArcPoints(FTransform Transform, float Radius, int NumPoints, float StartAngle, float EndAngle, bool bIncludeEndPoint)
{
	TArray<FVector> Points;
	NumPoints = FMath::Max(2, NumPoints);
	Points.Reserve(NumPoints);

	double AngleStartRad = FMathd::DegToRad * double(StartAngle);
	double AngleRangeRad = FMathd::DegToRad * double(EndAngle) - AngleStartRad;
	double ToAngle = AngleRangeRad / double(NumPoints - (int)bIncludeEndPoint); // if bIncludeEndPoint, adjust fraction to reach end at PtIdx = NumPoints-1
	for (int PtIdx = 0; PtIdx < NumPoints; PtIdx++)
	{
		double Angle = AngleStartRad + PtIdx * ToAngle;
		Points.Emplace(Transform.TransformPosition(FVector3d(FMathd::Cos(Angle) * (double)Radius, FMathd::Sin(Angle) * (double)Radius, 0.0)));
	}

	return Points;
}

}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::CreateCirclePath3D(FTransform Transform, float Radius, int NumPoints)
{
	FGeometryScriptPolyPath PolyPath;
	PolyPath.Reset();
	PolyPath.bClosedLoop = true;

	*PolyPath.Path = PolyPathInternal::CreateArcPoints(Transform, Radius, NumPoints, 0, 360, false);

	return PolyPath;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::CreateArcPath3D(FTransform Transform, float Radius, int NumPoints, float StartAngle, float EndAngle)
{
	FGeometryScriptPolyPath PolyPath;
	PolyPath.Reset();

	PolyPath.bClosedLoop = false;

	*PolyPath.Path = PolyPathInternal::CreateArcPoints(Transform, Radius, NumPoints, StartAngle, EndAngle, true);

	return PolyPath;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::CreateCirclePath2D(FVector2D Center, float Radius, int NumPoints)
{
	FGeometryScriptPolyPath PolyPath;
	PolyPath.Reset();

	PolyPath.bClosedLoop = true;

	FTransform TransformCenter(FVector(Center.X, Center.Y, 0));
	*PolyPath.Path = PolyPathInternal::CreateArcPoints(TransformCenter, Radius, NumPoints, 0, 360, false);

	return PolyPath;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::CreateArcPath2D(FVector2D Center, float Radius, int NumPoints, float StartAngle, float EndAngle)
{
	FGeometryScriptPolyPath PolyPath;
	PolyPath.Reset();

	PolyPath.bClosedLoop = false;

	FTransform TransformCenter(FVector(Center.X, Center.Y, 0));
	*PolyPath.Path = PolyPathInternal::CreateArcPoints(TransformCenter, Radius, NumPoints, StartAngle, EndAngle, true);

	return PolyPath;
}

void UGeometryScriptLibrary_PolyPathFunctions::ConvertPolyPathToArray(FGeometryScriptPolyPath PolyPath, TArray<FVector>& PathVertices)
{
	PathVertices.Reset();
	if (PolyPath.Path.IsValid())
	{
		PathVertices.Append(*PolyPath.Path);
	}
}

void UGeometryScriptLibrary_PolyPathFunctions::ConvertArrayToPolyPath(const TArray<FVector>& PathVertices, FGeometryScriptPolyPath& PolyPath)
{
	PolyPath.Reset();
	PolyPath.Path->Append(PathVertices);
}

void UGeometryScriptLibrary_PolyPathFunctions::ConvertPolyPathToArrayOfVector2D(FGeometryScriptPolyPath PolyPath, TArray<FVector2D>& PathVertices)
{
	PathVertices.Reset();
	if (PolyPath.Path.IsValid())
	{
		PathVertices.Reserve(PolyPath.Path->Num());
		for (const FVector& V : *PolyPath.Path)
		{
			PathVertices.Emplace(V.X, V.Y);
		}
	}
}

void UGeometryScriptLibrary_PolyPathFunctions::ConvertArrayOfVector2DToPolyPath(const TArray<FVector2D>& PathVertices, FGeometryScriptPolyPath& PolyPath)
{
	PolyPath.Reset();
	PolyPath.Path->Reserve(PathVertices.Num());
	for (const FVector2D& V : PathVertices)
	{
		PolyPath.Path->Emplace(V.X, V.Y, 0);
	}
}

namespace UE::Local::SplinePathHelpers
{
	// Get the sampling range, and whether it is a distance range (otherwise, assume it's a time range)
	void GetRangeFromSamplingOptions(const USplineComponent* Spline, const FGeometryScriptSplineSamplingOptions& SamplingOptions, float& OutStart, float& OutEnd, bool& bOutIsDistanceRange)
	{
		OutStart = SamplingOptions.RangeStart;
		OutEnd = SamplingOptions.RangeEnd;
		bOutIsDistanceRange = 
			   SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline
			|| SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::DistanceRange;
		if (bOutIsDistanceRange)
		{
			float SplineLength = Spline->GetSplineLength();
			
			if (SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline)
			{
				OutStart = 0;
				OutEnd = SplineLength;
			}
		}
		// else it's a time range; values can be left as they are
	}

	// Note: this does *not* support wrapped values
	float DistanceToTime(const USplineComponent* Spline, float Distance, bool bConstantSpeed)
	{
		if (bConstantSpeed)
		{
			return Distance * Spline->Duration / Spline->GetSplineLength();
		}
		else
		{
			checkSlow(Distance >= 0 && Distance <= Spline->GetSplineLength());
			return Spline->GetTimeAtDistanceAlongSpline(Distance);
		}
	};

	// Convert a Time parameter from a 'Constant Speed' to a 'Varying Speed' value, or vice versa
	// e.g., if bWasConstantSpeed is true, this reports what Time would get to the same location on the spline if we were instead evaluating the spline at varying speed
	float SwitchTimeType(const USplineComponent* Spline, float Time, bool bWasConstantSpeed, bool bAllowWrappingIfClosed = true)
	{
		const float Duration = Spline->Duration;
		float InRangeTime = Time;
		float TimeAtStartOfLoop = 0;
		if (!bAllowWrappingIfClosed)
		{
			InRangeTime = FMath::Clamp(Time, 0, Duration);
		}
		else
		{
			InRangeTime = FMath::Fmod(Time, Duration);
			if (InRangeTime < 0)
			{
				InRangeTime += Duration;
			}
			TimeAtStartOfLoop = FMath::Floor(Time / Duration) * Duration;
		}
		const float InRangeTimeFrac = InRangeTime / Duration;
		float InRangeDistance = 0;
		if (bWasConstantSpeed)
		{
			InRangeDistance = InRangeTimeFrac * Spline->GetSplineLength();
		}
		else
		{
			const int32 NumSegments = Spline->GetNumberOfSplineSegments();
			// Note: 'InputKey' values correspond to the spline in parameter space, in the range of 0 to NumSegments
			const float InRangeInputKey = InRangeTimeFrac * NumSegments;
			InRangeDistance = Spline->GetDistanceAlongSplineAtSplineInputKey(InRangeInputKey);
		}

		float ConvertedInRangeTime = DistanceToTime(Spline, InRangeDistance, !bWasConstantSpeed);
		return TimeAtStartOfLoop + ConvertedInRangeTime;
	}

	// Helper to handle UniformDistance or UniformTime sampling
	bool IterateSamplesInRange(
		const USplineComponent* Spline,
		const FGeometryScriptSplineSamplingOptions& SamplingOptions,
		bool bWantPos,
		TFunctionRef<void(const FVector&)> PositionFunc,
		bool bWantTime,
		TFunctionRef<void(float)> TimeFunc
	)
	{
		if (!bWantPos && !bWantTime)
		{
			return false;
		}

		float Start, End;
		bool bWasDistanceRange;
		GetRangeFromSamplingOptions(Spline, SamplingOptions, Start, End, bWasDistanceRange);
		// For consistent handling below, always convert Distance to a TimeRange_ConstantSpeed-based range (by re-scaling)
		if (bWasDistanceRange)
		{
			float DistanceToTimeFrac = Spline->Duration / Spline->GetSplineLength();
			Start *= DistanceToTimeFrac;
			End *= DistanceToTimeFrac;
		}

		const bool bAllowWrap = Spline->IsClosedLoop() && SamplingOptions.RangeMethod != EGeometryScriptEvaluateSplineRange::FullSpline;

		// UniformDistance and UniformTime sampling correspond to two different parameterizations of the spline:
		// TimeRange_ConstantSpeed and TimeRange_VariableSpeed, respectively.
		// To make the sampling easier, we convert the range values to use the same parameterization as the sampling.
		bool bTimeRangeWasConstantSpeed = SamplingOptions.RangeMethod != EGeometryScriptEvaluateSplineRange::TimeRange_VariableSpeed;
		const bool bUniformSampleSpacing = SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::UniformDistance;
		if (bUniformSampleSpacing != bTimeRangeWasConstantSpeed)
		{
			Start = UE::Local::SplinePathHelpers::SwitchTimeType(Spline, Start, bTimeRangeWasConstantSpeed, bAllowWrap);
			End = UE::Local::SplinePathHelpers::SwitchTimeType(Spline, End, bTimeRangeWasConstantSpeed, bAllowWrap);
		}
		
		const float MaxValue = Spline->Duration;
		if (!bAllowWrap)
		{
			End = FMath::Min(MaxValue, End);
			Start = FMath::Max(0, Start);
		}

		const float Range = End - Start;
		if (Range < 0)
		{
			return false;
		}

		auto GetPos = [bUniformSampleSpacing, Spline, &SamplingOptions](float Value) -> FVector
		{
			return Spline->GetLocationAtTime(Value, SamplingOptions.CoordinateSpace, bUniformSampleSpacing);
		};
		auto WrapValue = [MaxValue](float Value)
		{
			float WrappedValue = FMath::Fmod(Value, MaxValue);
			if (WrappedValue < 0)
			{
				WrappedValue += MaxValue;
			}
			return WrappedValue;
		};

		int32 UseSamples = FMath::Max(2, SamplingOptions.NumSamples);
		// If we span 0 range, just report the single point
		if (Range == 0)
		{
			UseSamples = 1;
		}
		// In non-loops, we adjust DivNum so we exactly sample the end of the spline
		// In loops we don't sample the endpoint, by convention, as it's the same as the start
		const bool bSampleExactEnd = !(Spline->IsClosedLoop() && SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline);
		const float DivNum = float(UseSamples - (int32)bSampleExactEnd);

		for (int32 Idx = 0; Idx < UseSamples; Idx++)
		{
			float Value = Start + Range * ((float)Idx / DivNum);
			if (bAllowWrap)
			{
				Value = WrapValue(Value);
			}

			if (bWantPos)
			{
				FVector Pos = GetPos(Value);
				PositionFunc(Pos);
			}
			if (bWantTime)
			{
				TimeFunc(Value);
			}
		}
		return true;
	}

	// return true if it's a full loop, and we've omitted the final point (which matched the start point)
	bool ErrorBasedSampleInRange(const USplineComponent* Spline, FGeometryScriptSplineSamplingOptions SamplingOptions, TArray<FVector>& OutPositions, TArray<double>& OutDistances)
	{
		// this function does error tolerance sampling so we expect the options to match that
		ensure(SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance);

		bool bIsLoop = Spline->IsClosedLoop() && SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline;
		float SquaredErrorTolerance = FMath::Max(KINDA_SMALL_NUMBER, SamplingOptions.ErrorTolerance * SamplingOptions.ErrorTolerance);
		float UseRangeStart, UseRangeEnd;
		bool bIsDistanceRange;
		UE::Local::SplinePathHelpers::GetRangeFromSamplingOptions(Spline, SamplingOptions, UseRangeStart, UseRangeEnd, bIsDistanceRange);
		if (bIsDistanceRange)
		{
			bool bAllowWrap = SamplingOptions.RangeMethod != EGeometryScriptEvaluateSplineRange::FullSpline;
			Spline->ConvertSplineToPolyline_InDistanceRange(
				SamplingOptions.CoordinateSpace, SquaredErrorTolerance, UseRangeStart, UseRangeEnd, OutPositions, OutDistances, bAllowWrap);
		}
		else
		{
			bool bConstantSpeed = SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::TimeRange_ConstantSpeed;
			Spline->ConvertSplineToPolyline_InTimeRange(
				SamplingOptions.CoordinateSpace, SquaredErrorTolerance, UseRangeStart, UseRangeEnd, bConstantSpeed, OutPositions, OutDistances, true);
		}
		if (bIsLoop && !OutPositions.IsEmpty())
		{
			// delete the duplicate end-point for loops
			OutPositions.RemoveAt(OutPositions.Num() - 1);
			OutDistances.RemoveAt(OutDistances.Num() - 1);
		}
		checkSlow(OutPositions.Num() == OutDistances.Num());

		return bIsLoop;
	}
}

void UGeometryScriptLibrary_PolyPathFunctions::ConvertSplineToPolyPath(const USplineComponent* Spline, FGeometryScriptPolyPath& PolyPath, FGeometryScriptSplineSamplingOptions SamplingOptions)
{
	PolyPath.Reset();
	if (Spline)
	{
		bool bIsLoop = Spline->IsClosedLoop() && SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline;
		PolyPath.bClosedLoop = bIsLoop;
		if (SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance)
		{
			TArray<double> Distances_Unused;
			UE::Local::SplinePathHelpers::ErrorBasedSampleInRange(Spline, SamplingOptions, *PolyPath.Path, Distances_Unused);
		}
		else
		{
			int32 ExpectSamples = FMath::Max(2, SamplingOptions.NumSamples);
			PolyPath.Path->Reserve(ExpectSamples);
			UE::Local::SplinePathHelpers::IterateSamplesInRange(
				Spline,
				SamplingOptions,
				true,
				[&PolyPath](FVector Pos)
				{
					PolyPath.Path->Add(Pos);
				},
				false,
				[](float) {});
		}
	}
}


bool UGeometryScriptLibrary_PolyPathFunctions::SampleSplineToTransforms(
	const USplineComponent* Spline, 
	TArray<FTransform>& Frames, 
	TArray<double>& FrameTimes,
	FGeometryScriptSplineSamplingOptions SamplingOptions,
	FTransform RelativeTransform,
	bool bIncludeScale
)
{
	Frames.Reset();
	FrameTimes.Reset();

	// If we're using a time-based sampling method, times are constant-speed only if the sampling also is
	// (Note: This is how the time values are generated in IterateSamplesInRange, so if we want to change this, we'd need to also change their generation there.
	//   ... also, if this is changed, it should be done in a way that doesn't break the behavior of existing scripts that used this function.)
	bool bOutputConstantSpeed = SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::UniformDistance;
	// If we're using error-tolerance-based sampling, we can match output time values to the input range time values
	if (SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance)
	{
		bOutputConstantSpeed = SamplingOptions.RangeMethod != EGeometryScriptEvaluateSplineRange::TimeRange_VariableSpeed;
	}
	
	if (!Spline)
	{
		return bOutputConstantSpeed;
	}

	auto AddAtTime = [Spline, &Frames, &FrameTimes, &SamplingOptions, &RelativeTransform, bIncludeScale, bOutputConstantSpeed](float Time)
	{
		FTransform Transform = Spline->GetTransformAtTime(Time, SamplingOptions.CoordinateSpace, bOutputConstantSpeed, bIncludeScale);
		FTransform::Multiply(&Transform, &RelativeTransform, &Transform);
		Frames.Add(Transform);
		FrameTimes.Add(Time);
	};

	bool bIsLoop = Spline->IsClosedLoop() && SamplingOptions.RangeMethod == EGeometryScriptEvaluateSplineRange::FullSpline;
	if (SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance)
	{
		TArray<double> Distances;
		TArray<FVector> Positions_Unused;
		UE::Local::SplinePathHelpers::ErrorBasedSampleInRange(Spline, SamplingOptions, Positions_Unused, Distances);
		Frames.Reserve(Distances.Num());
		FrameTimes.Reserve(Distances.Num());
		for (float Dist : Distances)
		{
			float Time = UE::Local::SplinePathHelpers::DistanceToTime(Spline, Dist, bOutputConstantSpeed);
			AddAtTime(Time);
		}
	}
	else
	{
		int32 ExpectSamples = FMath::Max(2, SamplingOptions.NumSamples);
		Frames.Reserve(ExpectSamples);
		FrameTimes.Reserve(ExpectSamples);
		UE::Local::SplinePathHelpers::IterateSamplesInRange(
			Spline,
			SamplingOptions,
			false,
			[](FVector Pos) {},
			true,
			[&](float Time) 
			{
				AddAtTime(Time);
			});
	}

	return bOutputConstantSpeed;
}



TArray<FVector> UGeometryScriptLibrary_PolyPathFunctions::Conv_GeometryScriptPolyPathToArray(FGeometryScriptPolyPath PolyPath)
{
	TArray<FVector> PathVertices;
	ConvertPolyPathToArray(PolyPath, PathVertices);
	return PathVertices;
}

TArray<FVector2D> UGeometryScriptLibrary_PolyPathFunctions::Conv_GeometryScriptPolyPathToArrayOfVector2D(FGeometryScriptPolyPath PolyPath)
{
	TArray<FVector2D> PathVertices;
	ConvertPolyPathToArrayOfVector2D(PolyPath, PathVertices);
	return PathVertices;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::Conv_ArrayToGeometryScriptPolyPath(const TArray<FVector>& PathVertices)
{
	FGeometryScriptPolyPath PolyPath;
	ConvertArrayToPolyPath(PathVertices, PolyPath);
	return PolyPath;
}

FGeometryScriptPolyPath UGeometryScriptLibrary_PolyPathFunctions::Conv_ArrayOfVector2DToGeometryScriptPolyPath(const TArray<FVector2D>& PathVertices)
{
	FGeometryScriptPolyPath PolyPath;
	ConvertArrayOfVector2DToPolyPath(PathVertices, PolyPath);
	return PolyPath;
}


#undef LOCTEXT_NAMESPACE
