// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/PolygonFunctions.h"

#include "Curve/GeneralPolygon2.h"
#include "Curve/PolygonIntersectionUtils.h"
#include "Curve/PolygonOffsetUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolygonFunctions)

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_PolygonFunctions"

namespace
{
	// Helper to support looping the vertex index
	static int32 PosMod(int32 InVertexIndex, int32 VertexCount)
	{
		int32 Index = InVertexIndex % VertexCount;
		if (Index < 0)
		{
			Index += VertexCount;
		}
		return Index;
	}

	// Helper to safely get the polygon at the requested index, or null if no such polygon exists
	static const UE::Geometry::FGeneralPolygon2d* GetGeneralPolygonHelper(const FGeometryScriptGeneralPolygonList& PolygonList, int PolygonIndex)
	{
		if (PolygonList.Polygons.IsValid())
		{
			const TArray<UE::Geometry::FGeneralPolygon2d>& Polygons = *(PolygonList.Polygons);
			if (Polygons.IsValidIndex(PolygonIndex))
			{
				return &Polygons[PolygonIndex];
			}
		}

		return nullptr;
	}

	static UE::Geometry::FGeneralPolygon2d* GetGeneralPolygonHelper(FGeometryScriptGeneralPolygonList& PolygonList, int PolygonIndex)
	{
		if (PolygonList.Polygons.IsValid())
		{
			TArray<UE::Geometry::FGeneralPolygon2d>& Polygons = *(PolygonList.Polygons);
			if (Polygons.IsValidIndex(PolygonIndex))
			{
				return &Polygons[PolygonIndex];
			}
		}

		return nullptr;
	}

	// Helper to safely get the polygon at the requested indices, or null if no such polygon exists
	// if HoleIndex is -1, returns the outer polygon
	static const UE::Geometry::FPolygon2d* GetSubPolygonHelper(const FGeometryScriptGeneralPolygonList& PolygonList, int PolygonIndex, int32 HoleIndex)
	{
		const UE::Geometry::FGeneralPolygon2d* GeneralPolygon = GetGeneralPolygonHelper(PolygonList, PolygonIndex);
		if (GeneralPolygon)
		{
			if (HoleIndex == INDEX_NONE)
			{
				return &GeneralPolygon->GetOuter();
			}
			else
			{
				const TArray<UE::Geometry::FPolygon2d>& Holes = GeneralPolygon->GetHoles();
				if (Holes.IsValidIndex(HoleIndex))
				{
					return &Holes[HoleIndex];
				}
			}
		}

		return nullptr;
	}
}


int32 UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonVertexCount(FGeometryScriptSimplePolygon Polygon)
{
	if (Polygon.Vertices.IsValid())
	{
		return Polygon.Vertices->Num();
	}
	return 0;
}

FVector2D UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonVertex(FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, bool& bPolygonIsEmpty)
{
	bPolygonIsEmpty = !Polygon.Vertices.IsValid() || Polygon.Vertices->IsEmpty();
	if (!bPolygonIsEmpty)
	{
		return (*Polygon.Vertices)[PosMod(VertexIndex, Polygon.Vertices->Num())];
	}
	return FVector2D::ZeroVector;
}

void UGeometryScriptLibrary_SimplePolygonFunctions::SetPolygonVertex(FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, FVector2D VertexPosition, bool& bPolygonIsEmpty)
{
	bPolygonIsEmpty = !Polygon.Vertices.IsValid() || Polygon.Vertices->IsEmpty();
	if (!bPolygonIsEmpty)
	{
		(*Polygon.Vertices)[PosMod(VertexIndex, Polygon.Vertices->Num())] = VertexPosition;
	}
}

int32 UGeometryScriptLibrary_SimplePolygonFunctions::AddPolygonVertex(FGeometryScriptSimplePolygon& Polygon, FVector2D VertexPosition)
{
	if (!Polygon.Vertices.IsValid())
	{
		Polygon.Reset();
	}
	return Polygon.Vertices->Add(VertexPosition);
}

FVector2D UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonTangent(FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, bool& bPolygonIsEmpty)
{
	bPolygonIsEmpty = !Polygon.Vertices.IsValid() || Polygon.Vertices->IsEmpty();
	if (!bPolygonIsEmpty)
	{
		VertexIndex = PosMod(VertexIndex, Polygon.Vertices->Num());
		return UE::Geometry::CurveUtil::Tangent<double, FVector2D>(*Polygon.Vertices, VertexIndex, true);
	}
	return FVector2D::ZeroVector;
}

double UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonArcLength(FGeometryScriptSimplePolygon Polygon)
{
	if (Polygon.Vertices.IsValid() && !Polygon.Vertices->IsEmpty())
	{
		return UE::Geometry::CurveUtil::ArcLength<double, FVector2D>(*Polygon.Vertices, true);
	}
	return 0.0;
}

double UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonArea(FGeometryScriptSimplePolygon Polygon)
{
	if (Polygon.Vertices.IsValid() && !Polygon.Vertices->IsEmpty())
	{
		return UE::Geometry::CurveUtil::SignedArea2<double, FVector2D>(*Polygon.Vertices);
	}
	return 0.0;
}

FBox2D UGeometryScriptLibrary_SimplePolygonFunctions::GetPolygonBounds(FGeometryScriptSimplePolygon Polygon)
{
	if (Polygon.Vertices.IsValid() && !Polygon.Vertices->IsEmpty())
	{
		return FBox2D(*Polygon.Vertices);
	}
	return FBox2D(ForceInit);
}

void UGeometryScriptLibrary_SimplePolygonFunctions::ConvertSplineToPolygon(const USplineComponent* Spline, FGeometryScriptSimplePolygon& Polygon, FGeometryScriptSplineSamplingOptions SamplingOptions, EGeometryScriptAxis DropAxis)
{
	Polygon.Reset();
	if (Spline)
	{
		int32 Keep0 = int32(DropAxis == EGeometryScriptAxis::X);
		int32 Keep1 = 1 + int32(DropAxis != EGeometryScriptAxis::Z);
		auto ProjDropAxis = [Keep0, Keep1](const FVector& V3D) -> FVector2D
		{
			return FVector2D(V3D[Keep0], V3D[Keep1]);
		};

		bool bIsLoop = Spline->IsClosedLoop();
		if (SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance)
		{
			TArray<FVector> Path;
			float SquaredErrorTolerance = FMath::Max(KINDA_SMALL_NUMBER, SamplingOptions.ErrorTolerance * SamplingOptions.ErrorTolerance);
			Spline->ConvertSplineToPolyLine(SamplingOptions.CoordinateSpace, SquaredErrorTolerance, Path);
			if (bIsLoop)
			{
				Path.Pop(); // delete the duplicate end-point for loops
			}

			// Copy to FVector2D array
			Polygon.Vertices->SetNum(Path.Num());
			for (int32 Idx = 0; Idx < Path.Num(); ++Idx)
			{
				(*Polygon.Vertices)[Idx] = ProjDropAxis(Path[Idx]);
			}
		}
		else
		{
			float Duration = Spline->Duration;

			bool bUseConstantVelocity = SamplingOptions.SampleSpacing == EGeometryScriptSampleSpacing::UniformDistance;
			int32 UseSamples = FMath::Max(2, SamplingOptions.NumSamples); // Always use at least 2 samples
			// In non-loops, we adjust DivNum so we exactly sample the end of the spline
			// In loops we don't sample the endpoint, by convention, as it's the same as the start
			float DivNum = float(UseSamples - (int32)!bIsLoop);
			Polygon.Vertices->Reserve(UseSamples);
			for (int32 Idx = 0; Idx < UseSamples; Idx++)
			{
				float Time = Duration * ((float)Idx / DivNum);
				Polygon.Vertices->Add(ProjDropAxis(Spline->GetLocationAtTime(Time, SamplingOptions.CoordinateSpace, bUseConstantVelocity)));
			}
		}

	}
}

TArray<FVector> UGeometryScriptLibrary_SimplePolygonFunctions::Conv_GeometryScriptSimplePolygonToArray(FGeometryScriptSimplePolygon Polygon)
{
	TArray<FVector> Points3d;
	if (Polygon.Vertices.IsValid())
	{
		Points3d.Reserve(Polygon.Vertices->Num());
		for (const FVector2D& V : *Polygon.Vertices)
		{
			Points3d.Emplace(V.X, V.Y, 0.0);
		}
	}
	return Points3d;
}

TArray<FVector2D> UGeometryScriptLibrary_SimplePolygonFunctions::Conv_GeometryScriptSimplePolygonToArrayOfVector2D(FGeometryScriptSimplePolygon Polygon)
{
	if (Polygon.Vertices.IsValid())
	{
		return *Polygon.Vertices;
	}
	return TArray<FVector2D>();
}

FGeometryScriptSimplePolygon UGeometryScriptLibrary_SimplePolygonFunctions::Conv_ArrayToGeometryScriptSimplePolygon(const TArray<FVector>& PathVertices)
{
	FGeometryScriptSimplePolygon Polygon;
	Polygon.Reset();
	Polygon.Vertices->Reserve(PathVertices.Num());
	for (FVector V : PathVertices)
	{
		Polygon.Vertices->Emplace(V.X, V.Y);
	}
	return Polygon;
}

FGeometryScriptSimplePolygon UGeometryScriptLibrary_SimplePolygonFunctions::Conv_ArrayOfVector2DToGeometryScriptSimplePolygon(const TArray<FVector2D>& PathVertices)
{
	FGeometryScriptSimplePolygon Polygon;
	Polygon.Reset();
	*Polygon.Vertices = PathVertices;
	return Polygon;
}

int UGeometryScriptLibrary_PolygonListFunctions::GetPolygonCount(FGeometryScriptGeneralPolygonList PolygonList)
{
	return PolygonList.Polygons.IsValid() ? PolygonList.Polygons->Num() : 0;
}

int UGeometryScriptLibrary_PolygonListFunctions::GetPolygonVertexCount(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndices, int PolygonIndex, int HoleIndex)
{
	if (const UE::Geometry::FPolygon2d* Polygon = GetSubPolygonHelper(PolygonList, PolygonIndex, HoleIndex))
	{
		bValidIndices = true;
		return Polygon->VertexCount();
	}
	bValidIndices = false;
	return 0;
}

int UGeometryScriptLibrary_PolygonListFunctions::GetPolygonHoleCount(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int PolygonIndex)
{
	if (const UE::Geometry::FGeneralPolygon2d* Polygon = GetGeneralPolygonHelper(PolygonList, PolygonIndex))
	{
		bValidIndex = true;
		return Polygon->GetHoles().Num();
	}
	bValidIndex = false;
	return 0;
}

void UGeometryScriptLibrary_PolygonListFunctions::GetPolygonVertices(FGeometryScriptGeneralPolygonList PolygonList, TArray<FVector2D>& OutVertices, bool& bValidIndices, int PolygonIndex, int HoleIndex)
{
	if (const UE::Geometry::FPolygon2d* Polygon = GetSubPolygonHelper(PolygonList, PolygonIndex, HoleIndex))
	{
		bValidIndices = true;
		OutVertices = Polygon->GetVertices();
	}
	bValidIndices = false;
}

FGeometryScriptSimplePolygon UGeometryScriptLibrary_PolygonListFunctions::GetSimplePolygon(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndices, int32 PolygonIndex, int32 HoleIndex)
{
	FGeometryScriptSimplePolygon ToRet;
	ToRet.Reset();
	if (const UE::Geometry::FPolygon2d* Polygon = GetSubPolygonHelper(PolygonList, PolygonIndex, HoleIndex))
	{
		bValidIndices = true;
		*ToRet.Vertices = Polygon->GetVertices();
	}
	bValidIndices = false;
	return ToRet;
}

FVector2D UGeometryScriptLibrary_PolygonListFunctions::GetPolygonVertex(FGeometryScriptGeneralPolygonList PolygonList, bool& bIsValidVertex, int32 VertexIndex, int PolygonIndex, int HoleIndex)
{
	if (const UE::Geometry::FPolygon2d* Polygon = GetSubPolygonHelper(PolygonList, PolygonIndex, HoleIndex))
	{
		const TArray<FVector2D>& Vertices = Polygon->GetVertices();
		if (!Vertices.IsEmpty())
		{
			bIsValidVertex = true;
			return Vertices[PosMod(VertexIndex, Vertices.Num())];
		}
	}
	bIsValidVertex = false;
	UE_LOGF(LogGeometry, Warning, "GetPolygonVertex: No vertex found in Polygon List at Polygon Index: %d, Hole Index: %d, Vertex Index: %d", PolygonIndex, HoleIndex, VertexIndex);
	return FVector2D(0, 0);
}

double UGeometryScriptLibrary_PolygonListFunctions::GetPolygonArea(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int PolygonIndex)
{
	if (const UE::Geometry::FGeneralPolygon2d* Polygon = GetGeneralPolygonHelper(PolygonList, PolygonIndex))
	{
		bValidIndex = true;
		return Polygon->SignedArea();
	}
	bValidIndex = false;
	return 0;
}

double UGeometryScriptLibrary_PolygonListFunctions::GetPolygonListArea(FGeometryScriptGeneralPolygonList PolygonList)
{
	double AreaSum = 0.0;
	if (PolygonList.Polygons.IsValid())
	{
		for (const UE::Geometry::FGeneralPolygon2d& Polygon : *PolygonList.Polygons)
		{
			AreaSum += Polygon.SignedArea();
		}
	}
	return AreaSum;
}

FBox2D UGeometryScriptLibrary_PolygonListFunctions::GetPolygonBounds(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int PolygonIndex)
{
	if (const UE::Geometry::FGeneralPolygon2d* Polygon = GetGeneralPolygonHelper(PolygonList, PolygonIndex))
	{
		bValidIndex = true;
		return (FBox2D)Polygon->Bounds();
	}
	bValidIndex = false;
	return FBox2D(ForceInit);
}

FBox2D UGeometryScriptLibrary_PolygonListFunctions::GetPolygonListBounds(FGeometryScriptGeneralPolygonList PolygonList)
{
	UE::Geometry::FAxisAlignedBox2d Bounds;
	if (PolygonList.Polygons.IsValid())
	{
		for (const UE::Geometry::FGeneralPolygon2d& Polygon : *PolygonList.Polygons)
		{
			Bounds.Contain(Polygon.Bounds());
		}
	}
	return FBox2D(Bounds);
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::CreatePolygonListFromSinglePolygon(FGeometryScriptSimplePolygon OuterShape, const TArray<FGeometryScriptSimplePolygon>& Holes, bool bFixHoleOrientations)
{
	FGeometryScriptGeneralPolygonList PolygonList;
	PolygonList.Reset();
	AddPolygonToList(PolygonList, OuterShape, Holes, bFixHoleOrientations);
	return PolygonList;
}

int32 UGeometryScriptLibrary_PolygonListFunctions::AddPolygonToList(FGeometryScriptGeneralPolygonList& PolygonList, FGeometryScriptSimplePolygon OuterShape, const TArray<FGeometryScriptSimplePolygon>& Holes, bool bFixHoleOrientations)
{
	if (!PolygonList.Polygons.IsValid())
	{
		PolygonList.Reset();
	}

	TArray<FVector2D>* UseOuterVertices = OuterShape.Vertices.Get();
	TArray<FVector2D> EmptyVertices;
	if (!OuterShape.Vertices.IsValid())
	{
		// Fall back to an empty outer shape
		UseOuterVertices = &EmptyVertices;
	}
	int32 NewPolygonIndex = PolygonList.Polygons->Num();
	UE::Geometry::FGeneralPolygon2d& AddedPolygon = PolygonList.Polygons->Emplace_GetRef(*UseOuterVertices);
	for (const FGeometryScriptSimplePolygon& Hole : Holes)
	{
		if (!Hole.Vertices)
		{
			continue;
		}
		UE::Geometry::FPolygon2d HolePolygon(*Hole.Vertices);
		if (bFixHoleOrientations && HolePolygon.IsClockwise() == AddedPolygon.OuterIsClockwise())
		{
			HolePolygon.Reverse();
		}
		AddedPolygon.AddHole(HolePolygon, false, false);
	}
	return NewPolygonIndex;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::CreatePolygonListFromSimplePolygons(const TArray<FGeometryScriptSimplePolygon>& OuterPolygons)
{
	FGeometryScriptGeneralPolygonList PolygonList;
	PolygonList.Reset();
	for (const FGeometryScriptSimplePolygon& Polygon : OuterPolygons)
	{
		if (Polygon.Vertices.IsValid())
		{
			PolygonList.Polygons->Emplace(*Polygon.Vertices.Get());
		}
	}
	return PolygonList;
}

void UGeometryScriptLibrary_PolygonListFunctions::AppendPolygonList(FGeometryScriptGeneralPolygonList& PolygonList, FGeometryScriptGeneralPolygonList PolygonsToAppend)
{
	if (!PolygonList.Polygons)
	{
		PolygonList.Reset();
	}
	if (!PolygonsToAppend.Polygons)
	{
		return;
	}
	PolygonList.Polygons->Append(*PolygonsToAppend.Polygons);
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsUnion(FGeometryScriptGeneralPolygonList PolygonList, bool bCopyInputOnFailure)
{
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons)
	{
		return PolygonListResult;
	}
	UE::Geometry::PolygonsUnion(*PolygonList.Polygons, *PolygonListResult.Polygons, bCopyInputOnFailure);
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsDifference(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToSubtract)
{
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons)
	{
		return PolygonListResult;
	}
	if (!PolygonsToSubtract.Polygons)
	{
		// Subtracting nothing
		*PolygonListResult.Polygons = *PolygonList.Polygons;
	}
	UE::Geometry::PolygonsDifference(*PolygonList.Polygons, *PolygonsToSubtract.Polygons, *PolygonListResult.Polygons);
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsIntersection(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToSubtract)
{
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons || !PolygonsToSubtract.Polygons)
	{
		return PolygonListResult;
	}
	UE::Geometry::PolygonsIntersection(*PolygonList.Polygons, *PolygonsToSubtract.Polygons, *PolygonListResult.Polygons);
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsExclusiveOr(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToSubtract)
{
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons || !PolygonsToSubtract.Polygons)
	{
		return PolygonListResult;
	}
	UE::Geometry::PolygonsExclusiveOr(*PolygonList.Polygons, *PolygonsToSubtract.Polygons, *PolygonListResult.Polygons);
	return PolygonListResult;
}

namespace
{
	// Helper to convert the polygon/polypath join types from the GeometryScript enum to the corresponding GeometryAlgorithms enum
	static UE::Geometry::EPolygonOffsetJoinType ConvertJoinTypeEnum(EGeometryScriptPolyOffsetJoinType GeometryScriptJoinType)
	{
		using namespace UE::Geometry;
		switch (GeometryScriptJoinType)
		{
		case EGeometryScriptPolyOffsetJoinType::Square:
			return EPolygonOffsetJoinType::Square;
		case EGeometryScriptPolyOffsetJoinType::Round:
			return EPolygonOffsetJoinType::Round;
		case EGeometryScriptPolyOffsetJoinType::Miter:
			return EPolygonOffsetJoinType::Miter;
		default:
			checkNoEntry();
		}
		return EPolygonOffsetJoinType::Square;
	}

	// Helper to convert the open-path end types from the GeometryScript enum to the corresponding GeometryAlgorithms enum
	static UE::Geometry::EPolygonOffsetEndType ConvertEndTypeEnum(EGeometryScriptPathOffsetEndType GeometryScriptEndType)
	{
		using namespace UE::Geometry;
		switch (GeometryScriptEndType)
		{
		case EGeometryScriptPathOffsetEndType::Butt:
			return EPolygonOffsetEndType::Butt;
		case EGeometryScriptPathOffsetEndType::Square:
			return EPolygonOffsetEndType::Square;
		case EGeometryScriptPathOffsetEndType::Round:
			return EPolygonOffsetEndType::Round;
		default:
			checkNoEntry();
		}
		return EPolygonOffsetEndType::Square;
	}
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsOffset(
	FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions, double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	using namespace UE::Geometry;
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons)
	{
		return PolygonListResult;
	}
	bOperationSuccess = UE::Geometry::PolygonsOffset(
		Offset, *PolygonList.Polygons, *PolygonListResult.Polygons, bCopyInputOnFailure, OffsetOptions.MiterLimit,
		ConvertJoinTypeEnum(OffsetOptions.JoinType),
		OffsetOptions.bOffsetBothSides ? EPolygonOffsetEndType::Joined : EPolygonOffsetEndType::Polygon,
		OffsetOptions.MaximumStepsPerRadian, OffsetOptions.StepsPerRadianScale);
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsOffsets(
	FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions, double FirstOffset, double SecondOffset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	using namespace UE::Geometry;
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!PolygonList.Polygons)
	{
		return PolygonListResult;
	}
	bOperationSuccess = UE::Geometry::PolygonsOffsets(
		FirstOffset, SecondOffset, *PolygonList.Polygons, *PolygonListResult.Polygons, bCopyInputOnFailure, OffsetOptions.MiterLimit,
		ConvertJoinTypeEnum(OffsetOptions.JoinType),
		OffsetOptions.bOffsetBothSides ? EPolygonOffsetEndType::Joined : EPolygonOffsetEndType::Polygon,
		OffsetOptions.MaximumStepsPerRadian, OffsetOptions.StepsPerRadianScale);
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsMorphologyOpen(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
	double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	if (OffsetOptions.bOffsetBothSides)
	{
		UE_LOGF(LogGeometry, Warning, "The polygons Morphology Open operation was called with 'Offset Both Sides' enabled; this is likely not intended.");
	}
	return PolygonsOffsets(PolygonList, OffsetOptions, -Offset, +Offset, bOperationSuccess, bCopyInputOnFailure);
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::PolygonsMorphologyClose(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
	double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	if (OffsetOptions.bOffsetBothSides)
	{
		UE_LOGF(LogGeometry, Warning, "The polygons Morphology Close operation was called with 'Offset Both Sides' enabled; this is likely not intended.");
	}
	return PolygonsOffsets(PolygonList, OffsetOptions, +Offset, -Offset, bOperationSuccess, bCopyInputOnFailure);
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::CreatePolygonsFromPathOffset(TArray<FVector2D> Path, FGeometryScriptOpenPathOffsetOptions OffsetOptions,
	double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	using namespace UE::Geometry;
	FOffsetPolygon2d PolygonOffset;
	PolygonOffset.EndType = ConvertEndTypeEnum(OffsetOptions.EndType);
	PolygonOffset.JoinType = ConvertJoinTypeEnum(OffsetOptions.JoinType);
	PolygonOffset.MiterLimit = OffsetOptions.MiterLimit;
	PolygonOffset.MaxStepsPerRadian = OffsetOptions.MaximumStepsPerRadian;
	PolygonOffset.DefaultStepsPerRadianScale = OffsetOptions.StepsPerRadianScale;
	PolygonOffset.Offset = Offset;
	PolygonOffset.Polygons.Add(TArrayView<FVector2D>(Path.GetData(), Path.Num()));
	bOperationSuccess = PolygonOffset.ComputeResult();
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!bOperationSuccess && bCopyInputOnFailure)
	{
		PolygonListResult.Polygons->Emplace(FPolygon2d(Path));
	}
	else
	{
		*PolygonListResult.Polygons = MoveTemp(PolygonOffset.Result);
	}
	return PolygonListResult;
}

FGeometryScriptGeneralPolygonList UGeometryScriptLibrary_PolygonListFunctions::CreatePolygonsFromOpenPolyPathsOffset(TArray<FGeometryScriptPolyPath> PolyPaths, FGeometryScriptOpenPathOffsetOptions OffsetOptions,
	double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure)
{
	using namespace UE::Geometry;
	FOffsetPolygon2d PolygonOffset;
	PolygonOffset.EndType = ConvertEndTypeEnum(OffsetOptions.EndType);
	PolygonOffset.JoinType = ConvertJoinTypeEnum(OffsetOptions.JoinType);
	PolygonOffset.MiterLimit = OffsetOptions.MiterLimit;
	PolygonOffset.MaxStepsPerRadian = OffsetOptions.MaximumStepsPerRadian;
	PolygonOffset.DefaultStepsPerRadianScale = OffsetOptions.StepsPerRadianScale;

	PolygonOffset.Offset = Offset;
	// Convert PolyPath's FVector paths to FVector2D, and pass to PolygonOffset
	TArray<FVector2D> AllPathPts;
	TArray<int32> PathLens;
	PathLens.Reserve(PolyPaths.Num());
	for (int32 Idx = 0; Idx < PolyPaths.Num(); ++Idx)
	{
		const TSharedPtr<TArray<FVector>>& Path = PolyPaths[Idx].Path;
		if (Path.IsValid() && !Path->IsEmpty())
		{
			int32 PathLen = Path->Num();
			int32 Start = AllPathPts.Num();
			for (int32 PathIdx = 0; PathIdx < PathLen; ++PathIdx)
			{
				const FVector& Pt = (*Path)[PathIdx];
				AllPathPts.Emplace(Pt.X, Pt.Y);
			}
			PathLens.Add(PathLen);
		}
	}
	PolygonOffset.Polygons.Reserve(PathLens.Num());
	for (int32 Idx = 0, PathPtIdx = 0; Idx < PathLens.Num(); PathPtIdx += PathLens[Idx++])
	{
		PolygonOffset.Polygons.Add(TArrayView<FVector2D>(AllPathPts.GetData() + PathPtIdx, PathLens[Idx]));
	}
	bOperationSuccess = PolygonOffset.ComputeResult();
	FGeometryScriptGeneralPolygonList PolygonListResult;
	PolygonListResult.Reset();
	if (!bOperationSuccess && bCopyInputOnFailure)
	{
		for (const FGeometryScriptPolyPath& Path : PolyPaths)
		{
			if (Path.Path)
			{
				PolygonListResult.Polygons->Emplace(FPolygon2d(*Path.Path));
			}
		}
	}
	else
	{
		*PolygonListResult.Polygons = MoveTemp(PolygonOffset.Result);
	}
	return PolygonListResult;
}

#undef LOCTEXT_NAMESPACE
