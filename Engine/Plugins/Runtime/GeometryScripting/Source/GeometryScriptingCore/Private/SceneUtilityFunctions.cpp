// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/SceneUtilityFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Operations/DetectExteriorVisibility.h"

#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "ConversionUtils/SkinnedMeshToDynamicMesh.h"
#include "ConversionUtils/SplineComponentDeformDynamicMesh.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Physics/ComponentCollisionUtil.h"
#include "PlanarCut.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneUtilityFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_SceneUtilityFunctions"

UDynamicMeshPool* UGeometryScriptLibrary_SceneUtilityFunctions::CreateDynamicMeshPool()
{
	return NewObject<UDynamicMeshPool>();
}


UDynamicMesh* UGeometryScriptLibrary_SceneUtilityFunctions::CopyMeshFromComponent(
	USceneComponent* Component,
	UDynamicMesh* ToDynamicMesh,
	FGeometryScriptCopyMeshFromComponentOptions Options,
	bool bTransformToWorld,
	FTransform& LocalToWorld,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;
	
	if (ToDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyMeshFromComponent_InvalidInput", "CopyMeshFromComponent: ToDynamicMesh is Null"));
		return ToDynamicMesh;
	}

	UE::Conversion::FToMeshOptions ToMeshOptions;
	ToMeshOptions.bUseClosestLOD = false;
	ToMeshOptions.LODIndex = Options.RequestedLOD.LODIndex;
	auto SafeConvertLODType = [](EGeometryScriptLODType GSLODType) -> UE::Conversion::EMeshLODType
	{
		switch (GSLODType)
		{
		case EGeometryScriptLODType::MaxAvailable:
			return UE::Conversion::EMeshLODType::MaxAvailable;
		case EGeometryScriptLODType::HiResSourceModel:
			return UE::Conversion::EMeshLODType::HiResSourceModel;
		case EGeometryScriptLODType::SourceModel:
			return UE::Conversion::EMeshLODType::SourceModel;
		case EGeometryScriptLODType::RenderData:
			return UE::Conversion::EMeshLODType::RenderData;
		}
		checkNoEntry();
		return UE::Conversion::EMeshLODType::MaxAvailable;
	};
	ToMeshOptions.LODType = SafeConvertLODType(Options.RequestedLOD.LODType);
	ToMeshOptions.bWantNormals = Options.bWantNormals;
	ToMeshOptions.bWantTangents = Options.bWantTangents;
	ToMeshOptions.bWantInstanceColors = Options.bWantInstanceColors;
	UE::Geometry::FDynamicMesh3 NewMesh;
	FText ErrorMessage;
	bool bSuccess = UE::Conversion::SceneComponentToDynamicMesh(Component, ToMeshOptions, bTransformToWorld, NewMesh, LocalToWorld, ErrorMessage);
	if (bSuccess)
	{
		ToDynamicMesh->SetMesh(MoveTemp(NewMesh));
		Outcome = EGeometryScriptOutcomePins::Success;
	}
	else // failed
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CopyMeshFromComponent_ConversionError", "CopyMeshFromComponent Error: {0}"), ErrorMessage));
	}

	return ToDynamicMesh;
}



void UGeometryScriptLibrary_SceneUtilityFunctions::SetComponentMaterialList(
	UPrimitiveComponent* Component,
	const TArray<UMaterialInterface*>& MaterialList,
	UGeometryScriptDebug* Debug)
{
	if (Component == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SetComponentMaterialList_InvalidInput1", "SetComponentMaterialList: FromStaticMeshAsset is Null"));
		return;
	}

	for (int32 k = 0; k < MaterialList.Num(); ++k)
	{
		Component->SetMaterial(k, MaterialList[k]);
	}
}




UDynamicMesh* UGeometryScriptLibrary_SceneUtilityFunctions::CopyCollisionMeshesFromObject(
	UObject* FromObject,
	UDynamicMesh* ToDynamicMesh,
	bool bTransformToWorld,
	FTransform& LocalToWorld,
	EGeometryScriptOutcomePins& Outcome,
	bool bUseComplexCollision,
	int SphereResolution,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (ToDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyCollisionMeshesFromObject_InvalidInput", "CopyCollisionMeshesFromObject: ToDynamicMesh is Null"));
		return ToDynamicMesh;
	}

	FDynamicMesh3 AccumulatedMesh;
	ToDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		AccumulatedMesh.EnableMatchingAttributes(ReadMesh, true);
	});

	// Get the transform from the component (if applicable)
	LocalToWorld = FTransform::Identity;
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(FromObject))
	{
		LocalToWorld = SceneComponent->GetComponentTransform();
	}

	if (bUseComplexCollision)
	{
		// find the Complex Collision mesh interface
		IInterface_CollisionDataProvider* CollisionProvider = Cast<IInterface_CollisionDataProvider>(FromObject);
		// For static and skeletal mesh cases, try pulling from the asset if the component didn't give us collision data
		if (!CollisionProvider || !CollisionProvider->ContainsPhysicsTriMeshData(true))
		{
			if (USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(FromObject))
			{
				USkeletalMesh* SkeletalMesh = SkeletalMeshComp->GetSkeletalMeshAsset();
				CollisionProvider = Cast<IInterface_CollisionDataProvider>(SkeletalMesh);
			}
			else if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(FromObject))
			{
				UStaticMesh* StaticMesh = StaticMeshComp->GetStaticMesh();
				CollisionProvider = Cast<IInterface_CollisionDataProvider>(StaticMesh);
			}
		}

		if (CollisionProvider == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyCollisionMeshesFromObject_NoCollisionProvider", "CopyCollisionMeshesFromObject: Complex Collision Provider is not available for this object type"));
			return ToDynamicMesh;
		}

		FTransformSequence3d Transforms;
		if (bTransformToWorld)
		{
			Transforms.Append(LocalToWorld);
		}

		// generate mesh
		bool bFoundMeshErrors = false;
		UE::Geometry::ConvertComplexCollisionToMeshes(CollisionProvider, AccumulatedMesh, Transforms, bFoundMeshErrors, true, true);
	}
	else // simple collision
	{
		const UBodySetup* BodySetup = nullptr;
		if (UPrimitiveComponent* AnyComponent = Cast<UPrimitiveComponent>(FromObject))
		{
			BodySetup = AnyComponent->GetBodySetup();
		}
		else if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(FromObject))
		{
			BodySetup = StaticMesh->GetBodySetup();
		}

		if (BodySetup == nullptr)
		{
			UE::Geometry::AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CopyCollisionMeshesFromObject_NoBodySetup", "CopyCollisionMeshesFromObject: BodySetup is null or Object type is not supported"));
			return ToDynamicMesh;
		}

		FTransformSequence3d Transforms;
		FVector ExternalScale = FVector::OneVector;
		if (bTransformToWorld)
		{
			ExternalScale = LocalToWorld.GetScale3D();
			FTransform WithoutScale = LocalToWorld;
			WithoutScale.SetScale3D(FVector::OneVector);
			Transforms.Append(WithoutScale);
		}

		UE::Geometry::ConvertSimpleCollisionToMeshes(BodySetup->AggGeom, AccumulatedMesh, Transforms, SphereResolution, true, true, nullptr, false, ExternalScale);
	}

	ToDynamicMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		EditMesh = MoveTemp(AccumulatedMesh);
	});

	Outcome = EGeometryScriptOutcomePins::Success;

	return ToDynamicMesh;
}

void UGeometryScriptLibrary_SceneUtilityFunctions::DetermineMeshOcclusion(
	const TArray<UDynamicMesh*>& SourceMeshes,
	const TArray<FTransform>& SourceMeshTransforms,
	TArray<bool>& OutMeshIsHidden,
	const TArray<UDynamicMesh*>& TransparentMeshes,
	const TArray<FTransform>& TransparentMeshTransforms,
	TArray<bool>& OutTransparentMeshIsHidden,
	const TArray<UDynamicMesh*>& OccludeMeshes,
	const TArray<FTransform>& OccludeMeshTransforms,
	const FGeometryScriptDetermineMeshOcclusionOptions& OcclusionOptions,
	UGeometryScriptDebug* Debug)
{
	if (SourceMeshes.Num() != SourceMeshTransforms.Num())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_SourceArrayMismatch", "DetermineMeshOcclusion: SourceMeshes and SourceMeshTransforms arrays must have same length"));
		return;
	}
	if (OccludeMeshes.Num() != OccludeMeshTransforms.Num())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_OccludeArrayMismatch", "DetermineMeshOcclusion: OccludeMeshes and OccludeMeshTransforms arrays must have same length"));
		return;
	}
	if (TransparentMeshes.Num() != TransparentMeshTransforms.Num())
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_TransparentArrayMismatch", "DetermineMeshOcclusion: TransparentMeshes and TransparentMeshTransforms arrays must have same length"));
		return;
	}

	FDetectPerDynamicMeshExteriorVisibility Occlusion;
	auto AddInstances = [](const TArray<UDynamicMesh*>& Meshes, const TArray<FTransform>& Transforms, TArray<FDetectPerDynamicMeshExteriorVisibility::FDynamicMeshInstance>& OutInstances) -> bool
	{
		for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); ++MeshIndex)
		{
			UDynamicMesh* Mesh = Meshes[MeshIndex];
			if (!Mesh)
			{
				return false;
			}
			OutInstances.Emplace(Mesh->GetMeshPtr(), Transforms[MeshIndex]);
		}
		return true;
	};
	if (!AddInstances(SourceMeshes, SourceMeshTransforms, Occlusion.Instances))
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_InvalidSourceMesh", "DetermineMeshOcclusion: SourceMeshes array contained null mesh"));
		return;
	}
	if (!AddInstances(OccludeMeshes, OccludeMeshTransforms, Occlusion.OccludeInstances))
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_InvalidOccludeMesh", "DetermineMeshOcclusion: OccludeMeshes array contained null mesh"));
		return;
	}
	if (!AddInstances(TransparentMeshes, TransparentMeshTransforms, Occlusion.TransparentInstances))
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("DetermineMeshOcclusion_InvalidTransparentMesh", "DetermineMeshOcclusion: TransparentMeshes array contained null mesh"));
		return;
	}
	Occlusion.SamplingParameters.bDoubleSided = OcclusionOptions.bDoubleSided;
	Occlusion.SamplingParameters.SamplingDensity = OcclusionOptions.SamplingDensity;
	Occlusion.SamplingParameters.NumSearchDirections = OcclusionOptions.NumSearchDirections;

	Occlusion.ComputeHidden(OutMeshIsHidden, &OutTransparentMeshIsHidden);
}

#undef LOCTEXT_NAMESPACE