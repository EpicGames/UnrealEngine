// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/ShapeFunctions.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "OrientedBoxTypes.h"
#include "Intersection/IntersectionUtil.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "Distance/DistLine3Ray3.h"
#include "Distance/DistRay3Segment3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShapeFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_BoxFunctions"



FTransform UGeometryScriptLibrary_TransformFunctions::MakeTransformFromZAxis(FVector Location, FVector ZAxis)
{
	return FFrame3d(Location, ZAxis).ToFTransform();
}


FTransform UGeometryScriptLibrary_TransformFunctions::MakeTransformFromAxes(
	FVector Location, 
	FVector ZAxis, 
	FVector TangentAxis, 
	bool bTangentIsX)
{
	FFrame3d Frame(Location, ZAxis);
	// if axes are parallel we can't do the alignment so just ignore it?
	if (FMathd::Abs(ZAxis.Dot(TangentAxis)) < 0.999)
	{
		Frame.ConstrainedAlignAxis(
			(bTangentIsX) ? 0 : 1, TangentAxis, Frame.Z());
	}
	return Frame.ToFTransform();
}

FVector UGeometryScriptLibrary_TransformFunctions::GetTransformAxisVector(FTransform Transform, EGeometryScriptAxis Axis)
{
	switch(Axis)
	{
		case EGeometryScriptAxis::X: return ((FQuaterniond)Transform.GetRotation()).AxisX();
		case EGeometryScriptAxis::Y: return ((FQuaterniond)Transform.GetRotation()).AxisY();
		default:
		case EGeometryScriptAxis::Z: return ((FQuaterniond)Transform.GetRotation()).AxisZ();
	}
}

FRay UGeometryScriptLibrary_TransformFunctions::GetTransformAxisRay(FTransform Transform, EGeometryScriptAxis Axis)
{
	switch(Axis)
	{
		case EGeometryScriptAxis::X: return FRay( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisX() );
		case EGeometryScriptAxis::Y: return FRay( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisY() );
		default:
		case EGeometryScriptAxis::Z: return FRay( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisZ() );
	}
}

FPlane UGeometryScriptLibrary_TransformFunctions::GetTransformAxisPlane(FTransform Transform, EGeometryScriptAxis Axis)
{
	switch(Axis)
	{
		case EGeometryScriptAxis::X: return FPlane( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisX() );
		case EGeometryScriptAxis::Y: return FPlane( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisY() );
		default:
		case EGeometryScriptAxis::Z: return FPlane( Transform.GetLocation(), ((FQuaterniond)Transform.GetRotation()).AxisZ() );
	}
}


FRay UGeometryScriptLibrary_RayFunctions::MakeRayFromPoints(FVector A, FVector B)
{
	FVector Direction(B-A);
	if (Normalize(Direction) < FMathf::Epsilon)
	{
		Direction = FVector(0,0,1);
	}
	return FRay(A, Direction, true);
}

FRay UGeometryScriptLibrary_RayFunctions::MakeRayFromPointDirection(FVector Origin, FVector Direction, bool bDirectionIsNormalized)
{
	return FRay(Origin, Direction, bDirectionIsNormalized);
}

FRay UGeometryScriptLibrary_RayFunctions::GetTransformedRay(FRay Ray, FTransform TransformIn, bool bInvert)
{
	FTransformSRT3d Transform(TransformIn);
	return (bInvert) ? Transform.InverseTransformRay(Ray) : Transform.TransformRay(Ray);
}

FVector UGeometryScriptLibrary_RayFunctions::GetRayPoint(FRay Ray, double Distance)
{
	return Ray.PointAt(Distance);
}

void UGeometryScriptLibrary_RayFunctions::GetRayStartEnd(FRay Ray, double StartDistance, double EndDistance, FVector& StartPoint, FVector& EndPoint)
{
	StartPoint = Ray.PointAt(StartDistance);
	EndPoint = Ray.PointAt(EndDistance == 0 ? TNumericLimits<float>::Max() : EndDistance );
}

double UGeometryScriptLibrary_RayFunctions::GetRayParameter(FRay Ray, FVector Point)
{
	return Ray.GetParameter(Point);
}

double UGeometryScriptLibrary_RayFunctions::GetRayPointDistance(FRay Ray, FVector Point)
{
	return Ray.Dist(Point);
}

FVector UGeometryScriptLibrary_RayFunctions::GetRayClosestPoint(FRay Ray, FVector Point)
{
	return Ray.ClosestPoint(Point);
}

bool UGeometryScriptLibrary_RayFunctions::GetRaySphereIntersection(FRay Ray, FVector SphereCenter, double SphereRadius, double& Distance1, double& Distance2)
{
	Distance1 = Distance2 = (double)TNumericLimits<float>::Max();
	FLinearIntersection Intersection;
	bool bIntersects = IntersectionUtil::RaySphereIntersection(Ray.Origin, Ray.Direction, SphereCenter, SphereRadius, Intersection);
	if (bIntersects)
	{
		Distance1 = Intersection.parameter.Min;
		Distance2 = (Intersection.numIntersections > 1) ? Intersection.parameter.Max : Intersection.parameter.Min;
	}
	return bIntersects;
}

bool UGeometryScriptLibrary_RayFunctions::GetRayBoxIntersection(FRay Ray, FBox Box, double& HitDistance)
{
	HitDistance = (double)TNumericLimits<float>::Max();
	return FIntrRay3AxisAlignedBox3d::FindIntersection(Ray, FAxisAlignedBox3d(Box), HitDistance);
}

bool UGeometryScriptLibrary_RayFunctions::GetRayPlaneIntersection(FRay Ray, FPlane Plane, double& HitDistance)
{
	const FVector PlaneNormal = FVector(Plane.X, Plane.Y, Plane.Z);
	const FVector PlaneOrigin = PlaneNormal * Plane.W;
	const double DirDotN = Ray.Direction.Dot(PlaneNormal);
	if (FMathd::Abs(DirDotN) > FMathd::ZeroTolerance)
	{
		HitDistance = (Plane.W - Ray.Origin.Dot(PlaneNormal)) / DirDotN;
		return true;
	}
	HitDistance = 0;
	return false;
}


double UGeometryScriptLibrary_RayFunctions::GetRayLineClosestPoint(FRay Ray, FVector LineOrigin, FVector LineDirection, double& RayParameter, FVector& RayPoint, double& LineParameter, FVector& LinePoint)
{
	FDistLine3Ray3d Distance( FLine3d(LineOrigin, LineDirection), Ray );
	double Dist = Distance.Get();
	RayParameter = Distance.RayParameter;
	RayPoint = Distance.RayClosestPoint;
	LineParameter = Distance.LineParameter;
	LinePoint = Distance.LineClosestPoint;
	return Dist;
}

double UGeometryScriptLibrary_RayFunctions::GetRaySegmentClosestPoint(FRay Ray, FVector SegStartPoint, FVector SegEndPoint, double& RayParameter, FVector& RayPoint, FVector& SegPoint)
{
	FDistRay3Segment3d Distance( Ray, FSegment3d(SegStartPoint, SegEndPoint) );
	double Dist = Distance.Get();
	RayParameter = Distance.RayParameter;
	RayPoint = Distance.RayClosestPoint;
	SegPoint = Distance.SegmentClosestPoint;
	return Dist;
}



FBox UGeometryScriptLibrary_BoxFunctions::MakeBoxFromCenterSize(FVector Center, FVector Dimensions)
{
	FVector Extents( FMathd::Max(0, Dimensions.X * 0.5), FMathd::Max(0, Dimensions.Y * 0.5), FMathd::Max(0, Dimensions.Z * 0.5) );
	return FBox(Center - Extents, Center + Extents);
}

FBox UGeometryScriptLibrary_BoxFunctions::MakeBoxFromCenterExtents(FVector Center, FVector Extents)
{
	return FBox(Center - Extents, Center + Extents);
}


void UGeometryScriptLibrary_BoxFunctions::GetBoxCenterSize(FBox Box, FVector& Center, FVector& Dimensions)
{
	FVector Extents;
	Box.GetCenterAndExtents(Center, Extents);
	Dimensions = 2.0 * Extents;
}

FVector UGeometryScriptLibrary_BoxFunctions::GetBoxCorner(FBox Box, int CornerIndex)
{
	CornerIndex = FMath::Clamp(CornerIndex, 0, 7);
	switch (CornerIndex)		// matches Box.GetVertices()
	{
	default:
	case 0: return Box.Min;
	case 1: return FVector(Box.Min.X, Box.Min.Y, Box.Max.Z);
	case 2: return FVector(Box.Min.X, Box.Max.Y, Box.Min.Z);
	case 3: return FVector(Box.Max.X, Box.Min.Y, Box.Min.Z);
	case 4: return FVector(Box.Max.X, Box.Max.Y, Box.Min.Z);
	case 5: return FVector(Box.Max.X, Box.Min.Y, Box.Max.Z);
	case 6: return FVector(Box.Min.X, Box.Max.Y, Box.Max.Z);
	case 7: return Box.Max;
	}
}

FVector UGeometryScriptLibrary_BoxFunctions::GetBoxFaceCenter(FBox Box, int FaceIndex, FVector& FaceNormal)
{
	FaceIndex = FMath::Clamp(FaceIndex, 0, 5);
	FVector Center = 0.5 * (Box.Min + Box.Max);
	switch (FaceIndex)
	{
	default:
	case 0: FaceNormal = FVector(0, 0, -1); return FVector(Center.X, Center.Y, Box.Min.Z);
	case 1: FaceNormal = FVector(0, 0, 1); return FVector(Center.X, Center.Y, Box.Max.Z);
	case 2: FaceNormal = FVector(0, -1, 0); return FVector(Center.X, Box.Min.Y, Center.Z);
	case 3: FaceNormal = FVector(0, 1, 0); return FVector(Center.X, Box.Max.Y, Center.Z);
	case 4: FaceNormal = FVector(-1, 0, 0); return FVector(Box.Min.X, Center.Y, Center.Z);
	case 5: FaceNormal = FVector(1, 0, 0); return FVector(Box.Max.X, Center.Y, Center.Z);
	}
}

void UGeometryScriptLibrary_BoxFunctions::GetBoxVolumeArea(FBox Box, double& Volume, double& SurfaceArea)
{
	FVector Dimensions = Box.GetSize();
	double AreaXY = Dimensions.X * Dimensions.Y;
	double AreaXZ = Dimensions.X * Dimensions.Z;
	double AreaYZ = Dimensions.Y * Dimensions.Z;
	SurfaceArea = 2.0*AreaXY + 2.0*AreaXZ + 2.0*AreaYZ;
	Volume = Dimensions.X * Dimensions.Y * Dimensions.Z;
}

FBox UGeometryScriptLibrary_BoxFunctions::GetExpandedBox(FBox Box, FVector ExpandBy)
{
	FBox Result = Box.ExpandBy(ExpandBy);
	// ExpandBy with negative expansion factor does not clamp to original box center
	for (int32 j = 0; j < 3; ++j)
	{
		if (Result.Min[j] > Result.Max[j])
		{
			Result.Min[j] = Result.Max[j] = 0.5*(Box.Min[j] + Box.Max[j]);
		}
	}
	return Result;
}

FBox UGeometryScriptLibrary_BoxFunctions::GetTransformedBox(FBox Box, FTransform Transform)
{
	return Box.TransformBy(Transform);
}


bool UGeometryScriptLibrary_BoxFunctions::TestBoxBoxIntersection(FBox Box1, FBox Box2)
{
	return Box1.Intersect(Box2);
}


FBox UGeometryScriptLibrary_BoxFunctions::FindBoxBoxIntersection(FBox Box1, FBox Box2, bool& bIsIntersecting)
{
	bIsIntersecting = Box1.Intersect(Box2);
	return Box1.Overlap(Box2);
}

double UGeometryScriptLibrary_BoxFunctions::GetBoxBoxDistance(FBox Box1, FBox Box2)
{
	double DistSqr = FMathd::Max(0.0, Box1.ComputeSquaredDistanceToBox(Box2));
	return FMathd::Sqrt(DistSqr);
}

bool UGeometryScriptLibrary_BoxFunctions::TestPointInsideBox(FBox Box, FVector Point, bool bConsiderOnBoxAsInside)
{
	return (bConsiderOnBoxAsInside) ? Box.IsInsideOrOn(Point) : Box.IsInside(Point);
}

FVector UGeometryScriptLibrary_BoxFunctions::FindClosestPointOnBox(FBox Box, FVector Point, bool& bIsInside)
{
	bIsInside = Box.IsInside(Point);
	return Box.GetClosestPointTo(Point);
}

double UGeometryScriptLibrary_BoxFunctions::GetBoxPointDistance(FBox Box, FVector Point)
{
	double DistSqr = FMathd::Max(0.0, Box.ComputeSquaredDistanceToPoint(Point));
	return FMathd::Sqrt(DistSqr);
}

bool UGeometryScriptLibrary_BoxFunctions::TestBoxSphereIntersection(FBox Box, FVector SphereCenter, double SphereRadius)
{
	return FMath::SphereAABBIntersection(SphereCenter, SphereRadius*SphereRadius, Box);
}

///
/// Oriented Box functions
///

FOrientedBox UGeometryScriptLibrary_OrientedBoxFunctions::MakeOrientedBoxFromBox(FBox Box, FTransform Transform)
{
	FOrientedBox OrientedBox;
	FVector BoxCenter, BoxExtents;
	Box.GetCenterAndExtents(BoxCenter, BoxExtents);
	OrientedBox.Center = Transform.TransformPosition(BoxCenter);
	FVector OrientedExtents = (Transform.GetScale3D() * BoxExtents).GetAbs();
	OrientedBox.ExtentX = OrientedExtents.X;
	OrientedBox.ExtentY = OrientedExtents.Y;
	OrientedBox.ExtentZ = OrientedExtents.Z;
	OrientedBox.AxisX = Transform.GetUnitAxis(EAxis::X);
	OrientedBox.AxisY = Transform.GetUnitAxis(EAxis::Y);
	OrientedBox.AxisZ = Transform.GetUnitAxis(EAxis::Z);
	return OrientedBox;
}

FRotator UGeometryScriptLibrary_OrientedBoxFunctions::GetOrientedBoxOrientation(FOrientedBox Box)
{
	return FRotator(FQuat(FQuaterniond(FMatrix3d(Box.AxisX, Box.AxisY, Box.AxisZ, false))));
}


FVector UGeometryScriptLibrary_OrientedBoxFunctions::GetOrientedBoxCorner(FOrientedBox Box, int CornerIndex)
{
	CornerIndex = FMath::Clamp(CornerIndex, 0, 7);
	
	// matches Box.GetVertices() / GetBoxCorner() (and not the FOrientedBox3d GetCorner ordering)
	auto GetCornerDirection = [](int32 CornerIndex)
		{
			switch (CornerIndex)		
			{
			default:
			case 0: return FVector(-1.);
			case 1: return FVector(-1., -1., 1.);
			case 2: return FVector(-1., 1., -1.);
			case 3: return FVector(1., -1., -1.);
			case 4: return FVector(1., 1., -1.);
			case 5: return FVector(1., -1., 1.);
			case 6: return FVector(-1., 1., 1.);
			case 7: return FVector(1.);
			}
		};
	FVector Directions = GetCornerDirection(CornerIndex);
	return Box.Center
		+ Box.AxisX * Box.ExtentX * Directions.X
		+ Box.AxisY * Box.ExtentY * Directions.Y
		+ Box.AxisZ * Box.ExtentZ * Directions.Z;
}

FVector UGeometryScriptLibrary_OrientedBoxFunctions::GetOrientedBoxFaceCenter(FOrientedBox Box, int FaceIndex, FVector& FaceNormal)
{
	FaceIndex = FMath::Clamp(FaceIndex, 0, 5);
	switch (FaceIndex)
	{
	default:
	case 0:
		FaceNormal = -Box.AxisZ;
		return Box.Center + FaceNormal * Box.ExtentZ;
	case 1:
		FaceNormal = Box.AxisZ;
		return Box.Center + FaceNormal * Box.ExtentZ;
	case 2:
		FaceNormal = -Box.AxisY;
		return Box.Center + FaceNormal * Box.ExtentY;
	case 3:
		FaceNormal = Box.AxisY;
		return Box.Center + FaceNormal * Box.ExtentY;
	case 4:
		FaceNormal = -Box.AxisX;
		return Box.Center + FaceNormal * Box.ExtentX;
	case 5:
		FaceNormal = Box.AxisX;
		return Box.Center + FaceNormal * Box.ExtentX;
	}
}

void UGeometryScriptLibrary_OrientedBoxFunctions::GetOrientedBoxVolumeArea(FOrientedBox Box, double& Volume, double& SurfaceArea)
{
	FVector Dimensions = FVector(Box.ExtentX, Box.ExtentY, Box.ExtentZ) * 2.;
	double AreaXY = Dimensions.X * Dimensions.Y;
	double AreaXZ = Dimensions.X * Dimensions.Z;
	double AreaYZ = Dimensions.Y * Dimensions.Z;
	SurfaceArea = 2.0 * AreaXY + 2.0 * AreaXZ + 2.0 * AreaYZ;
	Volume = Dimensions.X * Dimensions.Y * Dimensions.Z;
}

FOrientedBox UGeometryScriptLibrary_OrientedBoxFunctions::GetExpandedOrientedBox(FOrientedBox Box, FVector ExpandBy, double MinExtent)
{
	// Min Extents cannot be negative
	MinExtent = FMath::Max(0, MinExtent);

	// Expand extents by the provided vector, clamping at MinExtent in case of a negative ExpandBy amount
	Box.ExtentX = FMath::Max(MinExtent, Box.ExtentX + ExpandBy.X);
	Box.ExtentY = FMath::Max(MinExtent, Box.ExtentY + ExpandBy.Y);
	Box.ExtentZ = FMath::Max(MinExtent, Box.ExtentZ + ExpandBy.Z);
	return Box;
}

FOrientedBox UGeometryScriptLibrary_OrientedBoxFunctions::GetTransformedOrientedBox(FOrientedBox OrientedBox, FTransform Transform)
{
	FOrientedBox Result;
	Result.Center = Transform.TransformPosition(OrientedBox.Center);
	FVector ScaledExtents = (Transform.GetScale3D() * FVector(OrientedBox.ExtentX, OrientedBox.ExtentY, OrientedBox.ExtentZ)).GetAbs();
	Result.ExtentX = ScaledExtents.X;
	Result.ExtentY = ScaledExtents.Y;
	Result.ExtentZ = ScaledExtents.Z;
	Result.AxisX = Transform.TransformVectorNoScale(OrientedBox.AxisX);
	Result.AxisY = Transform.TransformVectorNoScale(OrientedBox.AxisY);
	Result.AxisZ = Transform.TransformVectorNoScale(OrientedBox.AxisZ);
	return Result;
}

FBox UGeometryScriptLibrary_OrientedBoxFunctions::ConvertOrientedBoxToAxisAlignedBox(FOrientedBox Box)
{
	FBox Result;

	FVector Size = FVector(Box.ExtentX, Box.ExtentY, Box.ExtentZ) * 2.;
	FVector Corners[8];
	Corners[0] = Box.Center - Box.AxisX * Box.ExtentX - Box.AxisY * Box.ExtentY - Box.ExtentZ * Box.AxisZ;
	Corners[7] = Box.Center + Box.AxisX * Box.ExtentX + Box.AxisY * Box.ExtentY + Box.ExtentZ * Box.AxisZ;
	Corners[1] = Corners[0] + Size.Z * Box.AxisZ;
	Corners[2] = Corners[0] + Size.Y * Box.AxisY;
	Corners[3] = Corners[0] + Size.X * Box.AxisX;
	Corners[4] = Corners[7] - Size.X * Box.AxisX;
	Corners[5] = Corners[7] - Size.Y * Box.AxisY;
	Corners[6] = Corners[7] - Size.Z * Box.AxisZ;
	for (int32 Idx = 0; Idx < 8; ++Idx)
	{
		Result += Corners[Idx];
	}
	return Result;
}

bool UGeometryScriptLibrary_OrientedBoxFunctions::TestPointInsideOrOnOrientedBox(FOrientedBox Box, FVector Point)
{
	UE::Geometry::FOrientedBox3d OBox(Box);
	return OBox.Contains(Point);
}

FVector UGeometryScriptLibrary_OrientedBoxFunctions::FindClosestPointOnOrientedBox(FOrientedBox Box, FVector Point, bool& bIsInside)
{
	UE::Geometry::FOrientedBox3d OBox(Box);
	bIsInside = OBox.Contains(Point);
	return OBox.ClosestPoint(Point);
}

bool UGeometryScriptLibrary_OrientedBoxFunctions::TestOrientedBoxOrientedBoxIntersection(FOrientedBox Box1, FOrientedBox Box2)
{
	UE::Geometry::FOrientedBox3d OBox1(Box1), OBox2(Box2);
	return OBox1.Intersects(OBox2);
}

bool UGeometryScriptLibrary_OrientedBoxFunctions::TestBoxOrientedBoxIntersection(FBox Box1, FOrientedBox OrientedBox2)
{
	UE::Geometry::FOrientedBox3d OBox2(OrientedBox2);
	UE::Geometry::FAxisAlignedBox3d ABox1(Box1);
	return OBox2.Intersects(ABox1);
}


#undef LOCTEXT_NAMESPACE
