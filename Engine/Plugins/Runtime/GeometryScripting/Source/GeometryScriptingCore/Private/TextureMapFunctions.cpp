// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/TextureMapFunctions.h"

#include "Async/ParallelFor.h"
#include "AssetUtils/Texture2DUtil.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "Spatial/SampledScalarField2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureMapFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_TextureMapFunctions"

namespace UE::Geometry
{
namespace
{

void SampleImageBuilderAtUVPositions(
	const TImageBuilder<FVector4f>& ImageData,
	const FGeometryScriptUVList& UVList,
	const FGeometryScriptSampleTextureOptions& SampleOptions,
	FGeometryScriptColorList& ColorList)
{
	ColorList.Reset();
	if (UVList.List.IsValid() == false || UVList.List->Num() == 0)
	{
		return;
	}

	const TArray<FVector2D>& UVs = *UVList.List;
	int32 NumUVs = UVs.Num();

	TArray<FLinearColor>& Colors = *ColorList.List;
	Colors.SetNumUninitialized(NumUVs);

	for (int32 k = 0; k < NumUVs; ++k)
	{
		FVector2d UV = UVs[k];

		// Adjust UV value. 
		UV = UV * SampleOptions.UVScale + SampleOptions.UVOffset;

		FVector4f InterpValue;
		if (SampleOptions.bWrap)
		{
			constexpr EImageTilingMethod TilingMethod = EImageTilingMethod::Wrap;
			InterpValue = (SampleOptions.SamplingMethod == EGeometryScriptPixelSamplingMethod::Bilinear) ?
				ImageData.BilinearSampleUV<double, TilingMethod>(UV, FVector4f::Zero()) : ImageData.NearestSampleUV<TilingMethod>(UV);
		}
		else
		{
			constexpr EImageTilingMethod TilingMethod = EImageTilingMethod::Clamp;
			InterpValue = (SampleOptions.SamplingMethod == EGeometryScriptPixelSamplingMethod::Bilinear) ?
				ImageData.BilinearSampleUV<double, TilingMethod>(UV, FVector4f::Zero()) : ImageData.NearestSampleUV<TilingMethod>(UV);
		}

		Colors[k] = (FLinearColor)InterpValue;
	}
}

} // end anonymous namespace
} // end UE::Geometry namespace

void UGeometryScriptLibrary_TextureMapFunctions::SampleTexture2DAtUVPositions(
	FGeometryScriptUVList UVList,
	UTexture2D* TextureAsset,
	FGeometryScriptSampleTextureOptions SampleOptions,
	FGeometryScriptColorList& ColorList,
	UGeometryScriptDebug* Debug)
{
	if (TextureAsset == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SampleTexture2DAtUVPositions_InvalidInput2", "SampleTexture2DAtUVPositions: Texture is Null"));
		return;
	}

	TImageBuilder<FVector4f> ImageData;
	if (UE::AssetUtils::ReadTexture(TextureAsset, ImageData, false) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("SampleTexture2DAtUVPositions_TexReadFailed", "SampleTexture2DAtUVPositions: Error reading source texture data. If using this function at Runtime, The Compression Settings type on the UTexture2D Asset must be set to VectorDisplacementmap (RGBA8)."));
		return;
	}

	SampleImageBuilderAtUVPositions(ImageData, UVList, SampleOptions, ColorList);
}


void UGeometryScriptLibrary_TextureMapFunctions::SampleTextureRenderTarget2DAtUVPositions(  
	FGeometryScriptUVList UVList, 
	UTextureRenderTarget2D* Texture,
	FGeometryScriptSampleTextureOptions SampleOptions,
	FGeometryScriptColorList& ColorList,
	UGeometryScriptDebug* Debug)
{
	if (Texture == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("SampleTextureRenderTarget2DAtUVPositions_InvalidInput", "SampleTextureRenderTarget2DAtUVPositions: Texture is Null"));
		return;
	}

	FTextureRenderTargetResource* RenderTargetResource = Texture->GameThread_GetRenderTargetResource();
	if (RenderTargetResource == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("SampleTextureRenderTarget2DAtUVPositions_NullResource", "SampleTextureRenderTarget2DAtUVPositions: Render target resource is Null"));
		return;
	}

	FReadSurfaceDataFlags ReadSurfaceDataFlags(RCM_MinMax);
	ReadSurfaceDataFlags.SetLinearToGamma(false);
	
	TImageBuilder<FVector4f> ImageData;
	ImageData.SetDimensions(FImageDimensions(Texture->SizeX, Texture->SizeY));
	FLinearColor* Buffer = reinterpret_cast<FLinearColor*>(ImageData.GetImageBuffer().GetData());
	if (RenderTargetResource->ReadLinearColorPixelsPtr(Buffer, ReadSurfaceDataFlags, FIntRect(0, 0, Texture->SizeX, Texture->SizeY)) == false)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("SampleTextureRenderTarget2DAtUVPositions_GpuReadFailed", "SampleTextureRenderTarget2DAtUVPositions: Error reading texture data from the GPU."));
		return;
	}

	SampleImageBuilderAtUVPositions(ImageData, UVList, SampleOptions, ColorList);
}


#undef LOCTEXT_NAMESPACE
