// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/VectorMathFunctions.h"
#include "VectorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VectorMathFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_VectorMathFunctions"


namespace VectorMathFunctionsLocal
{

enum class EListOpResultCode
{
	Success = 0,
	DifferentSizes = 1,
};



template<typename ListType1, typename ListTypeOut, typename Func>
EListOpResultCode UnaryListToListOp(const ListType1& List1, ListTypeOut& OutputList, Func OperationFunc)
{
	if (List1.List.IsValid() == false)
	{
		return EListOpResultCode::DifferentSizes;
	}

	const auto& Array1 = *List1.List;
	int32 N = Array1.Num();

	// warning: OutputList may be List1 !!
	if (OutputList.List.IsValid() == false)
	{
		OutputList.Reset();
	}
	auto& OutputArray = *OutputList.List;
	if ( OutputArray.Num() != N )
	{
		OutputArray.SetNumUninitialized(N);
	}

	for (int32 k = 0; k < N; ++k)
	{
		OutputArray[k] = OperationFunc(Array1[k]);
	}

	return EListOpResultCode::Success;
}



template<typename ListType1, typename ListType2, typename ListType3, typename Func>
EListOpResultCode BinaryListToListOp(const ListType1& List1, const ListType2& List2, ListType3& OutputList, Func OperationFunc)
{
	if (List1.List.IsValid() == false || List2.List.IsValid() == false || List1.List->Num() != List2.List->Num() )
	{
		return EListOpResultCode::DifferentSizes;
	}

	const auto& Array1 = *List1.List;
	const auto& Array2 = *List2.List;
	int32 N = Array1.Num();

	// warning: OutputList may be List1 or List2 !!
	if (OutputList.List.IsValid() == false)
	{
		OutputList.Reset();
	}
	auto& OutputArray = *OutputList.List;
	if ( OutputArray.Num() != N )
	{
		OutputArray.SetNumUninitialized(N);
	}
	
	// ParallelFor ?
	for (int32 k = 0; k < N; ++k)
	{
		OutputArray[k] = OperationFunc(Array1[k], Array2[k]);
	}

	return EListOpResultCode::Success;
}



}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::VectorLength(FGeometryScriptVectorList VectorList)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		VectorList, ResultList, 
		[&](const FVector& Vector) { return Vector.Length(); } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorLength: VectorList is empty");
	}
	return ResultList;
}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::VectorDot(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		VectorListA, VectorListB, ResultList, 
		[&](const FVector& VectorA, const FVector& VectorB) { return FVector::DotProduct(VectorA, VectorB); } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorDot: Vector lists have different sizes");
	}
	return ResultList;
}


FGeometryScriptVectorList UGeometryScriptLibrary_VectorMathFunctions::VectorCross(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB)
{
	FGeometryScriptVectorList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		VectorListA, VectorListB, ResultList, 
		[&](const FVector& VectorA, const FVector& VectorB) { return FVector::CrossProduct(VectorA, VectorB); } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorCross: Vector lists have different sizes");
	}
	return ResultList;
}



void UGeometryScriptLibrary_VectorMathFunctions::VectorNormalizeInPlace(FGeometryScriptVectorList& VectorList, FVector SetOnFailure)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp( 
		VectorList, VectorList, [&](const FVector& Vec) { 
		return Vec.GetSafeNormal(UE_SMALL_NUMBER, SetOnFailure);
	} );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorNormalizeInPlace: VectorList is empty");
	}
}

void UGeometryScriptLibrary_VectorMathFunctions::VectorTransformInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FTransform Transform, bool bAsPosition)
{
	if (!VectorList.List.IsValid())
	{
		UE_LOGF(LogGeometry, Warning, "VectorTransformInPlace: VectorList is empty");
		return;
	}
	if (bAsPosition)
	{
		for (FVector& V : *VectorList.List)
		{
			V = Transform.TransformPosition(V);
		}
	}
	else
	{
		for (FVector& V : *VectorList.List)
		{
			V = Transform.TransformVector(V);
		}
	}

}

void UGeometryScriptLibrary_VectorMathFunctions::VectorInverseTransformInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FTransform Transform, bool bAsPosition)
{
	if (!VectorList.List.IsValid())
	{
		UE_LOGF(LogGeometry, Warning, "VectorInverseTransformInPlace: VectorList is empty");
		return;
	}
	if (bAsPosition)
	{
		for (FVector& V : *VectorList.List)
		{
			V = Transform.InverseTransformPosition(V);
		}
	}
	else
	{
		for (FVector& V : *VectorList.List)
		{
			V = Transform.InverseTransformVector(V);
		}
	}
}

void UGeometryScriptLibrary_VectorMathFunctions::VectorPlaneProjectInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FPlane Plane)
{
	if (!VectorList.List.IsValid())
	{
		UE_LOGF(LogGeometry, Warning, "VectorPlaneProjectInPlace: VectorList is empty");
		return;
	}
	for (FVector& V : *VectorList.List)
	{
		V = FVector::PointPlaneProject(V, Plane);
	}
}



FGeometryScriptVectorList UGeometryScriptLibrary_VectorMathFunctions::VectorBlend(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB, double ConstantA, double ConstantB)
{
	FGeometryScriptVectorList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		VectorListA, VectorListB, ResultList, 
		[&](const FVector& VectorA, const FVector& VectorB) { return ConstantA*VectorA + ConstantB*VectorB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorBlend: Vector lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::VectorBlendInPlace(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList& VectorListB, double ConstantA, double ConstantB)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		VectorListA, VectorListB, VectorListB, 
		[&](const FVector& VectorA, const FVector& VectorB) { return ConstantA*VectorA + ConstantB*VectorB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorBlendInPlace: Vector lists have different sizes");
	}
}



FGeometryScriptVectorList UGeometryScriptLibrary_VectorMathFunctions::ScalarVectorMultiply(FGeometryScriptScalarList ScalarList, FGeometryScriptVectorList VectorList, double ScalarMultiplier)
{
	FGeometryScriptVectorList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarList, VectorList, ResultList, 
		[&](double Scalar, const FVector& Vector) { return (FVector::FReal)(Scalar * ScalarMultiplier) * Vector; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarVectorMultiply: Scalar and Vector lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ScalarVectorMultiplyInPlace(FGeometryScriptScalarList ScalarList, FGeometryScriptVectorList& VectorList, double ScalarMultiplier)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarList, VectorList, VectorList, 
		[&](double Scalar, const FVector& Vector) { return (FVector::FReal)(Scalar * ScalarMultiplier) * Vector; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarVectorMultiplyInPlace: Scalar and Vector lists have different sizes");
	}
}


FGeometryScriptVectorList UGeometryScriptLibrary_VectorMathFunctions::ConstantVectorMultiply(double Constant, FGeometryScriptVectorList VectorList)
{
	FGeometryScriptVectorList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		VectorList, ResultList, 
		[&](const FVector& Vector) { return (FVector::FReal)(Constant) * Vector; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ConstantVectorMultiply: VectorList is empty");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ConstantVectorMultiplyInPlace(double Constant, FGeometryScriptVectorList& VectorList)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		VectorList, VectorList, 
		[&](const FVector& Vector) { return (FVector::FReal)(Constant) * Vector; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ConstantVectorMultiplyInPlace: VectorList is empty");
	}
}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::VectorToScalar(FGeometryScriptVectorList VectorList, double ConstantX, double ConstantY, double ConstantZ)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		VectorList, ResultList, 
		[&](const FVector& Vector) { return ConstantX*Vector.X + ConstantY*Vector.Y + ConstantZ*Vector.Z; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "VectorToScalar: VectorList is empty");
	}
	return ResultList;
}




FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::ScalarInvert(FGeometryScriptScalarList ScalarList, double Numerator, double SetOnFailure, double Epsilon)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		ScalarList, ResultList, 
		[&](const double& Scalar) { return (FMathd::Abs(Scalar) > Epsilon) ? (Numerator / Scalar) : SetOnFailure; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarInvert: Scalar lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ScalarInvertInPlace(FGeometryScriptScalarList& ScalarList, double Numerator, double SetOnFailure, double Epsilon)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		ScalarList, ScalarList, 
		[&](const double& Scalar) { return (FMathd::Abs(Scalar) > Epsilon) ? (Numerator / Scalar) : SetOnFailure; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarInvertInPlace: ScalarList is empty");
	}
}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::ScalarBlend(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList ScalarListB, double ConstantA, double ConstantB)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarListA, ScalarListB, ResultList, 
		[&](const double& ScalarA, const double& ScalarB) { return ConstantA*ScalarA + ConstantB*ScalarB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarBlend: Scalar lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ScalarBlendInPlace(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList& ScalarListB, double ConstantA, double ConstantB)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarListA, ScalarListB, ScalarListB, 
		[&](const double& ScalarA, const double& ScalarB) { return ConstantA*ScalarA + ConstantB*ScalarB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarBlendInPlace: Scalar lists have different sizes");
	}
}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::ScalarMultiply(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList ScalarListB, double ScalarMultiplier)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarListA, ScalarListB, ResultList, 
		[&](const double& ScalarA, const double& ScalarB) { return ScalarMultiplier*ScalarA*ScalarB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarMultiply: Scalar lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ScalarMultiplyInPlace(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList& ScalarListB, double ScalarMultiplier)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::BinaryListToListOp(
		ScalarListA, ScalarListB, ScalarListB, 
		[&](const double& ScalarA, const double& ScalarB) { return ScalarMultiplier*ScalarA*ScalarB; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ScalarMultiplyInPlace: Scalar lists have different sizes");
	}
}


FGeometryScriptScalarList UGeometryScriptLibrary_VectorMathFunctions::ConstantScalarMultiply(double Constant, FGeometryScriptScalarList ScalarList)
{
	FGeometryScriptScalarList ResultList;
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		ScalarList, ResultList, 
		[&](const double& Scalar) { return Constant*Scalar; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ConstantScalarMultiply: Scalar lists have different sizes");
	}
	return ResultList;
}

void UGeometryScriptLibrary_VectorMathFunctions::ConstantScalarMultiplyInPlace(double Constant, FGeometryScriptScalarList& ScalarList)
{
	VectorMathFunctionsLocal::EListOpResultCode Result = VectorMathFunctionsLocal::UnaryListToListOp(
		ScalarList, ScalarList, 
		[&](const double& Scalar) { return Constant*Scalar; } );
	if ( Result == VectorMathFunctionsLocal::EListOpResultCode::DifferentSizes )
	{
		UE_LOGF(LogGeometry, Warning, "ConstantScalarMultiplyInPlace: ScalarList is empty");
	}
}


#undef LOCTEXT_NAMESPACE

