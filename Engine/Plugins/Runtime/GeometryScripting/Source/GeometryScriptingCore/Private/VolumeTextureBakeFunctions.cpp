// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/VolumeTextureBakeFunctions.h"
#include "UDynamicMesh.h"

#include "DynamicMesh/MeshTransforms.h"
#include "Implicit/SweepingMeshSDF.h"
#include "Spatial/MeshAABBTree3.h"
#include "Engine/VolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeTextureBakeFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_VolumeTextureBakeFunctions"

namespace UE::VolumeTextureBakeLocal
{
	// Format to use for volume texture data; may be exposed to caller in future
	// Note this is only how source data will be stored, and not the final format used to represent the texture in cook/on gpu
	enum class EVolumeTextureFormat : uint8
	{
		// Integers from 0 to 257
		Unorm8 = 0,
		// 16-bit float
		Float16,
		// 32-bit float (aka just float)
		Float32
	};

	static bool InitializeWithSDF(const FDynamicMesh3& Mesh, UVolumeTexture* VolumeTexture, 
		const FComputeDistanceFieldSettings& DistanceSettings, const FDistanceFieldToTextureSettings& TextureSettings,
		EVolumeTextureFormat Format)
	{
		check(VolumeTexture);

		FAxisAlignedBox3d MeshBounds = Mesh.GetBounds();

		typedef TSweepingMeshSDF<FDynamicMesh3, false> SDFType;
		SDFType SDF;
		const int32 MaxVoxelCount = SDF.ApproxMaxCellsPerDimension;
		constexpr int32 MinVoxelCount = 2;

		FVector3i Dimensions = (FVector3i)DistanceSettings.VoxelsPerDimensions;
		for (int32 Dim = 0; Dim < 3; ++Dim)
		{
			Dimensions[Dim] = FMath::Clamp(Dimensions[Dim], MinVoxelCount, MaxVoxelCount);
			if (DistanceSettings.bRequirePower2)
			{
				Dimensions[Dim] = FMath::RoundUpToPowerOfTwo(Dimensions[Dim]);
			}
		}
		FVector3f CellSize = (FVector3f)(MeshBounds.Diagonal() / ((FVector3d)Dimensions - FVector3d::One()));
		
		SDF.Mesh = &Mesh;
		SDF.CellSize = CellSize;
		SDF.NarrowBandMaxDistance = DistanceSettings.NarrowBandWidth * (DistanceSettings.NarrowBandUnits == EDistanceFieldUnits::NumberOfVoxels ? CellSize.GetMin() : 1);
		SDF.ExactBandWidth = FMath::CeilToInt32(SDF.NarrowBandMaxDistance / CellSize.GetMin());
		SDF.ExpandBounds = FVector3d::Zero(); // ExpandBounds not used since we exactly specify the grid origin and dimensions

		// for meshes with long triangles relative to the width of the narrow band, don't use the AABB tree
		// TODO: technically the NarrowBandOnly and FullGrid method can use the AABB tree if it's available; should benchmark to figure it if/when it's worth computing
		SDF.Spatial = nullptr;
		double AvgEdgeLen = TMeshQueries<FDynamicMesh3>::AverageEdgeLength(Mesh);
		TMeshAABBTree3<FDynamicMesh3> Spatial;
		if (DistanceSettings.ComputeMode == EDistanceFieldComputeMode::NarrowBand)
		{
			if (SDF.ShouldUseSpatial(SDF.ExactBandWidth, CellSize.GetMin(), AvgEdgeLen))
			{
				SDF.ComputeMode = SDFType::EComputeModes::NarrowBand_SpatialFloodFill;
				Spatial.SetMesh(&Mesh, true);
				SDF.Spatial = &Spatial;
			}
			else
			{
				SDF.ComputeMode = SDFType::EComputeModes::NarrowBandOnly;
			}
		}
		else // DistanceSettings.ComputeMode == EDistanceFieldComputeMode::FullGrid
		{
			SDF.ComputeMode = SDFType::EComputeModes::FullGrid;
		}

		if (!SDF.Compute((FVector3f)MeshBounds.Min, Dimensions))
		{
			return false;
		}

		// Voxel conversion for supported formats
		auto QueryVoxel8 = [&SDF, TextureSettings](int32 PosX, int32 PosY, int32 PosZ, void* Value)
		{
			float SD = (float)SDF.Grid.GetValue(PosX, PosY, PosZ);
			uint8* const Voxel = static_cast<uint8*>(Value);
			Voxel[0] = (uint8)FMath::Clamp(TextureSettings.Offset + SD * TextureSettings.Scale, 0, 255);
		};
		auto QueryVoxel16 = [&SDF, TextureSettings](int32 PosX, int32 PosY, int32 PosZ, void* Value)
		{
			float SD = (float)SDF.Grid.GetValue(PosX, PosY, PosZ);
			FFloat16* const Voxel = static_cast<FFloat16*>(Value);
			Voxel[0] = TextureSettings.Offset + SD * TextureSettings.Scale;
		};
		auto QueryVoxel32 = [&SDF, TextureSettings](int32 PosX, int32 PosY, int32 PosZ, void* Value)
		{
			float SD = (float)SDF.Grid.GetValue(PosX, PosY, PosZ);
			float* const Voxel = static_cast<float*>(Value);
			Voxel[0] = TextureSettings.Offset + SD * TextureSettings.Scale;
		};

		// fill volume texture from level set
		FVector3i Dims = SDF.Grid.GetDimensions();
		switch (Format)
		{
		case EVolumeTextureFormat::Unorm8:
			return VolumeTexture->UpdateSourceFromFunction(QueryVoxel8, Dims.X, Dims.Y, Dims.Z, TSF_G8);
		case EVolumeTextureFormat::Float16:
			return VolumeTexture->UpdateSourceFromFunction(QueryVoxel16, Dims.X, Dims.Y, Dims.Z, TSF_R16F);
		case EVolumeTextureFormat::Float32:
			return VolumeTexture->UpdateSourceFromFunction(QueryVoxel32, Dims.X, Dims.Y, Dims.Z, TSF_R32F);
		default:
			UE_LOGF(LogGeometry, Error, "Unknown volume texture format");
			return false;
		}
	}
}

bool UGeometryScriptLibrary_VolumeTextureBakeFunctions::BakeSignedDistanceToVolumeTexture(
	const UDynamicMesh* TargetMesh,
	UVolumeTexture* VolumeTexture,
	FComputeDistanceFieldSettings DistanceSettings, FDistanceFieldToTextureSettings TextureSettings
)
{
	if (TargetMesh == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "BakeSignedDistanceToVolumeTexture: Target Mesh was null");
		return false;
	}

	if (VolumeTexture == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "BakeSignedDistanceToVolumeTexture: Volume Texture was null");
		return false;
	}

#if WITH_EDITOR
	bool bSuccess = false;
	TargetMesh->ProcessMesh([VolumeTexture, &DistanceSettings, &TextureSettings, &bSuccess](const FDynamicMesh3& Mesh)
	{
		using namespace UE::VolumeTextureBakeLocal;
		bSuccess = InitializeWithSDF(Mesh, VolumeTexture, DistanceSettings, TextureSettings, EVolumeTextureFormat::Float32);
	});
	return bSuccess;
#else
	UE_LOGF(LogGeometry, Warning, "BakeSignedDistanceToVolumeTexture: Baking signed distance to volume texture is only supported in editor");
	return false;
#endif
}

#undef LOCTEXT_NAMESPACE

