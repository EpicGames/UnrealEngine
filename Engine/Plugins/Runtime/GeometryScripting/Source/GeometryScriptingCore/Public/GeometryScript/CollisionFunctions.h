// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "CollisionFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;
class UDynamicMeshComponent;
class UStaticMesh;

UENUM(BlueprintType)
enum class EGeometryScriptCollisionGenerationMethod : uint8
{
	AlignedBoxes = 0,
	OrientedBoxes = 1,
	MinimalSpheres = 2,
	Capsules = 3,
	ConvexHulls = 4,
	SweptHulls = 5,
	MinVolumeShapes = 6,
	LevelSets = 7
};


UENUM(BlueprintType)
enum class EGeometryScriptSweptHullAxis : uint8
{
	X = 0,
	Y = 1,
	Z = 2,
	/** Use X/Y/Z axis with smallest axis-aligned-bounding-box dimension */
	SmallestBoxDimension = 3,
	/** Compute projected hull for each of X/Y/Z axes and use the one that has the smallest volume  */
	SmallestVolume = 4
};


USTRUCT(BlueprintType)
struct FGeometryScriptCollisionFromMeshOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmitTransaction = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptCollisionGenerationMethod Method = EGeometryScriptCollisionGenerationMethod::MinVolumeShapes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoDetectSpheres = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoDetectBoxes = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoDetectCapsules = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MinThickness = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSimplifyHulls = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int ConvexHullTargetFaceCount = 25;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ConvexHullGeometricTolerance = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxConvexHullsPerMesh = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ConvexDecompositionSearchFactor = .5f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ConvexDecompositionErrorTolerance = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ConvexDecompositionMinPartThickness = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SweptHullSimplifyTolerance = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptSweptHullAxis SweptHullAxis = EGeometryScriptSweptHullAxis::Z;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRemoveFullyContainedShapes = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxShapeCount = 0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptSetSimpleCollisionOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmitTransaction = true;
};


USTRUCT(BlueprintType)
struct FGeometryScriptSetStaticMeshCollisionOptions
{
	GENERATED_BODY()
public:
	// Whether to mark the static mesh collision as customized when it is set, so that it will not be overwritten on next import.
	// If false, Static Mesh collision will not be un-marked as Customized; its state will just be left unchanged.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bMarkAsCustomized = true;
};


// Method to distribute sampling spheres, used by FComputeNegativeSpaceOptions
UENUM(BlueprintType)
enum class ENegativeSpaceSampleMethod : uint8
{
	// Place sample spheres in a uniform grid pattern
	Uniform,
	// Use voxel-based subtraction and offsetting methods to specifically target concavities
	VoxelSearch,
	// A more-principled version of VoxelSearch that attempts to target specifically the space that is reachable by characters at least as large as a MinRadius sphere
	NavigableVoxelSearch
};

// Defines direction-specific custom scaling to apply to the FComputeNegativeSpaceOptions::NegativeSpaceTolerance value, based on the direction to the surface
// For example, this can be used to more closely fit collision shapes on ground surfaces
USTRUCT(BlueprintType)
struct FNegativeSpaceDirectionalToleranceScale
{
	GENERATED_BODY()

	// Direction to apply custom tolerance scale
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Direction = FVector(0,0,-1);

	// Angle range around direction within which the scale will be applied, in degrees
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = 0, ClampMax = 180, Units = deg))
	double AngleRange = 45.0;

	// Scale factor to apply to tolerance. Should be in the (0, 1] range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = .1, ClampMax = 1, UIMin = .25))
	double ScaleFactor = 1.0;
};

// Options controlling how to sample the negative space of shapes, e.g. to define a region that must be avoided when merging collision shapes
USTRUCT(BlueprintType)
struct FComputeNegativeSpaceOptions
{
	GENERATED_BODY()

	/** Method to use to find and sample negative space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace)
	ENegativeSpaceSampleMethod SampleMethod = ENegativeSpaceSampleMethod::Uniform;

	/** Whether to require that all candidate locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (EditCondition = "SampleMethod == EGeometryScriptNegativeSpaceSampleMethod::VoxelSearch", EditConditionHides))
	bool bRequireSearchSampleCoverage = false;

	/** When performing Voxel Search, only look for negative space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (EditCondition = "SampleMethod == EGeometryScriptNegativeSpaceSampleMethod::VoxelSearch", EditConditionHides))
	bool bOnlyConnectedToHull = false;

	/** When performing Voxel Search, maximum number of voxels to use along each dimension */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (ClampMin = 4, ClampMax = 1290))
	int32 MaxVoxelsPerDim = 128;

	/** Approximate number of spheres to consider when covering negative space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (ClampMin = 1))
	int32 TargetNumSamples = 50;

	/** Minimum desired spacing between sphere centers; if > 0, will attempt not to place sphere centers closer than this */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (ClampMin = 0, Units = cm))
	double MinSampleSpacing = 1.0;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (ClampMin = .01, UIMin = .1, Units = cm))
	double NegativeSpaceTolerance = 2.0;

	/** Optionally scale the negative space tolerances in custom directions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, AdvancedDisplay)
	TArray<FNegativeSpaceDirectionalToleranceScale> ToleranceDirectionalScales;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, meta = (ClampMin = 0, Units = cm))
	double MinRadius = 10.0;
};


/**
 * Settings to define the important regions for a convex decomposition to preserve for a given input shape.
 * 
 * Note: this is similar to FComputeNegativeSpaceOptions, but with better default behavior and more intuitive parameters.
 */
USTRUCT(BlueprintType)
struct FNavigableConvexDecompositionOptions
{
	GENERATED_BODY()
public:
	/// Minimum radius of characters/manipulators that should be able to navigate an input shape
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double MinRadius = 100;

	/// Tolerance distance: convex decomposition should be no further than this from an input shape, in the navigable regions
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double Tolerance = 10;

	/// Whether to only consider navigable space that is accessible from outside the shape. (Note this parameter is called bOnlyConnectedToHull elsewhere.)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bIgnoreUnreachableInternalSpace = true;

	/// Optional list of locations that we expect to be navigable
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> CustomNavigablePositions;

	/// Optional list of planes defining unreachable space (on their negative side)
	/// Use this for example to specify a ground plane, if a mesh will always be placed on ground and need not be navigable from below.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FPlane> UnreachablePlanes;

	/** Optionally scale the tolerance in custom directions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace, AdvancedDisplay)
	TArray<FNegativeSpaceDirectionalToleranceScale> ToleranceDirectionalScales;
};


// Options controlling how collision shapes can be merged together
USTRUCT(BlueprintType)
struct FGeometryScriptMergeSimpleCollisionOptions
{
	GENERATED_BODY()
public:

	/**
	 * If > 0, merge down to at most this many simple shapes. (If <= 0, this value is ignored.)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxShapeCount = 0;

	/**
	 * Error tolerance to use to decide to convex hulls together, in cm.
	 * If merging two hulls would increase the volume by more than this ErrorTolerance cubed, the merge is not accepted.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = "0", UIMax = "100.", Units = cm))
	double ErrorTolerance = 0.0;

	/**
	 * Always attempt to merge parts thicker than this, ignoring ErrorTolerance and MaxShapeCount.
	 * Note: Negative space, if set, will still prevent merges.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = "0", UIMax = "100.", Units = cm))
	double MinThicknessTolerance = 0.0;

	/** Whether to consider merges between every shape. If false, will only merge shapes that have overlapping or nearby bounding boxes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bConsiderAllPossibleMerges = false;

	// Negative space that must be preserved during merging
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace)
	FGeometryScriptSphereCovering PrecomputedNegativeSpace;

	// Whether to compute a new sphere covering representing the negative space of the input shapes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace)
	bool bComputeNegativeSpace = false;

	// Options controlling how the negative space is computed, if ComputeNegativeSpace is true
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NegativeSpace)
	FComputeNegativeSpaceOptions ComputeNegativeSpaceOptions;

	// Controls for how smooth shapes can be triangulated when/if converted to a convex hull for a merge
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConvexHulls)
	FGeometryScriptSimpleCollisionTriangulationOptions ShapeToHullTriangulation;
};

// Methods to simplify convex hulls, used by FGeometryScriptConvexHullSimplificationOptions
UENUM(BlueprintType)
enum class EGeometryScriptConvexHullSimplifyMethod : uint8
{
	// Simplify convex hulls using a general mesh-based simplifier, and taking the convex hull of the simplified mesh
	MeshQSlim,
	// Simplify convex hulls by merging hull faces that have similar normals
	AngleTolerance
};

USTRUCT(BlueprintType)
struct FGeometryScriptConvexHullSimplificationOptions
{
	GENERATED_BODY()
public:

	/** Method to use to simplify convex hulls */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptConvexHullSimplifyMethod SimplificationMethod = EGeometryScriptConvexHullSimplifyMethod::MeshQSlim;

	/** Simplified hull should stay within this distance of the initial convex hull. Used by the MeshQSlim simplification method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SimplificationDistanceThreshold = 10.f;

	/** Simplified hull should preserve angles larger than this (in degrees). Used by the AngleTolerance simplification method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SimplificationAngleThreshold = 10.f;

	/** 
	 * The minimum number of faces to use for the convex hull.
	 * Note that for the MeshQSlim method all faces are triangles, while the AngleTolerance method can consider more general polygons. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = 4))
	int32 MinTargetFaceCount = 12;
};

USTRUCT(BlueprintType)
struct FGeometryScriptConvexHullApproximationOptions
{
	GENERATED_BODY()
public:

	/** Whether to attempt to replace convex hulls with spheres */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFitSpheres = true;

	/** Whether to attempt to replace convex hulls with boxes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFitBoxes = true;

	/** Whether to attempt to replace convex hulls with capsules */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFitCapsules = false;

	/** Approximating shape should be at least this close to the original shape */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float DistanceThreshold = 5.f;

	/** Acceptable difference between approximating shape volume and convex hull volume, as a fraction of convex hull volume */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float VolumeDiffThreshold_Fraction = .15f;
};

USTRUCT(BlueprintType)
struct FGeometryScriptTransformCollisionOptions
{
	GENERATED_BODY()
public:

	/** Whether to log a warning when a requested transform is not compatible with the simple collision shapes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWarnOnInvalidTransforms = true;

	/**
	 * If true, we apply the Transform to each collision shape separately, and pivot the Transform around the local center of each shape.
	 * Otherwise, we apply the Transform to all shapes in the same space, with the pivot at the origin of the origin of that space.
	 * 
	 * For example, if we apply a uniform 2x scale to a sphere w/ center (1,1,1), with this enabled, the center will not move and only the radius will scale.
	 * If this setting is not enabled, the 2x scale will move the sphere center to (2,2,2)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCenterTransformPivotPerShape = false;
};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Collision"))
class UGeometryScriptLibrary_CollisionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Set a static mesh as the custom collision for another static mesh to use.
	 * Note: Only works if editor-only data is available.
	 * 
	 * @return true on success.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API bool SetStaticMeshCustomComplexCollision(
		UStaticMesh* StaticMeshAsset,
		UStaticMesh* StaticMeshCollisionAsset,
		bool bEmitTransaction,
		bool bMarkCollisionAsCustomized = true,
		UGeometryScriptDebug* Debug = nullptr
	);

	/** 
	* Generates Simple Collision shapes for a Static Mesh Asset based on the input Dynamic Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	SetStaticMeshCollisionFromMesh(
		UDynamicMesh* FromDynamicMesh, 
		UStaticMesh* ToStaticMeshAsset, 
		FGeometryScriptCollisionFromMeshOptions Options,
		FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions = FGeometryScriptSetStaticMeshCollisionOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Copy the Simple Collision Geometry from the Source Component to the Static Mesh Asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API void 
	SetStaticMeshCollisionFromComponent(
		UStaticMesh* StaticMeshAsset, 
		UPrimitiveComponent* SourceComponent,
		FGeometryScriptSetSimpleCollisionOptions Options = FGeometryScriptSetSimpleCollisionOptions(),
		FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions = FGeometryScriptSetStaticMeshCollisionOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/*
	 * @returns true if the static mesh has customized collision. If no editor data is available, returns false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision")
	static UE_API UPARAM(DisplayName = "IsCustomized") bool StaticMeshHasCustomizedCollision(UStaticMesh* StaticMeshAsset);

	/** 
	* Generate Simple Collision shapes for a Dynamic Mesh Component based on the input Dynamic Mesh. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	SetDynamicMeshCollisionFromMesh(
		UDynamicMesh* FromDynamicMesh, 
		UDynamicMeshComponent* ToDynamicMeshComponent,
		FGeometryScriptCollisionFromMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

    /** 
	* Clears Simple Collisions from the Dynamic Mesh Component. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API void
	ResetDynamicMeshCollision(
		UDynamicMeshComponent* Component,
		bool bEmitTransaction = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Clears the Simple Collision shapes
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision")
	static UE_API void ResetSimpleCollision(UPARAM(ref) FGeometryScriptSimpleCollision& SimpleCollision);

	/*
	 * Get the simple collision from a Primitive Component
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Simple Collision") FGeometryScriptSimpleCollision
	GetSimpleCollisionFromComponent(
		UPrimitiveComponent* Component,
		UGeometryScriptDebug* Debug = nullptr);

	/*
	 * Set the simple collision on a Dynamic Mesh Component
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API void
	SetSimpleCollisionOfDynamicMeshComponent(
		const FGeometryScriptSimpleCollision& SimpleCollision,
		UDynamicMeshComponent* DynamicMeshComponent,
		FGeometryScriptSetSimpleCollisionOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/*
	 * Get the simple collision from a Static Mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Simple Collision") FGeometryScriptSimpleCollision
	GetSimpleCollisionFromStaticMesh(UStaticMesh* StaticMesh, UGeometryScriptDebug* Debug = nullptr);

	/*
	 * Set the simple collision on a Static Mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API void
	SetSimpleCollisionOfStaticMesh(
		const FGeometryScriptSimpleCollision& SimpleCollision,
		UStaticMesh* StaticMesh, 
		FGeometryScriptSetSimpleCollisionOptions Options, 
		FGeometryScriptSetStaticMeshCollisionOptions StaticMeshCollisionOptions = FGeometryScriptSetStaticMeshCollisionOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Generate Simple Collision shapes for an input Dynamic Mesh shape
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Simple Collision") FGeometryScriptSimpleCollision
	GenerateCollisionFromMesh(
		UDynamicMesh* FromDynamicMesh,
		FGeometryScriptCollisionFromMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/*
	 * Count of number of simple collision shapes
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (ScriptMethod))
	static int32 GetSimpleCollisionShapeCount(const FGeometryScriptSimpleCollision& SimpleCollision)
	{
		return SimpleCollision.AggGeom.GetElementCount();
	}

	/*
	 * Transform simple collision shapes
	 * @param bSuccess	Indicates whether all collision shapes were accurately transformed. On failure, shapes will still be copied over and a best-effort transform will still be applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug", AutoCreateRefTerm = "TransformOptions"))
	static UE_API UPARAM(DisplayName = "Transformed Collision") FGeometryScriptSimpleCollision TransformSimpleCollisionShapes(
		const FGeometryScriptSimpleCollision& SimpleCollision,
		FTransform Transform,
		const FGeometryScriptTransformCollisionOptions& TransformOptions,
		bool& bSuccess,
		UGeometryScriptDebug* Debug = nullptr
	);

	/*
	 * Add simple collision shapes from AppendCollision to CollisionToUpdate
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API void CombineSimpleCollision(
		UPARAM(ref) FGeometryScriptSimpleCollision& CollisionToUpdate,
		const FGeometryScriptSimpleCollision& AppendCollision,
		UGeometryScriptDebug* Debug = nullptr
	);

	/*
	 * Combine the SimpleCollisionArray collision shapes into a single SimpleCollision
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug"))
	static UE_API void CombineSimpleCollisionArray(
		const TArray<FGeometryScriptSimpleCollision>& SimpleCollisionArray,
		FGeometryScriptSimpleCollision& SimpleCollision,
		UGeometryScriptDebug* Debug = nullptr
	);
	
	/**
	 * Simplify any convex hulls in the given simple collision representation. Updates the passed-in Simple Collision.
	 *
	 * @param SimpleCollision		The collision in which to attempt to simplify the convex hulls
	 * @param ConvexSimplifyOptions	Options controlling how convex hulls are simplified
	 * @param bHasSimplified		Indicates whether any convex hulls were modified
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (ScriptMethod, HidePin = "Debug", AutoCreateRefTerm = "SimplifyOptions"))
	static UE_API void SimplifyConvexHulls(
		UPARAM(ref) FGeometryScriptSimpleCollision& SimpleCollision,
		const FGeometryScriptConvexHullSimplificationOptions& SimplifyOptions,
		bool& bHasSimplified,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Attempt to approximate any convex hulls in the given simple collision representation. Updates the passed-in Simple Collision.
	 * Convex hulls that aren't well approximated (to tolerances set in ApproximateOptions) will remain as convex hulls.
	 * 
	 * @param SimpleCollision		The collision in which to attempt to approximate the convex hulls
	 * @param ConvexSimplifyOptions	Options controlling how convex hulls are approximated
	 * @param bHasApproximated		Indicates whether any convex hulls were replaced with simpler approximations
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (ScriptMethod, HidePin = "Debug", AutoCreateRefTerm = "ApproximateOptions"))
	static UE_API void ApproximateConvexHullsWithSimplerCollisionShapes(
		UPARAM(ref) FGeometryScriptSimpleCollision& SimpleCollision,
		const FGeometryScriptConvexHullApproximationOptions& ApproximateOptions,
		bool& bHasApproximated,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Attempt to merge collision shapes to create a representation with fewer overall shapes.
	 * 
	 * @param SimpleCollision		The collision to attempt to simplify by merging shapes
	 * @param MergeOptions			Options controlling how shapes can be merged
	 * @param bHasMerged			Indicates whether any shapes have been merged
	 * @return						Simple Collision with collision shapes merged, as allowed by settings
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug", AutoCreateRefTerm = "MergeOptions"))
	static UE_API UPARAM(DisplayName = "Merged Simple Collision") FGeometryScriptSimpleCollision MergeSimpleCollisionShapes(
		const FGeometryScriptSimpleCollision& SimpleCollision,
		const FGeometryScriptMergeSimpleCollisionOptions& MergeOptions,
		bool& bHasMerged,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Compute the negative space of an input mesh surface that should be protected when merging simple collision shapes
	 * 
	 * @param MeshBVH				A Dynamic Mesh BVH structure of the surface for which we will compute negative space
	 * @param NegativeSpaceOptions	Options controlling how the negative space is generated
	 * @return						A set of spheres that cover the negative space of the input shape
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug", AutoCreateRefTerm = "NegativeSpaceOptions"))
	static UE_API UPARAM(DisplayName = "Negative Space") FGeometryScriptSphereCovering ComputeNegativeSpace(
		const FGeometryScriptDynamicMeshBVH& MeshBVH,
		const FComputeNegativeSpaceOptions& NegativeSpaceOptions,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Compute the 'navigable' convex decomposition of an input mesh surface, i.e. a convex decomposition 
	 * appropriate for a character of (or larger than) a given size
	 *
	 * @param TargetMesh	Mesh to decompose to convex hulls
	 * @param Options		Options controlling the convex decomposition
	 * @return				The resulting convex hulls as simple collision shapes
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Collision", meta = (HidePin = "Debug", AutoCreateRefTerm = "Options"))
	static UE_API UPARAM(DisplayName = "Convex Decomposition") FGeometryScriptSimpleCollision ComputeNavigableConvexDecomposition(
		const UDynamicMesh* TargetMesh,
		const FNavigableConvexDecompositionOptions& Options,
		UGeometryScriptDebug* Debug = nullptr);



	/**
	 * @return An array of the spheres in the given Sphere Covering
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "SphereCovering To Array Of Spheres", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|Collision")
	static UE_API TArray<FSphere> Conv_GeometryScriptSphereCoveringToSphereArray(const FGeometryScriptSphereCovering& SphereCovering);

	/**
	* @return A sphere covering containing the spheres in the given Spheres array
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Array Of Spheres To SphereCovering", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|Collision")
	static UE_API FGeometryScriptSphereCovering Conv_SphereArrayToGeometryScriptSphereCovering(const TArray<FSphere>& Spheres);


};

#undef UE_API
