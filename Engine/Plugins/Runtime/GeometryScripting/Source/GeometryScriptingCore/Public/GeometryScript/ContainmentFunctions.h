// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "OrientedBoxTypes.h"
#include "ContainmentFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

USTRUCT(BlueprintType)
struct FGeometryScriptConvexHullOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPrefilterVertices = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int PrefilterGridResolution = 128;

	/** Try to simplify each convex hull to this triangle count. If 0, no simplification */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SimplifyToFaceCount = 0;

	/** After SimplifyToFaceCount is applied, further simplify within this tolerance. If SimplifyToFaceCount <= 0, or this tolerance is <= 0, does not apply. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float GeometricToleranceAfterTarget = 0.f;
};



USTRUCT(BlueprintType)
struct FGeometryScriptSweptHullOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPrefilterVertices = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int PrefilterGridResolution = 128;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MinThickness = 0.01f;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSimplify = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MinEdgeLength = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SimplifyTolerance = 0.1f;
};



USTRUCT(BlueprintType)
struct FGeometryScriptConvexDecompositionOptions
{
	GENERATED_BODY()
public:
	/** How many convex pieces to target per mesh when creating convex decompositions.  If ErrorTolerance is set, can create fewer pieces. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DecompositionOptions)
	int NumHulls = 2;

	/** How much additional decomposition decomposition + merging to do, as a fraction of max pieces.  Larger values can help better-cover small features, while smaller values create a cleaner decomposition with less overlap between hulls. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DecompositionOptions)
	double SearchFactor = .5;

	/** Error tolerance to guide convex decomposition (in cm); we stop adding new parts if the volume error is below the threshold.  For volumetric errors, value will be cubed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DecompositionOptions)
	double ErrorTolerance = 0;

	/** Minimum part thickness for convex decomposition (in cm); hulls thinner than this will be merged into adjacent hulls, if possible. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DecompositionOptions)
	double MinPartThickness = .1;

	/** Try to simplify each convex hull to this triangle count. If 0, no simplification */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConvexHullOptions)
	int SimplifyToFaceCount = 0;
};

UENUM()
enum class FGeometryScriptFitOrientedBoxMethod : uint8
{
	// Use an iterative method to fit the oriented box, starting from a fast initial guess and locally refining
	FastIterative,
	// Use a precise method to fit the oriented box, testing a potentially large space of options for the best fit
	Precise
};

USTRUCT(BlueprintType)
struct FGeometryScriptFitOrientedBoxOptions
{
	GENERATED_BODY()
public:
	/* Minimum length of the resulting bounding box along any of its principal directions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(ClampMin=0))
	double MinBoxDimension = .01;

	/* Method to use for fitting the bounding box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptFitOrientedBoxMethod FitMethod = FGeometryScriptFitOrientedBoxMethod::FastIterative;

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Containment"))
class UGeometryScriptLibrary_ContainmentFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Compute the Convex Hull of a given Target Mesh, or part of the mesh if an optional Selection is provided, and put the result in Hull Mesh
	 * @param CopyToMesh	The Dynamic Mesh to store the convex hull geometry.
	 * @param CopyToMeshOut	The resulting convex hull.
	 * @param Selection	Selection of mesh faces/vertices to contain in the convex hull. If not provided, entire mesh is used.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Containment", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeMeshConvexHull(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
		UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptConvexHullOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Compute the Swept Hull of a given Target Mesh for a given 3D Plane defined by ProjectionFrame, and put the result in Hull Mesh
	* The Swept Hull is a linear sweep of the 2D convex hull of the mesh vertices projected onto the plane (the sweep precisely contains the mesh extents along the plane normal)
	* @param CopyToMesh	The Dynamic Mesh to store the swept hull geometry.
	* @param CopyToMeshOut	The resulting swept hull.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Containment", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeMeshSweptHull(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
		UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
		FTransform ProjectionFrame,
		FGeometryScriptSweptHullOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute a Convex Hull Decomposition of the given TargetMesh. Assuming more than one hull is requested,
	 * multiple hulls will be returned that attempt to approximate the mesh. If simplification settings are enabled,
	 * there is no guarantee that the entire mesh is contained in the hulls.
	 * 
	 * @warning this function can be quite expensive, and the results are expected to change in the future as the Convex Decomposition algorithm is improved
	 * @param CopyToMesh The Dynamic Mesh to store the convex hulls as a single, combined mesh. Note: SplitMeshByComponents can separate this result into its convex parts.
	 * @param CopyToMeshOut A combined mesh of the convex hulls. Note: SplitMeshByComponents can separate this result into its convex parts.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Containment|Experimental", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeMeshConvexDecomposition(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Hull Mesh", ref) UDynamicMesh* CopyToMesh, 
		UPARAM(DisplayName = "Hull Mesh") UDynamicMesh*& CopyToMeshOut, 
		FGeometryScriptConvexDecompositionOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute an oriented box fitting a given Target Mesh, or optionally a selection of the mesh
	 *
	 * @param Selection	Selection of mesh faces/vertices to contain in the oriented box. If not provided, entire mesh is used.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Containment", meta = (ScriptMethod, KeyWords = "Fit Bounding OBB"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ComputeMeshOrientedBox(
		UDynamicMesh* TargetMesh,
		FOrientedBox& OrientedBoxOut,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptFitOrientedBoxOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
