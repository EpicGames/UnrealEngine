// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryScriptTypes.h"
#include "GeometryScriptSelectionTypes.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


namespace UE::Geometry { struct FGeometrySelection; }

//~ Note that Edge selections are represented by triangle, index-in-triangle pairs,
//~ so each non-boundary edge can be represented in the selection twice (once per 'half edge').
//~ Currently, our convention is to store both representations in the selection.
/**
 * Type of index stored in a FGeometryScriptMeshSelection
 */
UENUM(BlueprintType)
enum class EGeometryScriptMeshSelectionType : uint8
{
	Vertices = 0,
	Edges = 3,
	Triangles = 1,
	Polygroups = 2 UMETA(DisplayName = "PolyGroups")
};

/**
 * Type of Conversion to apply to a FGeometryScriptMeshSelection
 */

UENUM(BlueprintType)
enum class UE_DEPRECATED(5.5, "This enum is unused, and may be removed in the future. To convert selection types, we use EGeometryScriptMeshSelectionType to specify the desired result type.")
	EGeometryScriptMeshSelectionConversionType : uint8
{
	NoConversion = 0,
	ToVertices = 1,
	ToTriangles = 2,
	ToPolygroups = 3 UMETA(DisplayName = "To PolyGroups")
};

/**
 * Type of Combine operation to use when combining multiple FGeometryScriptMeshSelection
 */
UENUM(BlueprintType)
enum class EGeometryScriptCombineSelectionMode : uint8
{
	Add,
	Subtract,
	Intersection
};

/**
 * Behavior of operations when a MeshSelection is empty
 */
UENUM(BlueprintType)
enum class EGeometryScriptEmptySelectionBehavior : uint8
{
	FullMeshSelection = 0,
	EmptySelection = 1
};

/**
 * FGeometryScriptMeshSelection is a container for a Mesh Selection used in Geometry Script.
 * The actual selection representation is not exposed to BP, 
 * use functions in MeshSelectionFunctions/etc to manipulate the selection.
 * 
 * Internally the selection is stored as a SharedPtr to a FGeometrySelection, which
 * stores a TSet (so unique add and remove are efficient, but the selection cannot
 * be directly indexed without converting to an Array). 
 *
 * Note that the Selection storage is not a UProperty, and is not
 * serialized. FGeometryScriptMeshSelection instances *cannot* be serialized,
 * they are only transient data structures, that can be created and used Blueprint instances.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Mesh Selection"))
struct FGeometryScriptMeshSelection
{
	GENERATED_BODY()

	UE_API FGeometryScriptMeshSelection();

	UE_API void SetSelection(const FGeometryScriptMeshSelection& Selection);
	UE_API void SetSelection(const UE::Geometry::FGeometrySelection& Selection);
	UE_API void SetSelection(UE::Geometry::FGeometrySelection&& Selection);
	UE_API void ClearSelection();

	UE_API bool IsEmpty() const;

	UE_API EGeometryScriptMeshSelectionType GetSelectionType() const;

	// Note that for edge selections, this can return more elements than expected because both can be redundantly represented
	// i.e. an edge can be in the selection once per 'half edge', since it is represented by triangle/index-in-triangle pair.
	// Call GetNumUniqueValidSelected(Mesh) to get the number of unique elements in the selection
	UE_API int32 GetNumSelected() const;
	// Return the number of valid, unique elements in the selection; e.g., with no double-counting of selected edges
	UE_API int32 GetNumUniqueSelected(const UE::Geometry::FDynamicMesh3& Mesh) const;
	UE_API void DebugPrint() const;

	static bool ConvertIndexTypeToSelectionType(EGeometryScriptIndexType IndexType, EGeometryScriptMeshSelectionType& OutSelectionType)
	{
		switch (IndexType)
		{
		case EGeometryScriptIndexType::Triangle:
			OutSelectionType = EGeometryScriptMeshSelectionType::Triangles;
			return true;
		case EGeometryScriptIndexType::Edge:
			OutSelectionType = EGeometryScriptMeshSelectionType::Edges;
			return true;
		case EGeometryScriptIndexType::Vertex:
			OutSelectionType = EGeometryScriptMeshSelectionType::Vertices;
			return true;
		case EGeometryScriptIndexType::PolygroupID:
			OutSelectionType = EGeometryScriptMeshSelectionType::Polygroups;
			return true;
		}
		return false;
	}

	/** 
	 * Combine SelectionB with current selection, updating current selection, using CombineMode to control how combining happens
	 */
	UE_API void CombineSelectionInPlace(const FGeometryScriptMeshSelection& SelectionB, EGeometryScriptCombineSelectionMode CombineMode);

	/**
	 * Convert the current selection to a TArray, optionally converting to ConvertToType.
	 * For (Tri|Group)=>Vtx, all triangle vertices (in triangles or PolyGroups) are included.
	 * For Vtx=>Tri, all one-ring vertices are included. For Group=>Tri, all Triangles are found via enumerating over mesh.
	 * (Tri|Vtx)=>Group, all GroupIDs of all triangles/one-ring triangles are included
	 */
	UE_API EGeometryScriptIndexType ConvertToMeshIndexArray(const UE::Geometry::FDynamicMesh3& Mesh, TArray<int32>& IndexListOut, EGeometryScriptIndexType ConvertToType = EGeometryScriptIndexType::Any) const;

	/**
	 * Call PerTriangleFunc for each TriangleID in the Selection.
	 * For Vertex Selections, Vertex one-rings are enumerated and accumulated in a TSet.
	 * For PolyGroup Selections, a full mesh iteration is used to find all Triangles in the groups.
	 */
	UE_API void ProcessByTriangleID(const UE::Geometry::FDynamicMesh3& Mesh,
		TFunctionRef<void(int32)> PerTriangleFunc,
		bool bProcessAllTrisIfSelectionEmpty = false) const;

	/**
	 * Call PerVertexFunc for each VertexID in the Selection.
	 * For Triangle Selections, Triangles Vertex tuples are enumerated and accumulated in a TSet.
	 * For PolyGroup Selections, a full mesh iteration is used to find all Triangle Vertices in the groups (accumulated in a TSet)
	 */
	UE_API void ProcessByVertexID(const UE::Geometry::FDynamicMesh3& Mesh,
		TFunctionRef<void(int32)> PerVertexFunc,
		bool bProcessAllVertsIfSelectionEmpty = false) const;

	/**
	 * Call PerEdgeFunc for each EdgeID in the Selection.
	 * For Vertex Selections, Vertex Edge one-rings are enumerated and accumulated in a TSet.
	 * For Triangle Selections, Triangle Edges are enumerated and accumulated in a TSet.
	 * For PolyGroup Selections, a full mesh iteration is used to find all Triangle Edges in the groups (accumulated in a TSet)
	 */
	UE_API void ProcessByEdgeID(const UE::Geometry::FDynamicMesh3& Mesh,
		TFunctionRef<void(int32)> PerEdgeFunc,
		bool bProcessAllVertsIfSelectionEmpty = false) const;


	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptMeshSelection& Other) const
	{
		return GeoSelection.Get() == Other.GeoSelection.Get(); 
	}
	bool operator!=(const FGeometryScriptMeshSelection& Other) const
	{
		return GeoSelection.Get() != Other.GeoSelection.Get(); 
	}

private:
	// keeping this private for now in case it needs to be revised in 5.2+
	TSharedPtr<UE::Geometry::FGeometrySelection> GeoSelection;
};


template<>
struct TStructOpsTypeTraits<FGeometryScriptMeshSelection> : public TStructOpsTypeTraitsBase2<FGeometryScriptMeshSelection>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

#undef UE_API
