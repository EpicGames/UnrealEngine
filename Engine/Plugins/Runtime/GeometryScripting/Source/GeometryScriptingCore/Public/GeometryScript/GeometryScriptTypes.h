// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryBase.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "GeometryScriptTypes.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(template<typename MeshType> class TMeshAABBTree3);
PREDECLARE_GEOMETRY(template<typename MeshType> class TFastWindingTree);
PREDECLARE_GEOMETRY(typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3);
PREDECLARE_GEOMETRY(template<typename RealType> class TGeneralPolygon2);
PREDECLARE_GEOMETRY(typedef TGeneralPolygon2<double> FGeneralPolygon2d);
PREDECLARE_GEOMETRY(class FSphereCovering);


UENUM(BlueprintType)
enum class EGeometryScriptOutcomePins : uint8
{
	Failure,
	Success
};

UENUM(BlueprintType)
enum class EGeometryScriptSearchOutcomePins : uint8
{
	Found,
	NotFound
};

UENUM(BlueprintType)
enum class EGeometryScriptContainmentOutcomePins : uint8
{
	Inside, 
	Outside
};


/**
 * The Type of LOD in a Mesh Asset (ie a StaticMesh Asset)
 */
UENUM(BlueprintType)
enum class EGeometryScriptLODType : uint8
{
	/** The Maximum-quality available SourceModel LOD (HiResSourceModel if it is available, otherwise SourceModel LOD0) */
	MaxAvailable,
	/** The HiRes SourceModel. LOD Index is ignored. HiResSourceModel is not available at Runtime. */
	HiResSourceModel,
	/** 
	 * The SourceModel mesh at a given LOD Index. Note that a StaticMesh Asset with Auto-Generated LODs may not have a valid SourceModel for every LOD Index 
	 * SourceModel meshes are not available at Runtime.
	 */
	SourceModel,
	/** 
	 * The Render mesh at at given LOD Index. 
	 * A StaticMesh Asset derives its RenderData LODs from it's SourceModel LODs. RenderData LODs always exist for every valid LOD Index.
	 * However the RenderData LODs are not identical to SourceModel LODs, in particular they will be split at UV seams, Hard Normal creases, etc.
	 * RenderData LODs in a StaticMesh Asset are only available at Runtime if the bAllowCPUAccess flag was enabled on the Asset at Cook time.
	 */
	RenderData
};


UENUM(BlueprintType)
enum class EGeometryScriptAxis : uint8
{
	X = 0,
	Y = 1,
	Z = 2
};


UENUM(BlueprintType)
enum class EGeometryScriptCoordinateSpace : uint8
{
	Local = 0,
	World = 1
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshReadLOD
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	EGeometryScriptLODType LODType = EGeometryScriptLODType::MaxAvailable;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	int32 LODIndex = 0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshWriteLOD
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	bool bWriteHiResSource = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	int32 LODIndex = 0;
};



//
// Collision Shapes
//

// Holds simple shapes that can be used for collision
USTRUCT(BlueprintType)
struct FGeometryScriptSimpleCollision
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FKAggregateGeom AggGeom;
	
};

// A set of spheres used to represent a volume
USTRUCT(BlueprintType, meta = (DisplayName = "Sphere Covering"))
struct FGeometryScriptSphereCovering
{
	GENERATED_BODY()
public:
	TSharedPtr<UE::Geometry::FSphereCovering> Spheres;

	UE_API void Reset();

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptSphereCovering& Other) const
	{
		return Spheres.Get() == Other.Spheres.Get();
	}
	bool operator!=(const FGeometryScriptSphereCovering& Other) const
	{
		return Spheres.Get() != Other.Spheres.Get();
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptSphereCovering> : public TStructOpsTypeTraitsBase2<FGeometryScriptSphereCovering>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};



// Settings to control the triangulation of simple collision primitives -- used for conversion to mesh or convex hull geometry
USTRUCT(BlueprintType)
struct FGeometryScriptSimpleCollisionTriangulationOptions
{
	GENERATED_BODY()
public:

	// When triangulating a sphere by deforming a cube to the sphere, number of vertices to use along each edge of the cube
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 SphereStepsPerSide = 4;

	// When triangulating a capsule's spherical endcaps, number of vertices to use on the arcs across the endcaps.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 CapsuleHemisphereSteps = 5;

	// When triangulating a capsule, number of vertices to use for the circular cross-sections
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 CapsuleCircleSteps = 8;

	// Whether to cheaply approximate level sets with cubes. Otherwise, will use marching cubes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bApproximateLevelSetsWithCubes = false;

};


//
// Triangles
//

USTRUCT(BlueprintType)
struct FGeometryScriptTriangle
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Vector0 = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Vector1 = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Vector2 = FVector::ZeroVector;
};


USTRUCT(BlueprintType)
struct FGeometryScriptTrianglePoint
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bValid = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int TriangleID = -1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector BaryCoords = FVector::ZeroVector;
};


USTRUCT(BlueprintType)
struct FGeometryScriptUVTriangle
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UV0 = FVector2D::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UV1 = FVector2D::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UV2 = FVector2D::ZeroVector;
};




//
// Colors
//

USTRUCT(BlueprintType)
struct FGeometryScriptColorFlags
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Color)
	bool bRed = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Color)
	bool bGreen = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Color)
	bool bBlue = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Color)
	bool bAlpha = true;

	bool AllSet() const { return bRed && bGreen && bBlue && bAlpha; }
};



//
// PolyGroups
//

/**
 * FGeometryScriptGroupLayer identifies a PolyGroup Layer of a Mesh.
 * The Default Layer always exists, Extended layers may or may not exist.
 */
USTRUCT(BlueprintType)
struct FGeometryScriptGroupLayer
{
	GENERATED_BODY()
public:
	/** If true,the default/standard PolyGroup Layer is used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	bool bDefaultLayer = true;

	/** Index of an extended PolyGroup Layer (which may or may not exist on any given Mesh) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LOD)
	int ExtendedLayerIndex = 0;
};


//
// Attributes
//

USTRUCT(BlueprintType, meta = (DisplayName = "Weight Map Handle"))
struct FGeometryScriptWeightMapHandle
{
	GENERATED_BODY()

public:

	bool IsValid() const
	{
		return WeightMapAttributeLayerIndex >= 0;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Handle")
	int WeightMapAttributeLayerIndex = -1;
};


//
// List Types
//

// By default structs exposed to Python will use a per-UPROPERTY comparison. When this doesn't give correct results
// (e.g., for structs with no properties, which will compare equal in all cases because there are no properties to
// compare), it is necessary to define explicit equality operators and add the following WithIdenticalViaEquality trait.
// Note that users can write blueprints/python scripts which pass these lists to many function calls so we would like to
// avoid copying them (very slow if they have millions of elements) so we defined the equality operations using pointer
// equality but this is potentially confusing if users expect that different lists with the same elements compare equal.

UENUM(BlueprintType)
enum class EGeometryScriptIndexType : uint8
{
	// Index lists of Any type are compatible with any other index list type
	Any,
	Triangle,
	Edge,
	Vertex,
	MaterialID,
	PolygroupID UMETA(DisplayName = "PolyGroup ID")
};

USTRUCT(BlueprintType, meta = (DisplayName = "Index List"))
struct FGeometryScriptIndexList
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IndexList)
	EGeometryScriptIndexType IndexType = EGeometryScriptIndexType::Any;
public:
	TSharedPtr<TArray<int>> List;

	void Reset(EGeometryScriptIndexType TargetIndexType, int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<int>>();
		}
		List->Reset(Num);
		IndexType = TargetIndexType;
	}

	bool IsCompatibleWith(EGeometryScriptIndexType OtherType) const
	{
		return IndexType == OtherType || IndexType == EGeometryScriptIndexType::Any;
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptIndexList& Other) const
	{
		return IndexType == Other.IndexType && List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptIndexList& Other) const
	{
		return !(*this == Other);
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptIndexList> : public TStructOpsTypeTraitsBase2<FGeometryScriptIndexList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};


USTRUCT(BlueprintType, meta = (DisplayName = "Triangle List"))
struct FGeometryScriptTriangleList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FIntVector>> List;

	void Reset(int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<FIntVector>>();
		}
		List->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptTriangleList& Other) const
	{
		return List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptTriangleList& Other) const
	{
		return List.Get() != Other.List.Get(); 
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptTriangleList> : public TStructOpsTypeTraitsBase2<FGeometryScriptTriangleList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};


USTRUCT(BlueprintType, meta = (DisplayName = "Scalar List"))
struct FGeometryScriptScalarList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<double>> List;

	void Reset(int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<double>>();
		}
		List->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptScalarList& Other) const
	{
		return List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptScalarList& Other) const
	{
		return List.Get() != Other.List.Get(); 
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptScalarList> : public TStructOpsTypeTraitsBase2<FGeometryScriptScalarList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};



USTRUCT(BlueprintType, meta = (DisplayName = "Vector List"))
struct FGeometryScriptVectorList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FVector>> List;

	void Reset(int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<FVector>>();
		}
		List->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptVectorList& Other) const
	{
		return List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptVectorList& Other) const
	{
		return List.Get() != Other.List.Get(); 
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptVectorList> : public TStructOpsTypeTraitsBase2<FGeometryScriptVectorList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};


USTRUCT(BlueprintType, meta = (DisplayName = "UV List"))
struct FGeometryScriptUVList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FVector2D>> List;

	void Reset(int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<FVector2D>>();
		}
		List->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptUVList& Other) const
	{
		return List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptUVList& Other) const
	{
		return List.Get() != Other.List.Get(); 
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptUVList> : public TStructOpsTypeTraitsBase2<FGeometryScriptUVList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};



USTRUCT(BlueprintType, meta = (DisplayName = "Color List"))
struct FGeometryScriptColorList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FLinearColor>> List;

	void Reset(int32 Num = 0)
	{
		if (List.IsValid() == false)
		{
			List = MakeShared<TArray<FLinearColor>>();
		}
		List->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptColorList& Other) const
	{
		return List.Get() == Other.List.Get(); 
	}
	bool operator!=(const FGeometryScriptColorList& Other) const
	{
		return List.Get() != Other.List.Get(); 
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptColorList> : public TStructOpsTypeTraitsBase2<FGeometryScriptColorList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};


USTRUCT(BlueprintType, meta = (DisplayName = "PolyPath"))
struct FGeometryScriptPolyPath
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FVector>> Path;

	UPROPERTY(EditAnywhere, Category = "Options")
	bool bClosedLoop = false;

	void Reset(int32 Num = 0)
	{
		if (!Path.IsValid())
		{
			Path = MakeShared<TArray<FVector>>();
		}
		Path->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptPolyPath& Other) const
	{
		return bClosedLoop == Other.bClosedLoop && Path.Get() == Other.Path.Get();
	}
	bool operator!=(const FGeometryScriptPolyPath& Other) const
	{
		return !(*this == Other);
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptPolyPath> : public TStructOpsTypeTraitsBase2<FGeometryScriptPolyPath>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

// A simple 2D Polygon with no holes
USTRUCT(BlueprintType, meta = (DisplayName = "Simple Polygon"))
struct FGeometryScriptSimplePolygon
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<FVector2D>> Vertices;

	void Reset(int32 Num = 0)
	{
		if (Vertices.IsValid() == false)
		{
			Vertices = MakeShared<TArray<FVector2D>>();
		}
		Vertices->Reset(Num);
	}

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptSimplePolygon& Other) const
	{
		return Vertices.Get() == Other.Vertices.Get();
	}
	bool operator!=(const FGeometryScriptSimplePolygon& Other) const
	{
		return Vertices.Get() != Other.Vertices.Get();
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptSimplePolygon> : public TStructOpsTypeTraitsBase2<FGeometryScriptSimplePolygon>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};



// A list of general polygons, which may have holes.
USTRUCT(BlueprintType, meta = (DisplayName = "PolygonList"))
struct FGeometryScriptGeneralPolygonList
{
	GENERATED_BODY()
public:
	TSharedPtr<TArray<UE::Geometry::FGeneralPolygon2d>> Polygons;

	UE_API void Reset(int32 Num = 0);

	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptGeneralPolygonList& Other) const
	{
		return Polygons.Get() == Other.Polygons.Get();
	}
	bool operator!=(const FGeometryScriptGeneralPolygonList& Other) const
	{
		return !(*this == Other);
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptGeneralPolygonList> : public TStructOpsTypeTraitsBase2<FGeometryScriptGeneralPolygonList>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};




//
// Spatial data structures
//


USTRUCT(BlueprintType, meta = (DisplayName = "DynamicMesh BVH Cache"))
struct FGeometryScriptDynamicMeshBVH
{
	GENERATED_BODY()
public:

	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3> Spatial;
	TSharedPtr<UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3>> FWNTree;
	
	// Required by TStructOpsTypeTraits interface
	bool operator==(const FGeometryScriptDynamicMeshBVH& Other) const
	{
		return Spatial.Get() == Other.Spatial.Get() && FWNTree.Get() == Other.FWNTree.Get();
	}
	bool operator!=(const FGeometryScriptDynamicMeshBVH& Other) const
	{
		return !(*this == Other);
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryScriptDynamicMeshBVH> : public TStructOpsTypeTraitsBase2<FGeometryScriptDynamicMeshBVH>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};





//
// Render Capture data structures
//


USTRUCT(BlueprintType)
struct FGeometryScriptRenderCaptureCamera
{
	GENERATED_BODY()

	/** The pixel resolution of render capture photo set, this value is used for width and height */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int Resolution = 256;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double FieldOfViewDegrees = 45.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector ViewPosition = FVector::Zero();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector ViewDirection = FVector::UnitX();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double NearPlaneDist = 1.0;
};



USTRUCT(BlueprintType)
struct FGeometryScriptRenderCaptureCamerasForBoxOptions
{
	GENERATED_BODY()

	/** The pixel resolution of render capture photos, this value is used for width and height */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int Resolution = 256;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double FieldOfViewDegrees = 45.;

	/** Enable 6 directions corresponding to views from box face centers to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromBoxFaces = true;

	/** Enable 4 directions corresponding to views from box upper corners to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromUpperCorners = false;

	/** Enable 4 directions corresponding to views from box lower corners to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromLowerCorners = false;

	/** Enable 4 directions corresponding to views from box upper edges centers to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromUpperEdges = false;

	/** Enable 4 directions corresponding to views from box lower edges centers to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromLowerEdges = false;

	/** Enable 4 directions corresponding to views from box side edges centers to the box center */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bViewFromSideEdges = false;

	/** Extra positions from which to deduce view directions on the box center (located at (0,0,0)) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> ExtraViewFromPositions;
};





//
// Errors/Debugging
//



UENUM(BlueprintType)
enum class EGeometryScriptDebugMessageType : uint8
{
	ErrorMessage,
	WarningMessage
};


UENUM(BlueprintType)
enum class EGeometryScriptErrorType : uint8
{
	// warning: must only append members!
	NoError,
	UnknownError,
	InvalidInputs,
	OperationFailed
};


USTRUCT(BlueprintType)
struct FGeometryScriptDebugMessage
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptDebugMessageType MessageType = EGeometryScriptDebugMessageType::ErrorMessage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptErrorType ErrorType = EGeometryScriptErrorType::UnknownError;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FText Message = FText();
};


UCLASS(MinimalAPI, BlueprintType, meta = (TestMetadata))
class UGeometryScriptDebug : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Messages)
	TArray<FGeometryScriptDebugMessage> Messages;


	void Append(const FGeometryScriptDebugMessage& MessageIn)
	{
		Messages.Add(MessageIn);
	}
};



namespace UE
{
namespace Geometry
{
	GEOMETRYSCRIPTINGCORE_API FGeometryScriptDebugMessage MakeScriptError(EGeometryScriptErrorType ErrorTypeIn, const FText& MessageIn);
	GEOMETRYSCRIPTINGCORE_API FGeometryScriptDebugMessage MakeScriptWarning(EGeometryScriptErrorType WarningTypeIn, const FText& MessageIn);


	GEOMETRYSCRIPTINGCORE_API void AppendError(UGeometryScriptDebug* Debug, EGeometryScriptErrorType ErrorTypeIn, const FText& MessageIn);
	GEOMETRYSCRIPTINGCORE_API void AppendWarning(UGeometryScriptDebug* Debug, EGeometryScriptErrorType WarningTypeIn, const FText& MessageIn);

	/**
	 * These variants of AppendError/Warning are for direct write to a debug message array.
	 * This may be useful in cases where a function is async and needs to accumulate
	 * debug messages for later collation on a game thread.
	 */
	GEOMETRYSCRIPTINGCORE_API void AppendError(TArray<FGeometryScriptDebugMessage>* DebugMessages, EGeometryScriptErrorType ErrorTypeIn, const FText& MessageIn);
	GEOMETRYSCRIPTINGCORE_API void AppendWarning(TArray<FGeometryScriptDebugMessage>* DebugMessages, EGeometryScriptErrorType WarningTypeIn, const FText& MessageIn);
}
}

#undef UE_API
