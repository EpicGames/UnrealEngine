// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "ListUtilityFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_List"))
class UGeometryScriptLibrary_ListUtilityFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	* Returns the number of Items in Index List.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetIndexListLength(FGeometryScriptIndexList IndexList);

	/**
	* Returns the index of the last element in the Index List.
	* Note, the value -1 will be returned if the list is empty or invalid. 
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetIndexListLastIndex(FGeometryScriptIndexList IndexList);

	/**
	* Returns the item associated with Index in the Index List.
	* If Index is not valid for this Index List the value -1 will be returned and bIsValidIndex will be set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetIndexListItem(FGeometryScriptIndexList IndexList, int Index, bool& bIsValidIndex);

	/**
	* Updates the value associated with Index in the Index List.  
	* If the Index is invalid, the operation will fail and in this case bValidIndex will be set to false on return.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void SetIndexListItem(UPARAM(ref) FGeometryScriptIndexList& IndexList, int Index, int NewValue, bool& bIsValidIndex);

	/**
	* Populates Index Array with the integer values stored in the Index List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertIndexListToArray(FGeometryScriptIndexList IndexList, TArray<int>& IndexArray);

	/**
	* Populates Index List of the specified Index Type from the integer values stored in the Index Array.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToIndexList(const TArray<int>& IndexArray, FGeometryScriptIndexList& IndexList, EGeometryScriptIndexType IndexType = EGeometryScriptIndexType::Any);

	/**
	* Updates Duplicate List to be identical to Index List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void DuplicateIndexList(FGeometryScriptIndexList IndexList, FGeometryScriptIndexList& DuplicateList);

	/**
	* Set each value in Index List to the given Clear Value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ClearIndexList(UPARAM(ref) FGeometryScriptIndexList& IndexList, int ClearValue = 0);

	/**
	* Returns the number of Triangles in the  Triangle list.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetTriangleListLength(FGeometryScriptTriangleList TriangleList);

	/**
	* Returns the index of the last element in the Triangle List.  
	* If the Triangle List is empty or invalid, the value 0 will be returned.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetTriangleListLastTriangle(FGeometryScriptTriangleList TriangleList);

	/**
	* Returns the integer triplet associated with the index Triangle in the Triangle  List.
	* If Triangle is not valid for this Triangle List, the triplet (-1, -1, -1) will be returned and bIsValidIndex set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API FIntVector GetTriangleListItem(FGeometryScriptTriangleList TriangleList, int Triangle, bool& bIsValidTriangle);

	/**
	* Converts Triangle List to Triangle Array by populating with the appropriate integer triplets.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertTriangleListToArray(FGeometryScriptTriangleList TriangleList, UPARAM(DisplayName = "Triangle Tuple Array") TArray<FIntVector>& TriangleArray);

	/**
	* Converts a Triangle Array of integer triplets to a Triangle List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToTriangleList(UPARAM(DisplayName = "Triangle Tuple Array") const TArray<FIntVector>& TriangleArray, FGeometryScriptTriangleList& TriangleList);


	/**
	* Returns the number of items in the Scalar List.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetScalarListLength(FGeometryScriptScalarList ScalarList);

	/**
	* Returns the index of the last Scalar in Scalar List.
	* If Scalar List is empty or invalid, the value -1 will be returned 
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetScalarListLastIndex(FGeometryScriptScalarList ScalarList);

	/**
	* Returns the Scalar value associated with Index in Scalar List.
	* If the Index is not valid for this Scalar List, the value 0.0 will be returned and bIsValidIndex set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API double GetScalarListItem(FGeometryScriptScalarList ScalarList, int Index, bool& bIsValidIndex);

	/**
	* Updates the value associated with Index in the Scalar List.  
	* If the Index is invalid, the operation will fail and bValidIndex will be set to false.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void SetScalarListItem(UPARAM(ref) FGeometryScriptScalarList& ScalarList, int Index, double NewValue, bool& bIsValidIndex);

	/**
	* Converts a Scalar List to an Scalar Array (an array of doubles).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertScalarListToArray(FGeometryScriptScalarList ScalarList, TArray<double>& ScalarArray);

	/**
	* Converts an array of doubles (Scalar Array) to Scalar List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToScalarList(UPARAM(DisplayName = "Scalar Array") const TArray<double>& VectorArray, FGeometryScriptScalarList& ScalarList);

	/**
	* Copies the contents of Scalar List into Duplicate List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void DuplicateScalarList(FGeometryScriptScalarList ScalarList, FGeometryScriptScalarList& DuplicateList);

	/**
	* Resets all the items in the Scalar List to the Clear Value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ClearScalarList(UPARAM(ref) FGeometryScriptScalarList& ScalarList, double ClearValue = 0.0);



	/**
	* Returns the number of items in the Vector List.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetVectorListLength(FGeometryScriptVectorList VectorList);

	/**
	* Returns the index of the last item in the Vector List.
	* If Vector List is empty or invalid, the value -1 will be returned. 
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetVectorListLastIndex(FGeometryScriptVectorList VectorList);

	/**
	* Returns the FVector stored in the VectorList at the specified location.
	* if the Index is not valid for this Vector List, the Zero Vector will be returned and bIsValidIndex set to false. 
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API FVector GetVectorListItem(FGeometryScriptVectorList VectorList, int Index, bool& bIsValidIndex);

	/**
	* Updates the value of the FVector stored in the Vector List at the specified location.
	* If the Index is invalid, the operation will fail and bValidIndex will be set to false.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void SetVectorListItem(UPARAM(ref) FGeometryScriptVectorList& VectorList, int Index, FVector NewValue, bool& bIsValidIndex);

	/**
	* Converts Vector List to an array of FVectors (Vector Array).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertVectorListToArray(FGeometryScriptVectorList VectorList, TArray<FVector>& VectorArray);

	/**
	* Converts an Array of FVectors (Vector Array) to Vector List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToVectorList(const TArray<FVector>& VectorArray, FGeometryScriptVectorList& VectorList);

	/**
	* Copies the contents of Vector List into Duplicate Vector List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void DuplicateVectorList(FGeometryScriptVectorList VectorList, FGeometryScriptVectorList& DuplicateList);

	/**
	* Resets all the items in the Vector List to the Clear Value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ClearVectorList(UPARAM(ref) FGeometryScriptVectorList& VectorList, FVector ClearValue = FVector::ZeroVector);



	/**
	* Returns the number of items in the UV List.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetUVListLength(FGeometryScriptUVList UVList);

	/**
	* Returns the index of the last item in the UV List.
	* If UV List is empty or invalid, the value -1 will be returned.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetUVListLastIndex(FGeometryScriptUVList UVList);

	/**
	* Returns the FVector2D stored in the UV List at the specified location.
	* If the Index is not valid for this UV List, the Zero Vector will be returned and bIsValidIndex set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API FVector2D GetUVListItem(FGeometryScriptUVList UVList, int Index, bool& bIsValidIndex);

	/**
	* Updates the value of the FVector2D stored in the UV List at the specified location.
	* If the Index is invalid, the operation will fail and bValidIndex will be set to false on return.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void SetUVListItem(UPARAM(ref) FGeometryScriptUVList& UVList, int Index, FVector2D NewUV, bool& bIsValidIndex);

	/**
	* Converts a UV List to an array of FVector2Ds (UV Array).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertUVListToArray(FGeometryScriptUVList UVList, TArray<FVector2D>& UVArray);

	/**
	* Converts an array of FVector2D (UV Array) to UV List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToUVList(const TArray<FVector2D>& UVArray, FGeometryScriptUVList& UVList);

	/**
	* Duplicates the contents of UV List into Duplicate List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void DuplicateUVList(FGeometryScriptUVList UVList, FGeometryScriptUVList& DuplicateList);

	/**
	* Resets all the items in the Vector List to the given Clear UV value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ClearUVList(UPARAM(ref) FGeometryScriptUVList& UVList, FVector2D ClearUV);


	/**
	* Returns the number of items in the Color List.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetColorListLength(FGeometryScriptColorList ColorList);

	/**
	* Returns the index of the last item in the Color List.
	* If Color List is empty or invalid, the value -1 will be returned.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API int GetColorListLastIndex(FGeometryScriptColorList ColorList);

	/**
	* Returns the FLinearColor stored in the Color List at the specified location.
	* If the Index is not valid for this Color List, FLinearColor::White will be returned and bIsValidIndex set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API FLinearColor GetColorListItem(FGeometryScriptColorList ColorList, int Index, bool& bIsValidIndex);

	/**
	* Updates the value of the FLinearColor stored in the Color List at the specified location.
	* If the Index is invalid, the operation will fail and bValidIndex will be set to false.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void SetColorListItem(UPARAM(ref) FGeometryScriptColorList& ColorList, int Index, FLinearColor NewColor, bool& bIsValidIndex);

	/**
	* Converts the Color List to an array of FLinearColor (Color Array).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ConvertColorListToArray(FGeometryScriptColorList ColorList, TArray<FLinearColor>& ColorArray);

	/**
	* Converts an array of FLinearColor (Color Array) to a Color List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils")
	static UE_API void ConvertArrayToColorList(const TArray<FLinearColor>& ColorArray, FGeometryScriptColorList& ColorList);

	/**
	* Duplicates the contents of Color List into Duplicate List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void DuplicateColorList(FGeometryScriptColorList ColorList, FGeometryScriptColorList& DuplicateList);

	/**
	* Resets all the items in the Color List to the specified Clear Color.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ClearColorList(UPARAM(ref) FGeometryScriptColorList& ColorList, FLinearColor ClearColor);

	/**
	* Populates a Scalar List with values that correspond to the 0, 1, 2, or 3 channel of a Color List.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ExtractColorListChannel(FGeometryScriptColorList ColorList, FGeometryScriptScalarList& ScalarList, int32 ChannelIndex = 0);

	/**
	* Populates a Vector List from a Color List. The channels in the Color List are mapped to vector components by means of X Channel Index, Y Channel Index, and Z Channel Index.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta=(ScriptMethod))
	static UE_API void ExtractColorListChannels(FGeometryScriptColorList ColorList, FGeometryScriptVectorList& VectorList, int32 XChannelIndex = 0, int32 YChannelIndex = 1, int32 ZChannelIndex = 2);

	/**
	 * Populates a Scalar List with values that correspond to the U or V component of a UV List.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void ExtractUVListComponent(FGeometryScriptUVList UVList, FGeometryScriptScalarList& ScalarList, int32 ComponentIndex = 0);

	/**
	 * Populates a Scalar List with values that correspond to the X Y or Z component of a Vector List.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void ExtractVectorListComponent(FGeometryScriptVectorList VectorList, FGeometryScriptScalarList& ScalarList, int32 ComponentIndex = 0);

	/**
	 * Populates a UV List with values that correspond to the X Y or Z components of a Vector List.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod, DisplayName = "Extract Vector List Components As UVs"))
	static UE_API void ExtractVectorListComponentsAsUVs(FGeometryScriptVectorList VectorList, FGeometryScriptUVList& UVList, 
		int32 UComponentIndex = 0, int32 VComponentIndex = 1);

	/** Sets the specified Color Channel of the Color List from the corresponding values in the Scalar List, resizing Color List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void SetColorListChannelFromScalars(UPARAM(ref) FGeometryScriptColorList& ColorList, FGeometryScriptScalarList ScalarList,
		int32 ColorChannel = 0, FLinearColor DefaultColor = FLinearColor::Black);

	/** Sets the specified Color Channels of the Color List from the corresponding values in the UV List, resizing Color List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod, DisplayName = "Set Color List Channels From UVs"))
	static UE_API void SetColorListChannelsFromUVs(UPARAM(ref) FGeometryScriptColorList& ColorList, FGeometryScriptUVList UVList,
		int32 UToChannelIndex = 0, int32 VToChannelIndex = 1, FLinearColor DefaultColor = FLinearColor::Black);

	/** Sets the specified Color Channels of the Color List from the corresponding values in the Vector List, resizing Color List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void SetColorListChannelsFromVectors(UPARAM(ref) FGeometryScriptColorList& ColorList, FGeometryScriptVectorList VectorList,
		int32 XToChannelIndex = 0, int32 YToChannelIndex = 1, int32 ZToChannelIndex = 2, FLinearColor DefaultColor = FLinearColor::Black);

	/** Sets the specified Component of the UV List from the corresponding values in the Scalar List, resizing UV List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void SetUVListComponentFromScalars(UPARAM(ref) FGeometryScriptUVList& UVList, FGeometryScriptScalarList ScalarList, 
		int32 UVComponent = 0, FVector2D DefaultUV = FVector2D::ZeroVector);

	/** Sets the specified Component of the Vector List from the corresponding values in the Scalar List, resizing Vector List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod))
	static UE_API void SetVectorListComponentFromScalars(UPARAM(ref) FGeometryScriptVectorList& VectorList, FGeometryScriptScalarList ScalarList, 
		int32 VectorComponent = 0, FVector DefaultVector = FVector::ZeroVector);
	
	/** Sets the specified Components of the Vector List from the corresponding values in the UV List, resizing Vector List as needed */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|ListUtils", meta = (ScriptMethod, DisplayName = "Set Vector List Components From UVs"))
	static UE_API void SetVectorListComponentsFromUVs(UPARAM(ref) FGeometryScriptVectorList& VectorList, FGeometryScriptUVList UVList,
		int32 UToVectorComponent = 0, int32 VToVectorComponent = 1, FVector DefaultVector = FVector::ZeroVector);


};

#undef UE_API
