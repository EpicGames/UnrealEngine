// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "Engine/EngineTypes.h"   // FMeshNaniteSettings
#include "MeshAssetFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UStaticMesh;
class UDynamicMesh;
class UMaterialInterface;

/** A set of options on how to handle the situation where the bone hierarchy on a skeletal geometry does not match the
  * reference skeleton on the skeletal asset being copied to. Does not apply when copying geometry to static meshes.
  */
UENUM(BlueprintType)
enum class EGeometryScriptBoneHierarchyMismatchHandling : uint8
{
	/** Do nothing to fix the mismatch. This is dangerous and should not be used lightly. The reference skeleton and the geometry's 
	  * bone hierarchy may not match, resulting in visual glitches during rendering. */
	DoNothing = 0,
	
	/** Remap the bone bindings and bone hierarchy on the geometry to match the reference skeleton. If no bone information is present
	  * then all vertices are mapped to the root bone. 
	  */
	RemapGeometryToReferenceSkeleton = 1,
	
	/** Generate a new reference skeleton on the skeletal mesh asset that matches the bone hierarchy of the geometry being copied in.
	  * Note that virtual bones are not retained from the old reference skeleton.  
	  * If no bone information is present, then a ref skeleton is created, with a single root bone at the origin, and all vertices bound
	  * to that root bone. 
	  * No attempt is made to ensure that this reference skeleton is compatible with the skeleton object. If this is a requirement, then
	  * it is the user's responsibility to ensure they are.
	  */
	CreateNewReferenceSkeleton = 2
};

/** Options to control whether lightmap UVs are generated */
UENUM(BlueprintType)
enum class EGeometryScriptGenerateLightmapUVOptions : uint8
{
	/** Match the lightmap UV generation setting of the target LOD, if it exists. For a new LOD, match LOD 0. */
	MatchTargetLODSetting UMETA(DisplayName = "Match Target LOD Setting"),
	/** Generate lightmap UVs */
	GenerateLightmapUVs UMETA(DisplayName = "Generate Lightmap UVs"),
	/** Do not generate lightmap UVs */
	DoNotGenerateLightmapUVs UMETA(DisplayName = "Do Not Generate Lightmap UVs")
};


USTRUCT(BlueprintType)
struct FGeometryScriptCopyMeshFromAssetOptions
{
	GENERATED_BODY()
	
	// Whether to apply Build Settings during the mesh copy.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bApplyBuildSettings = true;

	// Whether to request tangents on the copied mesh. If tangents are not requested, tangent-related build settings will also be ignored.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRequestTangents = true;

	// Whether to ignore the 'remove degenerates' option from Build Settings. Note: Only applies if 'Apply Build Settings' is enabled.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bIgnoreRemoveDegenerates = true;

	// Whether to scale the copied mesh by the Build Setting's 'Build Scale'. Note: This is considered separately from the 'Apply Build Settings' option.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUseBuildScale = true;
};

/**
 * Configuration settings for Nanite Rendering on StaticMesh Assets
 */
USTRUCT(BlueprintType)
struct FGeometryScriptNaniteOptions
{
	GENERATED_BODY()

	/** Set Nanite to Enabled/Disabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnabled = true;

	/** Percentage of triangles to maintain in Fallback Mesh used when Nanite is unavailable */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float FallbackPercentTriangles = 100.0f;

	/** Relative Error to maintain in Fallback Mesh used when Nanite is unavailable. Overrides FallbackPercentTriangles. Set to 0 to only use FallbackPercentTriangles (default). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float FallbackRelativeError = 0.0f;
};

// Options for how to use the NewMaterials array when copying materials to a mesh asset
UENUM(BlueprintType)
enum class ECopyMeshToAssetUseNewMaterialsOptions : uint8
{
	// Fully replace asset materials list with the provided set
	ReplaceAssetMaterials = 0,
	// Use the provided new materials as the material for the newly-set LOD, only adding asset materials as-needed
	SetLODMaterials = 1 UMETA(DisplayName = "Set LOD Materials")
};



USTRUCT(BlueprintType)
struct FGeometryScriptCopyMeshToAssetOptions
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeNormals = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeTangents = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableRemoveDegenerates = false;

	/** An option that specifies, for skeletal mesh assets, how mismatches between the existing reference skeleton on the asset, and the bone
	  * hierarchy stored on the geometry are handled. By default, no attempt is made to resolve this mismatch.
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBoneHierarchyMismatchHandling BoneHierarchyMismatchHandling = EGeometryScriptBoneHierarchyMismatchHandling::DoNothing;
	
	/** Deprecated. Use BoneHierarchyMismatchHandling instead.
	 */
	// UE_DEPRECATED(5.6, "Deprecated. Use BoneHierarchyMismatchHandling instead.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Deprecated)
	bool bRemapBoneIndicesToMatchAsset = false;

	/** Use the original vertex order found in the source data. This is useful if the inbound mesh was originally non-manifold, and needs to keep
	 *  the non-manifold structure when re-created. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUseOriginalVertexOrder = false;

	// Whether to use the build scale on the target asset. If enabled, the inverse scale will be applied when saving to the asset, and the BuildScale will be preserved. Otherwise, BuildScale will be set to 1.0 on the asset BuildSettings.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUseBuildScale = true;
	
	// Whether to update the materials on the asset with those in the New Materials array
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(DisplayName = "Use New Materials"))
	bool bReplaceMaterials = false;

	// If using New Materials, whether to use them as a full replacement of all asset materials, or just to update the target LOD
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	ECopyMeshToAssetUseNewMaterialsOptions NewMaterialUsage = ECopyMeshToAssetUseNewMaterialsOptions::ReplaceAssetMaterials;

	// New materials to set if Replace Materials is enabled. Ignored otherwise.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<TObjectPtr<UMaterialInterface>> NewMaterials;

	// Optional slot names for the New Materials. Ignored if not the same length as the New Materials array.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FName> NewMaterialSlotNames;

	// Whether to apply a cleanup pass to assigned materials -- fixing invalid material assignments
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCleanAssignedMaterials = false;

	// Whether to generate lightmap UVs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptGenerateLightmapUVOptions GenerateLightmapUVs = EGeometryScriptGenerateLightmapUVOptions::MatchTargetLODSetting;

	/** If enabled, NaniteSettings will be applied to the target Asset if possible */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bApplyNaniteSettings = false;

	/** Replaced FGeometryScriptNaniteOptions with usage of Engine FMeshNaniteSettings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Deprecated, AdvancedDisplay, meta=(DisplayName = "DEPRECATED NANITE SETTING") )
	FGeometryScriptNaniteOptions NaniteSettings = FGeometryScriptNaniteOptions();

	/** Nanite Settings applied to the target Asset, if bApplyNaniteSettings = true */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName = "Nanite Settings"))
	FMeshNaniteSettings NewNaniteSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmitTransaction = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDeferMeshPostEditChange = false;
};

USTRUCT(BlueprintType)
struct FGeometryScriptCopyMorphTargetToAssetOptions
{
	GENERATED_BODY()

	/** If true and the morph target with the given name exists, it will be overwritten. If false, will abort and print a 
	 * console error. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOverwriteExistingTarget = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmitTransaction = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDeferMeshPostEditChange = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCopyNormals = false;
};


USTRUCT(BlueprintType)
struct FGeometryScriptCopySkinWeightProfileToAssetOptions
{
	GENERATED_BODY()

	/** If true and a skin weight profile with the given name exists, it will be overwritten. 
	 *  If false, will abort and print a console error. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOverwriteExistingProfile = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmitTransaction = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDeferMeshPostEditChange = false;
};


// Although the class name indicates StaticMeshFunctions, that was a naming mistake that is difficult
// to correct. This class is intended to serve as a generic asset utils function library. The naming
// issue is only visible at the C++ level. It is not visible in Python or BP.
UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_AssetUtils"))
class UGeometryScriptLibrary_StaticMeshFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/** 
	* Check if a Static Mesh Asset has the RequestedLOD available, ie if CopyMeshFromStaticMesh will be able to
	* succeed for the given LODType and LODIndex. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API bool
	CheckStaticMeshHasAvailableLOD(
		UStaticMesh* StaticMeshAsset, 
		FGeometryScriptMeshReadLOD RequestedLOD,
		EGeometryScriptSearchOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Determine the number of available LODs of the requested LODType in a Static Mesh Asset
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta=(ScriptName="GetNumStaticMeshLodsOfType;GetNumStaticMeshLODsOfType"))
	static UE_API int
	GetNumStaticMeshLODsOfType(
		UStaticMesh* StaticMeshAsset, 
		EGeometryScriptLODType LODType = EGeometryScriptLODType::SourceModel);

	/** 
	* Extracts a Dynamic Mesh from a Static Mesh Asset. 
	* 
	* Note that the LOD Index in RequestedLOD will be silently clamped to the available number of LODs (SourceModel or RenderData)
	*
	* @param bUseSectionMaterials Whether to use the mesh section indices as material IDs. If true, use GetSectionMaterialListFromStaticMesh to get the corresponding materials. If false, use GetMaterialListFromStaticMesh to get the materials instead.
	*/
	//~ Note this V2 version adds the bUseSectionMaterials parameter
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug", DisplayName = "Copy Mesh From Static Mesh", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	CopyMeshFromStaticMeshV2(
		UStaticMesh* FromStaticMeshAsset, 
		UDynamicMesh* ToDynamicMesh, 
		FGeometryScriptCopyMeshFromAssetOptions AssetOptions,
		FGeometryScriptMeshReadLOD RequestedLOD,
		EGeometryScriptOutcomePins& Outcome,
		bool bUseSectionMaterials = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Extracts a Dynamic Mesh from a Static Mesh Asset, using section indices for the material IDs -- use GetSectionMaterialListFromStaticMesh to get the corresponding materials.
	*
	* Note that the LOD Index in RequestedLOD will be silently clamped to the available number of LODs (SourceModel or RenderData)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug", DisplayName = "Copy Mesh From Static Mesh with Section Materials", ExpandEnumAsExecs = "Outcome"))
	static UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh*
	CopyMeshFromStaticMesh(
		UStaticMesh* FromStaticMeshAsset,
		UDynamicMesh* ToDynamicMesh,
		FGeometryScriptCopyMeshFromAssetOptions AssetOptions,
		FGeometryScriptMeshReadLOD RequestedLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr)
	{
		return CopyMeshFromStaticMeshV2(FromStaticMeshAsset, ToDynamicMesh, AssetOptions, RequestedLOD, Outcome, true, Debug);
	}
	
	/**
	* Updates a Static Mesh Asset with new geometry converted from a Dynamic Mesh
	*
	* @param bUseSectionMaterials Whether to assume Dynamic Mesh material IDs are section indices in the target Static Mesh. Should match the value passed to CopyMeshFromStaticMesh. Has no effect if replacing the asset materials.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh*
	CopyMeshToStaticMesh(
		UDynamicMesh* FromDynamicMesh,
		UStaticMesh* ToStaticMeshAsset,
		FGeometryScriptCopyMeshToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		bool bUseSectionMaterials = true,
		UGeometryScriptDebug* Debug = nullptr);

	//~ Version for C++ api compatibility without bUseSectionMaterials parameter
	UE_DEPRECATED(5.5, "Use the version of this function with a bUseSectionMaterials parameter")
	static UDynamicMesh* CopyMeshToStaticMesh(
		UDynamicMesh* FromDynamicMesh,
		UStaticMesh* ToStaticMeshAsset,
		FGeometryScriptCopyMeshToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
	{
		return CopyMeshToStaticMesh(FromDynamicMesh, ToStaticMeshAsset, Options, TargetLOD, Outcome, true, Debug);
	}


    /** 
	* Extracts the Material List and corresponding Material Indices from the specified LOD of the Static Mesh Asset. 
	* The MaterialList is sorted by Section, so if CopyMeshToStaticMesh was used to create a DynamicMesh with bUseSectionMaterials=true, then the 
	* returned MaterialList here will correspond to the MaterialIDs in that DynamicMesh (as each Static Mesh Section becomes a MaterialID, in-order). 
	* So, the returned MaterialList can be passed directly to (eg) a DynamicMeshComponent.
	* 
	* @param MaterialIndex this returned array is the same size as MaterialList, with each value the index of that Material in the StaticMesh Material List
	* @param MateriaSlotNames this returned array is the same size as MaterialList, with each value the Slot Name of that Material in the StaticMesh Material List
	*
	* Note that the LOD Index in RequestedLOD will be silently clamped to the available number of LODs (SourceModel or RenderData)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API void
	GetSectionMaterialListFromStaticMesh(
		UStaticMesh* FromStaticMeshAsset, 
		FGeometryScriptMeshReadLOD RequestedLOD,
		TArray<UMaterialInterface*>& MaterialList,
		TArray<int32>& MaterialIndex,
		TArray<FName>& MaterialSlotNames,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Extracts the Material List and corresponding Material Indices from the specified LOD of the Skeletal Mesh Asset.
	* If Copy Mesh To Skeletal Mesh was used to create a Dynamic Mesh, then the returned Material List can be passed directly to a Dynamic Mesh Component.
	*
	* @param MaterialIndex this returned array is the same size as MaterialList, with each value the index of that Material in the Skeletal Mesh's Material List
	* @param MateriaSlotNames this returned array is the same size as MaterialList, with each value the Slot Name of that Material in the Skeletal Mesh's Material List
	*
	* Note that the LOD Index in RequestedLOD will be silently clamped to the available number of LODs
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug", DisplayName = "Get LOD Material List From Skeletal Mesh", ExpandEnumAsExecs = "Outcome"))
	static UE_API void
	GetLODMaterialListFromSkeletalMesh(
		USkeletalMesh* FromSkeletalMeshAsset,
		FGeometryScriptMeshReadLOD RequestedLOD,
		TArray<UMaterialInterface*>& MaterialList,
		TArray<int32>& MaterialIndex,
		TArray<FName>& MaterialSlotNames,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the asset materials from the static mesh asset. These will match the DynamicMesh material if CopyMeshFromStaticMesh
	 * was used to create a DynamicMesh with bUseSectionMaterials=false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|StaticMesh", meta = (HidePin = "Debug"))
	static UE_API void 
	GetMaterialListFromStaticMesh(const UStaticMesh* FromStaticMeshAsset,
		TArray<UMaterialInterface*>& MaterialList,
		TArray<FName>& MaterialSlotNames,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the asset materials from the skeletal mesh asset.
	 * Note: For LOD-specific materials, use GetLODMaterialListFromSkeletalMesh instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug"))
	static UE_API void
	GetMaterialListFromSkeletalMesh(const USkeletalMesh* FromSkeletalMeshAsset,
		TArray<UMaterialInterface*>& MaterialList,
		TArray<FName>& MaterialSlotNames,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Converts material map to a material list and a slot names list. Null materials will be kept in the list, and the list will have the same number of elements as the map.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials")
	static UE_API void ConvertMaterialMapToMaterialList(const TMap<FName, UMaterialInterface*>& MaterialMap,
		TArray<UMaterialInterface*>& MaterialList,
		TArray<FName>& MaterialSlotNames);

	/**
	 * Converts material list and slot names list to material map, which is the format expected by CreateNewSkeletalMeshAssetFromMesh.
	 * Material List and Material Slot Names should have the same length. However, if there are fewer slot names than materials, 
	 * slot names will be auto-generated (as '[Name of material]_[Index]', or 'Material_[Index]' for null materials)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials")
	static UE_API UPARAM(DisplayName = "Material Map") TMap<FName, UMaterialInterface*> ConvertMaterialListToMaterialMap(const TArray<UMaterialInterface*>& MaterialList,
		const TArray<FName>& MaterialSlotNames);

	/** 
	* Extracts a Dynamic Mesh from a Skeletal Mesh Asset. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	CopyMeshFromSkeletalMesh(
		USkeletalMesh* FromSkeletalMeshAsset, 
		UDynamicMesh* ToDynamicMesh,
		FGeometryScriptCopyMeshFromAssetOptions AssetOptions,
		FGeometryScriptMeshReadLOD RequestedLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/** 
	* Updates a Skeletal Mesh Asset with new geometry and bone weights data from a Dynamic Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	CopyMeshToSkeletalMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FGeometryScriptCopyMeshToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);


   /** 
	* Add a Dynamic Mesh morph target to a Skeletal Mesh Asset.
	* 
	* @param FromMorphTarget the dynamic mesh representing the geometry of the morph target
	* @param ToSkeletalMeshAsset the asset we are writing the morph target into
	* @param MorphTargetName the name of the morph target as it will appear in the UI
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	CopyMorphTargetToSkeletalMesh(
		UDynamicMesh* FromMorphTarget, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FName MorphTargetName,
		FGeometryScriptCopyMorphTargetToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

   /** 
	* Add a Dynamic Mesh skin weight profile to a Skeletal Mesh Asset.
	* 
	* @param FromDynamicMesh the dynamic mesh representing the geometry of the morph target
	* @param ToSkeletalMeshAsset the asset we are writing the morph target into
	* @param TargetProfileName the name of the skin weight profile as it will appear in the UI. Leave blank for the default profile.
	* @param SourceProfileName The name of the skin weight profile to copy from the dynamic mesh. Leave blank for the default profile.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SkeletalMesh", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	CopySkinWeightProfileToSkeletalMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeletalMesh* ToSkeletalMeshAsset,
		FName TargetProfileName,
		FName SourceProfileName,
		FGeometryScriptCopySkinWeightProfileToAssetOptions Options,
		FGeometryScriptMeshWriteLOD TargetLOD,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
