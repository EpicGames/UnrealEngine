// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshBakeFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

UENUM(BlueprintType)
enum class EGeometryScriptBakeResolution : uint8
{
	Resolution16 UMETA(DisplayName = "16 x 16"),
	Resolution32 UMETA(DisplayName = "32 x 32"),
	Resolution64 UMETA(DisplayName = "64 x 64"),
	Resolution128 UMETA(DisplayName = "128 x 128"),
	Resolution256 UMETA(DisplayName = "256 x 256"),
	Resolution512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 UMETA(DisplayName = "8192 x 8192")
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeBitDepth : uint8
{
	ChannelBits8 UMETA(DisplayName = "8 bits/channel"),
	ChannelBits16 UMETA(DisplayName = "16 bits/channel")
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeSamplesPerPixel : uint8
{
	Sample1 UMETA(DisplayName = "1"),
	Sample4 UMETA(DisplayName = "4"),
	Sample16 UMETA(DisplayName = "16"),
	Sample64 UMETA(DisplayName = "64"),
	Samples256 UMETA(DisplayName = "256")
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeFilteringType : uint8
{
	BSpline UMETA(DisplayName = "B-Spline"),
	Box UMETA(DisplayName = "Box")
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeTypes : uint8
{
	None,
	/* Normals in tangent space */
	TangentSpaceNormal     UMETA(DisplayName = "Tangent Normal"),
	/* Interpolated normals in object space */
	ObjectSpaceNormal      UMETA(DisplayName = "Object Normal"),
	/* Geometric face normals in object space */
	FaceNormal             ,
	/* Normals skewed towards the least occluded direction */
	BentNormal             ,
	/* Positions in object space */
	Position               ,
	/* Local curvature of the mesh surface */
	Curvature              ,
	/* Ambient occlusion sampled across the hemisphere */
	AmbientOcclusion       ,
	/* Transfer a given texture */
	Texture                ,
	/* Transfer a texture per material ID */
	MultiTexture           ,
	/* Interpolated vertex colors */
	VertexColor            ,
	/* Material IDs as unique colors */
	MaterialID             UMETA(DisplayName = "Material ID"),
	/* Constant value */
	Constant,
	/* UV Shell */
	UVShell                UMETA(DisplayName = "UV Shell"),
	/* Height */
	Height
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeOutputMode : uint8
{
	RGBA,
	PerChannel
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeNormalSpace : uint8
{
	/* Tangent space */
	Tangent,
	/* Object space */
	Object
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeVertexTopology : uint8
{
	/* Create new vertex color topology */
	CreateNew,
	/* Use existing vertex color topology */
	UseExisting
};

struct FGeometryScriptBakeTypes
{
};

struct FGeometryScriptBakeType_Occlusion : public FGeometryScriptBakeTypes
{
	/** Number of occlusion rays per sample */
	int32 OcclusionRays = 16;

	/** Maximum distance for occlusion rays to test for intersections; a value of 0 means infinity */
	float MaxDistance = 0.0f;

	/** Maximum spread angle in degrees for occlusion rays; for example, 180 degrees will cover the entire hemisphere whereas 90 degrees will only cover the center of the hemisphere down to 45 degrees from the horizon. */
	float SpreadAngle = 180.0f;

	/** Angle in degrees from the horizon for occlusion rays for which the contribution is attenuated to reduce faceting artifacts. */
	float BiasAngle = 15.0f;

	/** Normal space for Bent Normals. */
	EGeometryScriptBakeNormalSpace NormalSpace = EGeometryScriptBakeNormalSpace::Tangent;
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeCurvatureTypeMode : uint8
{
	/** Average of the minimum and maximum principal curvatures */
	Mean,
	/** Maximum principal curvature */
	Max,
	/** Minimum principal curvature */
	Min,
	/** Product of the minimum and maximum principal curvatures */
	Gaussian
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeCurvatureColorMode : uint8
{
	/** Map curvature values to grayscale such that black is negative, grey is zero, and white is positive */
	Grayscale,
	/** Map curvature values to red and blue such that red is negative, black is zero, and blue is positive */
	RedBlue,
	/** Map curvature values to red, green, blue such that red is negative, green is zero, and blue is positive */
	RedGreenBlue
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeCurvatureClampMode : uint8
{
	/** Include both negative and positive curvatures */
	None,
	/** Clamp negative curvatures to zero */
	OnlyPositive,
	/** Clamp positive curvatures to zero */
	OnlyNegative
};

struct FGeometryScriptBakeType_Curvature : public FGeometryScriptBakeTypes
{
	/** Type of curvature */
	EGeometryScriptBakeCurvatureTypeMode CurvatureType = EGeometryScriptBakeCurvatureTypeMode::Mean;

	/** How to map calculated curvature values to colors */
	EGeometryScriptBakeCurvatureColorMode ColorMapping = EGeometryScriptBakeCurvatureColorMode::Grayscale;

	/** Multiplier for how the curvature values fill the available range in the selected Color Mapping; a larger value means that higher curvature is required to achieve the maximum color value. */
	float ColorRangeMultiplier = 1.0f;

	/** Minimum for the curvature values to not be clamped to zero relative to the curvature for the maximum color value; a larger value means that higher curvature is required to not be considered as no curvature. */
	float MinRangeMultiplier = 0.0f;

	/** Clamping applied to curvature values before color mapping */
	EGeometryScriptBakeCurvatureClampMode Clamping = EGeometryScriptBakeCurvatureClampMode::None;
};

UENUM(BlueprintType)
enum class EGeometryScriptBakeHeightRangeMode : uint8
{
	/** Height range defined in absolute units in object space */
	Absolute,
	/** Height range defined as a ratio of the maximal bounding box axis */
	RelativeBounds
};

struct FGeometryScriptBakeType_Height : public FGeometryScriptBakeTypes
{
	/**
	 * When set to Absolute, Inner/Outer distance are interpreted as an absolute distance range in object space.
	 * When set to RelativeBounds, Inner/Outer distance are interpreted as a ratio of the maximal bounding box axis.
	 */
	EGeometryScriptBakeHeightRangeMode HeightRangeMode = EGeometryScriptBakeHeightRangeMode::Absolute;

	/** The absolute or relative inner distance from the target mesh, in object space units. */
	float InnerDistance = -1.0f;

	/** The absolute or relative outer distance from the target mesh, in object space units. */
	float OuterDistance = 1.0f;
};

struct FGeometryScriptBakeType_UVShell : public FGeometryScriptBakeTypes
{
	/** The source UV layer to sample. */
	int SourceUVLayer = 0;
	
	/** The pixel thickness of the wireframes. */
	float WireframeThickness = 1.0f;

	/** The color for wireframe samples. */
	FLinearColor WireframeColor = FLinearColor::Blue;

	/** The color for UV shell triangle fill samples. */
	FLinearColor ShellColor = FLinearColor::Gray;

	/** The color for samples external to UV shells. */
	FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
};

struct FGeometryScriptBakeType_Texture : public FGeometryScriptBakeTypes
{
	/** Source mesh texture that is to be resampled into a new texture */
	TObjectPtr<UTexture2D> SourceTexture = nullptr;

	/** UV channel to use for the source mesh texture */
	int SourceUVLayer = 0;
};

struct FGeometryScriptBakeType_MultiTexture : public FGeometryScriptBakeTypes
{
	/** For each material ID, the source texture that will be resampled in that material's region*/
	TArray<TObjectPtr<UTexture2D>> MaterialIDSourceTextures;

	/** UV channel to use for the source mesh texture */
	int SourceUVLayer = 0;
};

struct FGeometryScriptBakeType_Constant : public FGeometryScriptBakeTypes
{
	/** Constant value to set */
	float Value = 0.0f;
};

/**
 * Opaque struct for storing bake type options.
 */
USTRUCT(BlueprintType)
struct FGeometryScriptBakeTypeOptions
{
	GENERATED_BODY()

	/** The bake output type to generate */
	UPROPERTY(BlueprintReadOnly, Category = Type)
	EGeometryScriptBakeTypes BakeType = EGeometryScriptBakeTypes::None;

	TSharedPtr<FGeometryScriptBakeTypes> Options;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeTextureOptions
{
	GENERATED_BODY()

	/** The pixel resolution of the generated textures */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeResolution Resolution = EGeometryScriptBakeResolution::Resolution256;

	/** The bit depth for each channel of the generated textures */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeBitDepth BitDepth = EGeometryScriptBakeBitDepth::ChannelBits8;

	/** The pixel distance to pad baked content past the UV borders. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(ClampMin=0))
	int32 GutterSize = 4;

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeSamplesPerPixel SamplesPerPixel = EGeometryScriptBakeSamplesPerPixel::Sample1;

	/** Mask texture for filtering out samples/pixels from the output texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> SampleFilterMask = nullptr;

	/** Filtering Type to perform on samples */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeFilteringType FilteringType = EGeometryScriptBakeFilteringType::BSpline;

	/** Maximum allowed distance for the projection from target mesh to source mesh for the sample to be considered valid.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ProjectionDistance = 3.0f;

	/** If true, uses the world space positions for the projection from target mesh to source mesh, otherwise it uses their object space positions.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bProjectionInWorldSpace = false;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeVertexOptions
{
	GENERATED_BODY()

	/** Option to generate new vertex color topology or reuse existing vertex color topology. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeVertexTopology TopologyMode = EGeometryScriptBakeVertexTopology::CreateNew;

	/** If true, compute a separate vertex color for each unique normal on a vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSplitAtNormalSeams = false;

	/** If true, compute a separate vertex color for each unique UV on a vertex. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(DisplayName = "Split at UV Seams"))
	bool bSplitAtUVSeams = false;

	/** Maximum allowed distance for the projection from target mesh to source mesh for the sample to be considered valid.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ProjectionDistance = 3.0f;

	/** If true, uses the world space positions for the projection from target mesh to source mesh, otherwise it uses their object space positions.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bProjectionInWorldSpace = false;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeOutputType
{
	GENERATED_BODY()
	
	/** The bake output mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeOutputMode OutputMode = EGeometryScriptBakeOutputMode::RGBA;

	UPROPERTY(BlueprintReadWrite, Category = Output)
	FGeometryScriptBakeTypeOptions RGBA;

	UPROPERTY(BlueprintReadWrite, Category = Output)
	FGeometryScriptBakeTypeOptions R;

	UPROPERTY(BlueprintReadWrite, Category = Output)
	FGeometryScriptBakeTypeOptions G;

	UPROPERTY(BlueprintReadWrite, Category = Output)
	FGeometryScriptBakeTypeOptions B;

	UPROPERTY(BlueprintReadWrite, Category = Output)
	FGeometryScriptBakeTypeOptions A;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeTargetMeshOptions
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName="Target UV Channel"))
	int TargetUVLayer = 0;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeSourceMeshOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> SourceNormalMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName="Source Normal UV Channel"))
	int SourceNormalUVLayer = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeNormalSpace SourceNormalSpace = EGeometryScriptBakeNormalSpace::Tangent;
};

USTRUCT(BlueprintType)
struct FGeometryScriptBakeRenderCaptureOptions
{
	GENERATED_BODY()

	/*~
	 * Options to configure the Render Capture
	 */

	/**
	 * If not empty the given Render Capture Cameras are used to create the photos from which to Bake textures
	 * If empty then a default set of Render Capture Cameras with frustums containing the SourceActors bounding box is used
	 * Note: Providing a value is highly recommended, the default camera set is intended only to support legacy blueprints
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FGeometryScriptRenderCaptureCamera> Cameras;

	/**
	 * The pixel resolution of the default render capture camera photo sets.
	 * Only used if Cameras is empty
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(DisplayName="Default Capture Resolution"))
	EGeometryScriptBakeResolution RenderCaptureResolution = EGeometryScriptBakeResolution::Resolution256;

	/**
	 * Default field of view of the default render capture cameras.
	 * Only used if Cameras is empty
	 */
	UPROPERTY()
	double FieldOfViewDegrees = 45.0;

	/**
	 * Default near plane distance for the viewing frustums of the default render capture cameras.
	 * Only used if Cameras is empty
	 */
	UPROPERTY()
	double NearPlaneDist = 1.0;



	/*~
	 * Options to configure the Baking step to create textures from the Render Capture photos
	 */

	/** The pixel resolution of the generated textures */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeResolution Resolution = EGeometryScriptBakeResolution::Resolution256;

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBakeSamplesPerPixel SamplesPerPixel = EGeometryScriptBakeSamplesPerPixel::Sample1;
	
	/** Whether to use anti-aliasing in the render captures, this may introduce artefacts if pixels at different scene depths get blended */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, DisplayName="Render Capture Anti-aliasing")
	bool bRenderCaptureAntiAliasing = false;

	/**
	 * This threshold is used to detect occlusion artefacts (e.g., silhouettes/blotches in the base color) so that texels
	 * exhibiting them can be cleaned up i.e., rejected and filled in using the values of nearby valid texels instead.
	 * 
	 * If the threshold is zero, the cleanup step is skipped.
	 * If the threshold is too large, texels with artefacts won't be detected and the cleanup step is effectively skipped.
	 * If the threshold is too small, texels without artefacts get detected and results will be bad because there won't
	 * be enough nearby valid texels from which to infill values.
	 * 
	 * A good starting point is to choose a threshold around the size of the distance (in world space) between the
	 * target and source meshes; if the target was generated by VoxWrap then the voxel size estimates this distance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float CleanupTolerance = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bBaseColorMap = true;

	/** World space normal map */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bNormalMap = true;

	/** Packed Metallic/Roughness/Specular Map. If this is true the separated channel booleans are ignored */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, DisplayName="Packed MRS Map")
	bool bPackedMRSMap = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bMetallicMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRoughnessMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSpecularMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEmissiveMap = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOpacityMap = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSubsurfaceColorMap = true;
};

USTRUCT(BlueprintType)
struct FGeometryScriptRenderCaptureTextures
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> BaseColorMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasBaseColorMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> NormalMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasNormalMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, DisplayName="Packed MRS Map")
	TObjectPtr<UTexture2D> PackedMRSMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, DisplayName="Has Packed MRS Map")
	bool bHasPackedMRSMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> MetallicMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasMetallicMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> RoughnessMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasRoughnessMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> SpecularMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasSpecularMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> EmissiveMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasEmissiveMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> OpacityMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasOpacityMap = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> SubsurfaceColorMap = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHasSubsurfaceColorMap = false;
};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Bake"))
class UGeometryScriptLibrary_MeshBakeFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake", meta=(CompactNodeTitle = "->", BlueprintAutocast))
	static UE_API UPARAM(DisplayName="Resolution Out") int ConvertBakeResolutionToInt(EGeometryScriptBakeResolution BakeResolution);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeTangentNormal();

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeObjectNormal();
	
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeFaceNormal();

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeBentNormal(
		int OcclusionRays = 16,
		float MaxDistance = 0.0f,
		float SpreadAngle = 180.0f,
		EGeometryScriptBakeNormalSpace NormalSpace = EGeometryScriptBakeNormalSpace::Tangent);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypePosition();

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeCurvature(
		EGeometryScriptBakeCurvatureTypeMode CurvatureType = EGeometryScriptBakeCurvatureTypeMode::Mean,
		EGeometryScriptBakeCurvatureColorMode ColorMapping = EGeometryScriptBakeCurvatureColorMode::Grayscale,
		float ColorRangeMultiplier = 1.0f,
		float MinRangeMultiplier = 0.0f,
		EGeometryScriptBakeCurvatureClampMode Clamping = EGeometryScriptBakeCurvatureClampMode::None);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeAmbientOcclusion(
		int OcclusionRays = 16,
		float MaxDistance = 0.0f,
		float SpreadAngle = 180.0f,
		float BiasAngle = 15.0f);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeTexture(
		UTexture2D* SourceTexture = nullptr,
		int SourceUVLayer = 0);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeMultiTexture(
		const TArray<UTexture2D*>& MaterialIDSourceTextures,
		UPARAM(DisplayName="Source UV Channel") int SourceUVLayer = 0);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeVertexColor();

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeMaterialID();

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeUVShell(
		int SourceUVLayer = 0,
		float WireframeThickness = 1.0f,
		FLinearColor WireframeColor = FLinearColor::Blue,
		FLinearColor ShellColor = FLinearColor::Gray,
		FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeHeight(
		EGeometryScriptBakeHeightRangeMode RangeMode = EGeometryScriptBakeHeightRangeMode::Absolute,
		float InnerDistance = -1.0f,
		float OuterDistance = 1.0f);

	UFUNCTION(BlueprintPure, Category = "GeometryScript|Bake")
	static UE_API UPARAM(DisplayName="Bake Type Out") FGeometryScriptBakeTypeOptions MakeBakeTypeConstant(
		float Value = 0.0f);
	
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Bake", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName="Textures Out") TArray<UTexture2D*> BakeTexture(
		UDynamicMesh* TargetMesh,
		FTransform TargetTransform,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		UDynamicMesh* SourceMesh,
		FTransform SourceTransform,
		FGeometryScriptBakeSourceMeshOptions SourceOptions,
		const TArray<FGeometryScriptBakeTypeOptions>& BakeTypes,
		FGeometryScriptBakeTextureOptions BakeOptions,
		UGeometryScriptDebug* Debug = nullptr);

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Bake", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName="Target Mesh") UDynamicMesh* BakeVertex(
		UDynamicMesh* TargetMesh,
		FTransform TargetTransform,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		UDynamicMesh* SourceMesh,
		FTransform SourceTransform,
		FGeometryScriptBakeSourceMeshOptions SourceOptions,
		FGeometryScriptBakeOutputType BakeTypes,
		FGeometryScriptBakeVertexOptions BakeOptions,
		UGeometryScriptDebug* Debug = nullptr);

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Bake", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName="Textures Out") FGeometryScriptRenderCaptureTextures
	BakeTextureFromRenderCaptures(
		UDynamicMesh* TargetMesh,
		FTransform TargetLocalToWorld,
		FGeometryScriptBakeTargetMeshOptions TargetOptions,
		const TArray<AActor*> SourceActors,
		FGeometryScriptBakeRenderCaptureOptions BakeOptions,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
