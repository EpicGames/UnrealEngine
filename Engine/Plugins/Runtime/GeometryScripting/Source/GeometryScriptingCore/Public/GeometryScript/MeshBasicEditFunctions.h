// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "GeometryScript/MeshSpatialFunctions.h"
#include "MeshBasicEditFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;
namespace UE::Geometry
{
	class FDynamicMesh3;
}


USTRUCT(BlueprintType)
struct FGeometryScriptSimpleMeshBuffers
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> Vertices;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> Normals;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV0;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV2;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV3;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV4;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV5;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV6;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector2D> UV7;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FLinearColor> VertexColors;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FIntVector> Triangles;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<int> TriGroupIDs;
};


// Options for how attributes from a source and target mesh are combined into the target mesh
UENUM(BlueprintType)
enum class EGeometryScriptCombineAttributesMode : uint8
{
	// Include attributes enabled on either the source or target mesh
	EnableAllMatching,
	// Only include attributes that are already enabled on the target mesh
	UseTarget,
	// Make the target mesh have only the attributes that are enabled on the source mesh
	UseSource
};


/**
 * Control how details like mesh attributes are handled when one mesh is appended to another
 */
USTRUCT(BlueprintType)
struct FGeometryScriptAppendMeshOptions
{
	GENERATED_BODY()
public:

	// How attributes from each mesh are combined into the result
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptCombineAttributesMode CombineMode =
		EGeometryScriptCombineAttributesMode::EnableAllMatching;

	UE_API void UpdateAttributesForCombineMode(UE::Geometry::FDynamicMesh3& Target, const UE::Geometry::FDynamicMesh3& Source);
};

/**
 * Options for merging vertex pairs
 */
USTRUCT(BlueprintType)
struct FGeometryScriptMergeVertexOptions
{
	GENERATED_BODY()
public:

	// Whether to restrict merges to boundary vertices
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOnlyBoundary = true;

	// Whether to allow the merge to introduce a non-boundary bowtie vertex (has no effect if bOnlyBoundary is true)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowNonBoundaryBowties = false;

};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshEdits"))
class UGeometryScriptLibrary_MeshBasicEditFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DiscardMeshAttributes( 
		UDynamicMesh* TargetMesh, 
		bool bDeferChangeNotifications = false );



	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetVertexPosition( 
		UDynamicMesh* TargetMesh, 
		int VertexID, 
		FVector NewPosition, 
		bool& bIsValidVertex, 
		bool bDeferChangeNotifications = false );

	/**
	 * Set all vertex positions in the TargetMesh to the specified Positions.
	 * @param PositionList new vertex Positions. Size must be less than or equal to the MaxVertexID of TargetMesh  (ie gaps are supported).
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetAllMeshVertexPositions(
		UDynamicMesh* TargetMesh,
		FGeometryScriptVectorList PositionList,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Adds a new vertex to the mesh and returns a new Vertex ID (NewVertexIndex).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddVertexToMesh( 
		UDynamicMesh* TargetMesh, 
		FVector NewPosition, 
		int& NewVertexIndex,
		bool bDeferChangeNotifications = false );

	/**
	* Adds a list of vertices to the mesh, and populates the NewIndicesList with the corresponding new Vertex IDs.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddVerticesToMesh( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptVectorList NewPositionsList, 
		FGeometryScriptIndexList& NewIndicesList,
		bool bDeferChangeNotifications = false );

	/**
	* Removes a vertex from the mesh as indicated by the VertexID.  
	* Should the delete fail, e.g. if the specified vertex was not a mesh element, the flag bWasVertexDeleted will be set to false. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteVertexFromMesh( 
		UDynamicMesh* TargetMesh, 
		int VertexID,
		bool& bWasVertexDeleted,
		bool bDeferChangeNotifications = false );

	/**
	* Removes a list of vertices from the mesh.  
	* On return, NumDeleted will contain the actual number of vertices removed.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteVerticesFromMesh( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptIndexList VertexList,
		int& NumDeleted,
		bool bDeferChangeNotifications = false );

	/**
	* Adds a triangle (Vertex ID triplet) to the mesh and updates New Triangle Index with the resulting Triangle ID.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddTriangleToMesh( 
		UDynamicMesh* TargetMesh, 
		FIntVector NewTriangle,
		int& NewTriangleIndex,
		int NewTriangleGroupID = 0,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Adds a list of triangles to the mesh and populates the New Indices List with the corresponding new Triangle IDs.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddTrianglesToMesh( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptTriangleList NewTrianglesList,
		FGeometryScriptIndexList& NewIndicesList,
		int NewTriangleGroupID = 0,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Removes a triangle from the mesh as indicated by the Triangle ID.
	* Should the delete fail, e.g. if the specified triangle was not a mesh element, the flag bWasTriangleDelete will be set to false. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteTriangleFromMesh( 
		UDynamicMesh* TargetMesh, 
		int TriangleID,
		bool& bWasTriangleDeleted,
		bool bDeferChangeNotifications = false );

	/**
	* Removes a list of triangles from the mesh. 
	* On return, NumDeleted will contain the actual number of triangles removed.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteTrianglesFromMesh( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Triangle ID List") FGeometryScriptIndexList TriangleList,
		int& NumDeleted,
		bool bDeferChangeNotifications = false );

	/**
	 * Removes specified triangles, identified by mesh selection, from the mesh.
	 * On return, NumDeleted will contain the actual number of triangles removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteSelectedTrianglesFromMesh( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelection Selection,
		int& NumDeleted,
		bool bDeferChangeNotifications = false );

	/**
	 * Attempt to merge together two vertices, and report whether they were merged.
	 * Note that some merges may be prevented because they would create non-manifold edges in the mesh, which are not supported.
	 * 
	 * @param TargetMesh Mesh in which to merge vertices 
	 * @param VertexKeep Vertex to keep after merge
	 * @param VertexDiscard Vertex to discard after merge
	 * @param Options Options for merge, controlling which merges should be permitted
	 * @param bSuccess Flag indicating whether the merge succeeded
	 * @param InterpParam The kept vertex is moved to interpolated position Lerp(VertexKeep Position, VertexDiscard Position, InterpParam)
	 * @param bDeferChangeNotifications Whether to defer change notifications after this operation
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	MergeMeshVertexPair(
		UDynamicMesh* TargetMesh,
		int32 VertexKeep,
		int32 VertexDiscard,
		FGeometryScriptMergeVertexOptions Options,
		bool& bSuccess,
		double InterpParam = .5,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Attempt to merge together vertices in one selection to their closest vertices in the second selection, within a distance threshold.
	 * Note that some merges may be prevented because they would create non-manifold edges in the mesh, which are not supported.
	 * 
	 * @param TargetMesh Mesh in which to merge vertices 
	 * @param SelectionKeep Selection of vertices to be merged with SelectionDiscard, treated as 'kept' vertices for purposes of interpolation.
	 * @param SelectionDiscard Selection of vertices to be merged with SelectionKeep. Note: If a vertex is in both selections, it will be treated as if it were only in SelectionKeep.
	 * @param Options Options for merge, controlling which merges should be permitted
	 * @param NumMerged Number of vertices merged
	 * @param InterpParam The kept vertex is moved to interpolated position Lerp(VertexKeep Position, VertexDiscard Position, InterpParam)
	 * @param DistanceThreshold Vertices further apart than this threshold will not be merged
	 * @param bDeferChangeNotifications Whether to defer change notifications after this operation
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	MergeMeshVerticesInSelections(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection SelectionKeep,
		FGeometryScriptMeshSelection SelectionDiscard,
		FGeometryScriptMergeVertexOptions Options,
		int& NumMerged,
		double InterpParam = .5,
		double DistanceThreshold = 1,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr
	);



	/**
	 * Apply Append Transform to Append Mesh and then add its geometry to the Target Mesh.
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendMesh( 
		UDynamicMesh* TargetMesh, 
		UDynamicMesh* AppendMesh, 
		FTransform AppendTransform, 
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Apply Append Transform to Append Mesh and then add its geometry to the Target Mesh.
	 * Also combines materials lists of the Target and Append meshes, and updates the output mesh materials to reference the combined list.
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendMeshWithMaterials(
		UDynamicMesh* TargetMesh,
		const TArray<UMaterialInterface*>& TargetMeshMaterialList,
		UDynamicMesh* AppendMesh,
		const TArray<UMaterialInterface*>& AppendMeshMaterialList,
		TArray<UMaterialInterface*>& ResultMeshMaterialList,
		FTransform AppendTransform,
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		bool bCompactAppendedMaterials = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * For each transform in AppendTransforms, apply the transform to AppendMesh and then add its geometry to the TargetMesh.
	 * @param ConstantTransform the Constant transform will be applied after each Append transform
	 * @param bConstantTransformIsRelative if true, the Constant transform is applied "in the frame" of the Append Transform, otherwise it is applied as a second transform in local coordinates (ie rotate around the AppendTransform X axis, vs around the local X axis)
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendMeshTransformed( 
		UDynamicMesh* TargetMesh, 
		UDynamicMesh* AppendMesh, 
		const TArray<FTransform>& AppendTransforms, 
		FTransform ConstantTransform,
		bool bConstantTransformIsRelative = true,
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * For each transform in AppendTransforms, apply the transform to AppendMesh and then add its geometry to the TargetMesh.
	 * Also combines materials lists of the Target and Append meshes, and updates the output mesh materials to reference the combined list.
	 * @param ConstantTransform the Constant transform will be applied after each Append transform
	 * @param bConstantTransformIsRelative if true, the Constant transform is applied "in the frame" of the Append Transform, otherwise it is applied as a second transform in local coordinates (ie rotate around the AppendTransform X axis, vs around the local X axis)
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendMeshTransformedWithMaterials( 
		UDynamicMesh* TargetMesh, 
		const TArray<UMaterialInterface*>& TargetMeshMaterialList,
		UDynamicMesh* AppendMesh, 
		const TArray<UMaterialInterface*>& AppendMeshMaterialList,
		TArray<UMaterialInterface*>& ResultMeshMaterialList,
		const TArray<FTransform>& AppendTransforms, 
		FTransform ConstantTransform,
		bool bConstantTransformIsRelative = true,
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		bool bCompactAppendedMaterials = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Repeatedly apply AppendTransform to the AppendMesh, each time adding the geometry to TargetMesh.
	 * @param RepeatCount number of times to repeat the transform-append cycle
	 * @param bApplyTransformToFirstInstance if true, the AppendTransform is applied before the first mesh append, otherwise it is applied after
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendMeshRepeated( 
		UDynamicMesh* TargetMesh, 
		UDynamicMesh* AppendMesh, 
		FTransform AppendTransform, 
		int RepeatCount = 1,
		bool bApplyTransformToFirstInstance = true,
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Repeatedly apply AppendTransform to the AppendMesh, each time adding the geometry to TargetMesh.
	 * Also combines materials lists of the Target and Append meshes, and updates the output mesh materials to reference the combined list.
	 * @param RepeatCount number of times to repeat the transform-append cycle
	 * @param bApplyTransformToFirstInstance if true, the AppendTransform is applied before the first mesh append, otherwise it is applied after
	 * @param AppendOptions Control how details like mesh attributes are handled when one mesh is appended to another
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendMeshRepeatedWithMaterials( 
		UDynamicMesh* TargetMesh, 
		const TArray<UMaterialInterface*>& TargetMeshMaterialList,
		UDynamicMesh* AppendMesh, 
		const TArray<UMaterialInterface*>& AppendMeshMaterialList,
		TArray<UMaterialInterface*>& ResultMeshMaterialList,
		FTransform AppendTransform, 
		int RepeatCount = 1,
		bool bApplyTransformToFirstInstance = true,
		bool bDeferChangeNotifications = false,
		FGeometryScriptAppendMeshOptions AppendOptions = FGeometryScriptAppendMeshOptions(),
		bool bCompactAppendedMaterials = true,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Adds a set of vertices/triangles to the mesh, with Normals, UVs, and Colors; returning the new triangles indices 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendBuffersToMesh( 
		UDynamicMesh* TargetMesh, 
		const FGeometryScriptSimpleMeshBuffers& Buffers,
		FGeometryScriptIndexList& NewTriangleIndicesList,
		int MaterialID = 0,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr );

};

#undef UE_API
