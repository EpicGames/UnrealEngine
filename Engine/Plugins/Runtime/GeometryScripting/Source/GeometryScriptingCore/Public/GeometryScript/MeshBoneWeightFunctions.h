// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryScriptSelectionTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "SkeletalMeshAttributes.h"
#include "MeshBoneWeightFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;
class USkeleton;


USTRUCT(BlueprintType, meta = (DisplayName = "Bone Weights"))
struct FGeometryScriptBoneWeight
{
	GENERATED_BODY()

	FGeometryScriptBoneWeight() = default;
	FGeometryScriptBoneWeight(int32 InBoneIndex, float InWeight) :
		BoneIndex(InBoneIndex), Weight(InWeight)
	{
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneWeights)
	int32 BoneIndex = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneWeights)
	float Weight = 0;
};


USTRUCT(BlueprintType, meta = (DisplayName = "Bone Weights Profile"))
struct FGeometryScriptBoneWeightProfile
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneWeights)
	FName ProfileName = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;

	FName GetProfileName() const { return ProfileName; }
};

UENUM(BlueprintType)
enum class EGeometryScriptPruneBoneWeightsAssignmentType : uint8
{
	RenormalizeRemaining = 0,	/** Remove the bone from the bone weights and renormalize the remaining weights. */
	ReassignToParent = 1,		/** Re-assign the removed bone's weight to the parent bone. */
};

USTRUCT(BlueprintType)
struct FGeometryScriptPruneBoneWeightsOptions
{
	GENERATED_BODY()

	/** Specifies how the weight of the removed bone from a vertex's bone weights list gets reassigned. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPruneBoneWeightsAssignmentType ReassignmentType = EGeometryScriptPruneBoneWeightsAssignmentType::RenormalizeRemaining;

	/** Ignore invalid bones. Otherwise, if invalid bones are given, the operation terminates with an error */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bIgnoredInvalidBones = true;
};

UENUM(BlueprintType)
enum class EGeometryScriptSmoothBoneWeightsType : uint8
{
	DirectDistance = 0,		/** Compute weighting by using Euclidean distance from bone to vertex */
	GeodesicVoxel = 1,		/** Compute weighting by using geodesic distance from bone to vertex */
};


USTRUCT(BlueprintType)
struct FGeometryScriptSmoothBoneWeightsOptions
{
	GENERATED_BODY()

	/** The type of algorithm to use for computing the bone weight for each vertex */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptSmoothBoneWeightsType DistanceWeighingType = EGeometryScriptSmoothBoneWeightsType::DirectDistance;

	/** How rigid the binding should be. Higher values result in a more rigid binding (greater influence by bones
	 *  closer to the vertex than those further away).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Stiffness = 0.2f;

	/** Maximum number of bones that contribute to each weight. Set to 1 for a completely rigid binding. Higher values
	 *  to have more distant bones make additional contributions to the deformation at each vertex. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 MaxInfluences = 5;

	/** The resolution to build the voxelized representation of the mesh, for computing geodesic distance. Higher values
	 *  result in greater fidelity and less chance of disconnected parts contributing, but slower rate of computation and
	 *  more memory usage.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(EditCondition="DistanceWeighingType==EGeometryScriptSmoothBoneWeightsType::GeodesicVoxel"))
	int32 VoxelResolution = 256;
};

UENUM(BlueprintType)
enum class ETransferBoneWeightsMethod : uint8
{
	/** For every vertex on the TargetMesh, find the closest point on the surface of the SourceMesh and transfer 
	 * bone weights from it. This is usually a point on the SourceMesh triangle where the bone weights are computed via 
	 * interpolation of the bone weights at the vertices of the triangle via barycentric coordinates.
	 */
	ClosestPointOnSurface = 0,

	/** For every vertex on the target mesh, find the closest point on the surface of the source mesh. If that point 
	 * is within the search radius (controlled via SearchPercentage), and their normals differ by less than the 
	 * NormalThreshold, then we directly copy the weights from the source point to the target mesh vertex 
	 * (same as the ClosestPointOnSurface method). For all the vertices we didn't copy the weights directly, 
	 * automatically compute the smooth weights. 
	 */
	InpaintWeights = 1
};

UENUM(BlueprintType)
enum class EOutputTargetMeshBones : uint8
{
	/** When transferring weights, the SourceMesh bone attriubtes will be copied over to the TargetMesh, replacing any 
	 * existing one, and transferred weights will be indexing the copied bone attributes.
	 */
	SourceBones = 0,

    /** When transferring weights, if the TargetMesh has bone attributes, then the transferred SourceMesh weights will be 
     * reindexed with respect to the TargetMesh bones. In cases where a transferred SourceMesh weight refers to a bone 
     * not present in the TargetMesh bone attributes, then that weight is simply skipped, and an error message with 
     * information about the bone will be printed to the Output Log. For best results, the TargetMesh bone attributes 
     * should be a superset of all the bones that are indexed by the transferred weights.
     */
	TargetBones = 1
};

USTRUCT(BlueprintType)
struct FGeometryScriptTransferBoneWeightsOptions
{
	GENERATED_BODY()
	
	/** The type of algorithm to use for transferring the bone weights. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	ETransferBoneWeightsMethod TransferMethod = ETransferBoneWeightsMethod::ClosestPointOnSurface;

	/** Chooses which bone attributes to use for transferring weights to the TargetMesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EOutputTargetMeshBones OutputTargetMeshBones = EOutputTargetMeshBones::SourceBones;
	
	/** The identifier for the source bone/skin weight profile used to transfer the weights from. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptBoneWeightProfile SourceProfile = FGeometryScriptBoneWeightProfile();

	/** The identifier for the source bone/skin weight profile used to transfer the weights to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptBoneWeightProfile TargetProfile = FGeometryScriptBoneWeightProfile();

	/** Defines the search radius as the RadiusPercentage * (input mesh bounding box diagonal). All points not within the search
	  * radius will be ignored. If negative, all points are considered. Only used in the InpaintWeights algorithm.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (UIMin = -1, UIMax = 2, ClampMin = -1, ClampMax = 2, EditCondition = "TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	double RadiusPercentage = -1;

	/** Maximum angle (in degrees) difference between the target and the source point normals to be considred a match. 
	 * If negative, normals are ignored. Only used in the InpaintWeights algorithm.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (UIMin = -1, UIMax = 180, ClampMin = -1, ClampMax = 180, EditCondition="TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	double NormalThreshold = -1;

	/** If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (EditCondition = "TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	bool LayeredMeshSupport = true;

	/** The number of optional post-processing smoothing iterations applied to the vertices without the match. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100, EditCondition = "TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	int32 NumSmoothingIterations = 0; 

	/** The strength of each post-processing smoothing iteration. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1, EditCondition = "TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	float SmoothingStrength = 0.0f;

    /** Optional weight attribute name where a non-zero value indicates that we want the skinning weights for the vertex to be computed automatically instead of it being copied over from the source mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "TransferMethod == ETransferBoneWeightsMethod::InpaintWeights"))
	FName InpaintMask;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Bone Info"))
struct FGeometryScriptBoneInfo
{
	GENERATED_BODY()

	/** Index of the bone in the skeletal hierarchy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	int Index = INDEX_NONE;

	/** Bone name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	FName Name = NAME_None;
	
	/** Parent bone index. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	int ParentIndex = INDEX_NONE;

	/** Local/bone space reference transform. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	FTransform LocalTransform = FTransform::Identity;

	/** Global/world space reference transform. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	FTransform WorldTransform = FTransform::Identity;

	/** Bone color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bone)
	FLinearColor Color = FLinearColor::White;
};


UENUM(BlueprintType)
enum class EBonesToCopyFromSource : uint8
{
	/** Copy all bones from the source mesh to the target, regardless of whether they're bound or not. 
	 */
	AllBones = 0,

	/** Keep only bones that are actually bound to the target mesh, including all parent bones up to the root. */
	OnlyBoundAndParents = 1,
	
	/** Keep only bones that are actually bound to the target mesh and the root bone. Any existing bones between
	 *  the two will not be copied. Bound bones will have their parent as either the root bone or another bound bone. */
	OnlyBoundAndRoot = 2
};


USTRUCT(BlueprintType)
struct FGeometryScriptCopyBonesFromMeshOptions
{
	GENERATED_BODY()
	
	/** If the target Dynamic mesh has bone weights and a skeleton, re-index (re-bind) target weight indices from the  
	 * target skeleton to the source skeleton.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool ReindexWeights = false;

	/** Specify which bones are copied from the source mesh to the target. */
	UPROPERTY(EditAnyWhere, BlueprintReadWrite, Category = Options)
	EBonesToCopyFromSource BonesToCopyFromSource = EBonesToCopyFromSource::AllBones;
};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_BoneWeights"))
class UGeometryScriptLibrary_MeshBoneWeightFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Check whether the TargetMesh has a per-vertex Bone/Skin Weight Attribute set
	 * @param bHasBoneWeights will be returned true if the requested bone weight profile exists
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	MeshHasBoneWeights( 
		UDynamicMesh* TargetMesh,
		bool& bHasBoneWeights,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile() );

	/**
	 * Create a new BoneWeights attribute on the TargetMesh, if it does not already exist. If it does exist, 
	 * and bReplaceExistingProfile is passed as true, the attribute will be removed and re-added, to reset it. 
	 * @param bProfileExisted will be returned true if the requested bone weight profile already existed
	 * @param bReplaceExistingProfile if true, if the Profile already exists, it is reset
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	MeshCreateBoneWeights( 
		UDynamicMesh* TargetMesh,
		bool& bProfileExisted,
		bool bReplaceExistingProfile = false,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile() );

	/**
	 * Copies all bone weights from a source profile onto a target profile, on the same mesh, replacing all 
	 * weights that existed on the target profile. If either the source or the target profile didn't exist,
	 * then bProfileExisted will be set to false and no weights are copied.
	 * @param bProfileExisted will be returned true if both of the requested bone weight profiles exist
	 * @param TargetProfile The skin weight profile to copy to.
	 * @param SourceProfile The skin weight profile to copy from.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	MeshCopyBoneWeights( 
		UDynamicMesh* TargetMesh,
		bool& bProfileExisted,
		FGeometryScriptBoneWeightProfile TargetProfile,
		FGeometryScriptBoneWeightProfile SourceProfile = FGeometryScriptBoneWeightProfile()
		);
	
	/**
	 * Determine the largest bone weight index that exists on the Mesh
	 * @param bHasBoneWeights will be returned true if the requested bone weight profile exists
	 * @param MaxBoneIndex maximum bone index will be returned here, or -1 if no bone indices exist
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMaxBoneWeightIndex( 
		UDynamicMesh* TargetMesh,
		bool& bHasBoneWeights,
		int& MaxBoneIndex,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile() );

	/**
	 * Return an array of Bone/Skin Weights at a given vertex of TargetMesh
	 * @param VertexID requested vertex
	 * @param BoneWeights output array of bone index/weight pairs for the given Vertex
	 * @param bHasValidBoneWeights will be returned as true if the vertex has bone weights in the given profile, ie BoneWeights is valid
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetVertexBoneWeights( 
		UDynamicMesh* TargetMesh,
		int VertexID,
		TArray<FGeometryScriptBoneWeight>& BoneWeights,
		bool& bHasValidBoneWeights,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile() );

	/**
	 * Return the Bone/Skin Weight with the maximum weight at a given vertex of TargetMesh
	 * @param VertexID requested vertex
	 * @param BoneWeight the bone index and weight that was found to have the maximum weight
	 * @param bHasValidBoneWeights will be returned as true if the vertex has bone weights in the given profile and a largest weight was found
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetLargestVertexBoneWeight( 
		UDynamicMesh* TargetMesh,
		int VertexID,
		FGeometryScriptBoneWeight& BoneWeight,
		bool& bHasValidBoneWeights,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile() );

	/**
	 * Set the Bone/Skin Weights at a given vertex of TargetMesh
	 * @param VertexID vertex to update
	 * @param BoneWeights input array of bone index/weight pairs for the Vertex
	 * @param bIsValidVertexID will be returned as true if the vertex ID is valid
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetVertexBoneWeights( 
		UDynamicMesh* TargetMesh,
		int VertexID,
		const TArray<FGeometryScriptBoneWeight>& BoneWeights,
		bool& bIsValidVertexID,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Blends two bone weights using an Alpha value that ranges from 0 to 1, inclusive. If Alpha is 0, then only weights from BoneWeightsA are used,
	 * and if Alpha is 1, then only weights from BoneWeightsB are used. For any value in between the weights are linearly interpolated.
	 * Each bone weight from either array, that has the same bone index, are linearly interpolated. Any bone weights that are missing from either
	 * BoneWeightsA or BoneWeightsB, are assumed to exist and have a weight of 0. After blending, the result is renormalized and sorted.
	 * Values that are below the influence threshold, or exceeding the default bone weight limit (currently set to 12) will be thrown away.
	 * @param BoneWeightsA List of bone weights to blend, such that its influence is greatest when Alpha is 0 and smallest when Alpha is 1.
	 * @param BoneWeightsB List of bone weights to blend, such that its influence is greatest when Alpha is 1 and smallest when Alpha is 0.
	 * @param Alpha The blending factor, ranging from 0 to 1, inclusive. Values outside of this range are clamped.
	 * @param Result The resulting blend of the two bone weight arrays. 
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries|BoneWeights", meta = (HidePin = "Debug"))
	static UE_API void
	BlendBoneWeights(
		const TArray<FGeometryScriptBoneWeight>& BoneWeightsA,
		const TArray<FGeometryScriptBoneWeight>& BoneWeightsB,
		float Alpha,
		TArray<FGeometryScriptBoneWeight>& Result,
		UGeometryScriptDebug* Debug = nullptr
	);
	
	/**
	 * Set all vertices of the TargetMesh to the given Bone/Skin Weights
	 * @param BoneWeights input array of bone index/weight pairs for the Vertex
	 * @param Profile identifier for the bone/skin weight profile
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetAllVertexBoneWeights( 
		UDynamicMesh* TargetMesh,
		const TArray<FGeometryScriptBoneWeight>& BoneWeights,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile(),
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 *  Prunes the given bones from any bone weight assignment on the given profile. 
	 *  The bone weights are re-assigned based on the type of re-assignment specified in the options,
	 *  although in the case where the bone(s) being pruned are the sole bone weight on a vertex, then
	 *  the parent bone will be assigned as the sole bone weight for that vertex.
	 *  Bones are pruned iteratively from leaf to root, to ensure that weighs are progressively re-assigned
	 *  in case multiple bones along the same branch are being pruned.
	 *  @param BonesToPrune The list of bones to remove.
	 *  @param Options The options to set for the pruning algorithm.
	 *  @param Profile The skin weight profile to prune the bones from.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	PruneBoneWeights( 
		UDynamicMesh* TargetMesh,
		const TArray<FName>& BonesToPrune,
		FGeometryScriptPruneBoneWeightsOptions Options,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile(),
		UGeometryScriptDebug* Debug = nullptr);
	
	
	/** 
	 *  Computes a smooth skin binding for the given mesh to the skeleton provided.
	 *  @param Skeleton The skeleton to compute binding for the skin weights.
	 *  @param Options The options to set for the binding algorithm.
	 *  @param Profile The skin weight profile to update with the smooth binding.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ComputeSmoothBoneWeights(
		UDynamicMesh* TargetMesh, 
		USkeleton* Skeleton,
		FGeometryScriptSmoothBoneWeightsOptions Options,
		FGeometryScriptBoneWeightProfile Profile = FGeometryScriptBoneWeightProfile(),
		UGeometryScriptDebug* Debug = nullptr);
	
	/** 
	 * Transfer the bone weights from the SourceMesh to the TargetMesh. Assumes that the meshes are aligned. Otherwise, 
	 * use the TransformMesh geometry script function to align them.
	 * 
	 * @param SourceMesh The mesh we are transferring the weights from.
	 * @param TargetMesh The mesh we are transferring the weights to.
	 * @param Options The options to set for the transfer weight algorithm.
	 * @param Selection Optional subset of target mesh vertices to transfer weights to.
	 * If left empty, skin weights will be transferred to all target mesh vertices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	TransferBoneWeightsFromMesh(
		UDynamicMesh* SourceMesh,
		UDynamicMesh* TargetMesh,
		FGeometryScriptTransferBoneWeightsOptions Options = FGeometryScriptTransferBoneWeightsOptions(),
		FGeometryScriptMeshSelection Selection = FGeometryScriptMeshSelection(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Copy the bone attributes (skeleton) from the SourceMesh to the TargetMesh.
	 * @param SourceMesh Mesh we are copying the bone attributes from.
	 * @param TargetMesh Mesh we are copying the bone attributes to.
	 * @param Options An option object to control how the copying is performed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CopyBonesFromMesh(
		UDynamicMesh* SourceMesh,
		UDynamicMesh* TargetMesh,
		FGeometryScriptCopyBonesFromMeshOptions Options = FGeometryScriptCopyBonesFromMeshOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Copy the bone attributes (skeleton) from the SourceSkeleton to the TargetMesh.
	 * @param SourceSkeleton The skeleton asset we are copying the bone attributes from.
	 * @param TargetMesh Mesh we are copying the bone attributes to.
	 * @param Options An option object to control how the copying is performed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CopyBonesFromSkeleton(
		USkeleton* SourceSkeleton,
		UDynamicMesh* TargetMesh,
		FGeometryScriptCopyBonesFromMeshOptions Options = FGeometryScriptCopyBonesFromMeshOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Discard the bone attributes (skeleton) from the TargetMesh.
	 * @param TargetMesh Mesh we are discarding bone attributes from.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DiscardBonesFromMesh(
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the index of the bone with the given name.
	 * @param TargetMesh Mesh containing the bone attributes.
	 * @param BoneName The name of the bone whose index we are getting.
	 * @param bIsValidBoneName Set to true if the TargetMesh contains a bone with the given name, false otherwise.
	 * @param BoneIndex The index of the bone with the given name. Will be set to -1 if bIsValidBoneName is false.
	 */
	UE_DEPRECATED(5.3, "Use GetBoneInfo instead.")
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetBoneIndex(
		UDynamicMesh* TargetMesh,
		FName BoneName,
		bool& bIsValidBoneName,
		int& BoneIndex,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the name of the root bone.
	 * 
	 * @param TargetMesh Mesh containing the bone attributes.
	 * @param BoneName The name of the root bone.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetRootBoneName(
		UDynamicMesh* TargetMesh,
		FName& BoneName,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the information about the children of the bone.
	 * 
	 * @param TargetMesh Mesh containing the bone attributes.
	 * @param BoneName The name of bone.
	 * @param bRecursive If set to true, grandchildren will also be added recursively
	 * @param bIsValidBoneName Set to true if the TargetMesh contains a bone with the given name, false otherwise.
	 * @param ChildrenInfo The informattion of the children.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetBoneChildren(
		UDynamicMesh* TargetMesh,
		FName BoneName,
		bool bRecursive,
		bool& bIsValidBoneName,
		TArray<FGeometryScriptBoneInfo>& ChildrenInfo,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the bone information.
	 * 
	 * @param TargetMesh Mesh containing the bone attributes.
	 * @param BoneName The name of bone.
	 * @param bIsValidBoneName Set to true if the TargetMesh contains a bone with the given name, false otherwise.
	 * @param BoneInfo The information about the bone.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetBoneInfo(
		UDynamicMesh* TargetMesh,
		FName BoneName,
		bool& bIsValidBoneName,
		FGeometryScriptBoneInfo& BoneInfo,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get an array of bones representing the skeleton. Each entry contains information about the bone.
	 * 
	 * @param TargetMesh Mesh containing the bone attributes.
	 * @param BonesInfo Skeleton information.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries|BoneWeights", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllBonesInfo(
		UDynamicMesh* TargetMesh,
		TArray<FGeometryScriptBoneInfo>& BonesInfo,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
