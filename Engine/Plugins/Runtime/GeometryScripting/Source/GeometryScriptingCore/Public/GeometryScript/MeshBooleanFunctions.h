// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshBooleanFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API



UENUM(BlueprintType)
enum class EGeometryScriptBooleanOperation : uint8
{
	Union,
	Intersection,
	Subtract,
	TrimInside,
	TrimOutside,
	NewPolyGroupInside UMETA(DisplayName = "New PolyGroup Inside"),
	NewPolyGroupOutside UMETA(DisplayName = "New PolyGroup Outside")
};


// Options for the output coordinate space for the mesh boolean result
UENUM(BlueprintType)
enum class EGeometryScriptBooleanOutputSpace : uint8
{
	// Transform the boolean result into the local space of the target mesh
	TargetTransformSpace,
	// Transform the boolean result into the local space of the tool mesh
	ToolTransformSpace,
	// Keep the boolean result in the shared space where the boolean was computed
	SharedTransformSpace
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshBooleanOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillHoles = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSimplifyOutput = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SimplifyPlanarTolerance = 0.01f;

	// Whether to allow the Mesh Boolean operation to generate an empty mesh as its result
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowEmptyResult = false;

	// The coordinate space to use for the result mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptBooleanOutputSpace OutputTransformSpace = EGeometryScriptBooleanOutputSpace::TargetTransformSpace;
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshSelfUnionOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillHoles = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bTrimFlaps = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSimplifyOutput = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SimplifyPlanarTolerance = 0.01f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float WindingThreshold = 0.5f;
};



USTRUCT(BlueprintType)
struct FGeometryScriptMeshPlaneCutOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillHoles = true;

	// If non-negative and bFillHoles is true, specify the material ID to set on hole fill triangles
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 HoleFillMaterialID = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillSpans = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFlipCutSide = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVWorldDimension = 1.0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshPlaneSliceOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillHoles = true;

	// If non-negative and bFillHoles is true, specify the material ID to set on hole fill triangles
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 HoleFillMaterialID = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillSpans = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float GapWidth = 0.01f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVWorldDimension = 1.0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshMirrorOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bApplyPlaneCut = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFlipCutSide = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWeldAlongPlane = true;
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshIsoCurveOptions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float CurveIsoValueSnapTolerance = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (Units = "cm"))
	float SnapToExistingVertexTolerance = UE_KINDA_SMALL_NUMBER;

};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshBooleans"))
class UGeometryScriptLibrary_MeshBooleanFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Applies a Boolean operation (such as, Union, Intersect, and Subtract) to the Target Dynamic Mesh based on a Tool Dynamic Mesh.
	* @param TargetMesh  Dynamic Mesh to be acted upon
	* @param TargetTransform  used to position the TargetMesh
	* @param ToolMesh  Dynamic Mesh that acts as the cutting tool
	* @param ToolTransform used to position the ToolMesh
	* @param Operation selects the specific boolean operation
	* @param Options selects additional options that are applied to the result
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshBoolean( 
		UDynamicMesh* TargetMesh, 
		FTransform TargetTransform,
		UDynamicMesh* ToolMesh, 
		FTransform ToolTransform,
		EGeometryScriptBooleanOperation Operation,
		FGeometryScriptMeshBooleanOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Mesh-Boolean-Union an object with itself to repair self-intersections, remove floating geometry, etc.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshSelfUnion( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelfUnionOptions Options,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Applies a plane cut to a mesh, optionally filling any holes created.
	* @param TargetMesh Dynamic Mesh to updated with the cut
	* @param CutFrame defines the plane used to cut the TargetMesh
	* @param Options selects additional clean-up operations performed after the cut.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshPlaneCut( 
		UDynamicMesh* TargetMesh, 
		FTransform CutFrame,
		FGeometryScriptMeshPlaneCutOptions Options,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	 * Slices a mesh into two halves, with optional hole filling.
	 * @param TargetMesh Dynamic Mesh to be updated by the slice.
	 * @param CutFrame defines the plane used to slice the TargetMesh.
	 * @param Options selects additional clean-up operations performed after the cut.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshPlaneSlice( 
		UDynamicMesh* TargetMesh, 
		FTransform CutFrame,
		FGeometryScriptMeshPlaneSliceOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Mirrors a mesh across a plane, with optional cutting and welding of triangles.
	* @param TargetMesh Dynamic Mesh to be updated by the mirror operation.
	* @param MirrorFrame defines the plane used to mirror the TargetMesh.
	* @param Options selects  additional clean-up operations performed after the mirror.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshMirror( 
		UDynamicMesh* TargetMesh, 
		FTransform MirrorFrame,
		FGeometryScriptMeshMirrorOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Insert edges along the curve where the interpolated per-vertex values match a given isovalue
	*
	* @param TargetMesh Dynamic Mesh to be updated.
	* @param Options Settings for processing to perform after inserting the curve edges
	* @param PerVertexValue Values to interpolate, must be one per vertex
	* @param IsoValue Value at which to insert the new curve
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Booleans", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyMeshIsoCurves(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshIsoCurveOptions Options,
		const FGeometryScriptScalarList& PerVertexValue,
		float IsoValue = .5,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
