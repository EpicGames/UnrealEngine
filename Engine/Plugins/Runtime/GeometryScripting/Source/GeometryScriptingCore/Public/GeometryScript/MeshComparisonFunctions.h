// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshComparisonFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


USTRUCT(BlueprintType)
struct FGeometryScriptIsSameMeshOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckConnectivity = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckEdgeIDs = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckNormals = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckColors = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckUVs = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckGroups = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCheckAttributes = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Epsilon = 1e-06f;
};



USTRUCT(BlueprintType)
struct FGeometryScriptMeasureMeshDistanceOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSymmetric = true;
};

UENUM(BlueprintType)
enum class EGeometryScriptMeshDifferenceReason : uint8
{
	Unknown = 0,
	VertexCount,
	TriangleCount,
	EdgeCount,
	Vertex,
	Triangle,
	Edge,
	Connectivity,
	Normal,
	Color,
	UV,
	Group,
	Attribute
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshDifferenceInfo
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Difference)
	EGeometryScriptMeshDifferenceReason Reason = EGeometryScriptMeshDifferenceReason::Unknown;

	// String that may contain additional detail on the difference
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Difference)
	FString Detail;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Difference)
	int32 TargetMeshElementID = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Difference)
	int32 OtherMeshElementID = INDEX_NONE;

	// Indicates the type of element that TargetMeshElementID and OtherMeshElementID reference
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Difference)
	EGeometryScriptIndexType ElementIDType = EGeometryScriptIndexType::Any;
};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshComparison"))
class UGeometryScriptLibrary_MeshComparisonFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Returns true if the two input meshes are equivalent under the comparisons defined by the input options. If false, DifferenceInfo provides info on the first difference found.
	 * @param DifferenceInfo If the meshes are different, provides info on the first difference found.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Comparison", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	IsSameMeshAs(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* OtherMesh,
		FGeometryScriptIsSameMeshOptions Options,
		bool &bIsSameMesh,
		FGeometryScriptMeshDifferenceInfo& DifferenceInfo,
		UGeometryScriptDebug* Debug = nullptr);

	// Non-blueprint overload of IsSameMeshAs, without DifferenceInfo, for C++ API backwards compatibilty
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	IsSameMeshAs(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* OtherMesh,
		FGeometryScriptIsSameMeshOptions Options,
		bool& bIsSameMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Measures the min/max and average closest-point distances between two meshes.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Comparison", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	MeasureDistancesBetweenMeshes(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* OtherMesh,
		FGeometryScriptMeasureMeshDistanceOptions Options,
		double& MaxDistance,
		double& MinDistance,
		double& AverageDistance,
		double& RootMeanSqrDeviation,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Returns true if the two input meshes (with optional transforms) are geometrically intersecting.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Comparison", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	IsIntersectingMesh(
		UDynamicMesh* TargetMesh,
		FTransform TargetTransform,
		UDynamicMesh* OtherMesh,
		FTransform OtherTransform,
		bool &bIsIntersecting,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
