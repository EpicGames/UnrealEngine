// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet/KismetArrayLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshDecompositionFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshDecomposition"))
class UGeometryScriptLibrary_MeshDecompositionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create a new Mesh for each Connected Island (Component) of TargetMesh.
	 * New meshes are drawn from MeshPool if it is provided, otherwise new UDynamicMesh instances are allocated
	 * @param ComponentMeshes New List of meshes is returned here
	 * @param MeshPool New meshes in ComponentMeshes output list are allocated from this pool if it is provided (highly recommended!!)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta = (ScriptMethod, HidePin = "Debug", 
		DisplayName = "Split Mesh by Connected Islands", Keywords = "Connected Components Parts"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SplitMeshByComponents(  
		UDynamicMesh* TargetMesh, 
		TArray<UDynamicMesh*>& ComponentMeshes,
		UDynamicMeshPool* MeshPool,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create a new Mesh for each vertex-connected or vertex-overlapping part of TargetMesh.
	 * New meshes are drawn from MeshPool if it is provided, otherwise new UDynamicMesh instances are allocated
	 * @param ComponentMeshes New List of meshes is returned here
	 * @param MeshPool New meshes in ComponentMeshes output list are allocated from this pool if it is provided (highly recommended!!)
	 * @param ConnectVerticesThreshold Vertices closer than this distance will be classified as part of the same component, even if they aren't connected by the mesh triangulation
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta = (ScriptMethod, HidePin = "Debug", Keywords = "Islands Connected Components Parts"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SplitMeshByVertexOverlap(
		UDynamicMesh* TargetMesh,
		TArray<UDynamicMesh*>& ComponentMeshes,
		UDynamicMeshPool* MeshPool,
		double ConnectVerticesThreshold = .001,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create a new Mesh for each MaterialID of TargetMesh.
	 * New meshes are drawn from MeshPool if it is provided, otherwise new UDynamicMesh instances are allocated
	 * @param ComponentMeshes New List of meshes is returned here
	 * @param ComponentMaterialIDs MaterialID for each Mesh in ComponentMeshes is returned here
	 * @param MeshPool New meshes in ComponentMeshes output list are allocated from this pool if it is provided (highly recommended!!)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SplitMeshByMaterialIDs(  
		UDynamicMesh* TargetMesh, 
		TArray<UDynamicMesh*>& ComponentMeshes,
		TArray<int>& ComponentMaterialIDs,
		UDynamicMeshPool* MeshPool,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create a new Mesh for each Polygroup of TargetMesh. Note that this may be a *large* number of meshes!
	 * New meshes are drawn from MeshPool if it is provided, otherwise new UDynamicMesh instances are allocated
	 * @param ComponentMeshes New List of meshes is returned here
	 * @param ComponentPolygroups Original Polygroup for each Mesh in ComponentMeshes is returned here
	 * @param MeshPool New meshes in ComponentMeshes output list are allocated from this pool if it is provided (highly recommended!!)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Split Mesh By PolyGroups"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SplitMeshByPolygroups(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		TArray<UDynamicMesh*>& ComponentMeshes,
		TArray<int>& ComponentPolygroups,
		UDynamicMeshPool* MeshPool,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Sort meshes by their volume
	 * 
	 * Note: For meshes with open boundary, volume is computed with respect to the average vertex position.
	 * 
	 * @param Meshes The meshes to sort
	 * @param bStableSort Whether to preserve ordering for meshes with the same volume
	 * @param SortOrder Whether to sort in order of increasing or decreasing volume
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition")
	static UE_API void
	SortMeshesByVolume(
		UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
		bool bStableSort = false,
		EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sort meshes by their surface area
	 *
	 * @param Meshes The meshes to sort
	 * @param bStableSort Whether to preserve ordering for meshes with the same surface area
	 * @param SortOrder Whether to sort in order of increasing or decreasing surface area
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition")
	static UE_API void
	SortMeshesByArea(
		UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
		bool bStableSort = false,
		EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sort meshes by their axis-aligned bounding box volume
	 *
	 * @param Meshes The meshes to sort
	 * @param bStableSort Whether to preserve ordering for meshes with the same bounds volume
	 * @param SortOrder Whether to sort in order of increasing or decreasing bounds volume
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition")
	static UE_API void
	SortMeshesByBoundsVolume(
		UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
		bool bStableSort = false,
		EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * Sort meshes according to the values in a second array, which must have the same length as the Meshes array
	 * For example, if the values array is [3, 2, 1], with Ascending Sort Order, the Meshes array would be reversed
	 * 
	 * @param Meshes The meshes to sort
	 * @param ValuesToSortBy The values to reference for sort ordering, which must be one-to-one with the meshes
	 * @param bStableSort Whether to preserve ordering for meshes with the same corresponding value
	 * @param SortOrder Whether to sort in order of increasing or decreasing value
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition")
	static UE_API void
	SortMeshesByCustomValues(
		UPARAM(Ref) TArray<UDynamicMesh*>& Meshes,
		const TArray<float>& ValuesToSortBy,
		bool bStableSort = false,
		EArraySortOrder SortOrder = EArraySortOrder::Ascending);

	/**
	 * CopyMeshSelectionToMesh should be used instead of this function
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetSubMeshFromMesh(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Copy To Submesh", ref) UDynamicMesh* StoreToSubmesh, 
		UPARAM(DisplayName = "Triangle ID List") FGeometryScriptIndexList TriangleList,
		UPARAM(DisplayName = "Copy To Submesh") UDynamicMesh*& StoreToSubmeshOut, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Extract the triangles identified by Selection from TargetMesh and copy/add them to StoreToSubmesh
	 * @param bAppendToExisting if false (default), StoreToSubmesh is cleared, otherwise selected triangles are appended
	 * @param bPreserveGroupIDs if true, GroupIDs of triangles on TargetMesh are preserved in StoreToSubmesh. Otherwise new GroupIDs are allocated.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CopyMeshSelectionToMesh(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Copy To Submesh", ref) UDynamicMesh* StoreToSubmesh, 
		FGeometryScriptMeshSelection Selection,
		UPARAM(DisplayName = "Copy To Submesh") UDynamicMesh*& StoreToSubmeshOut, 
		bool bAppendToExisting = false,
		bool bPreserveGroupIDs = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Set CopyToMesh to be the same mesh as CopyFromMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Decomposition", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Copy From Mesh") UDynamicMesh* 
	CopyMeshToMesh(  
		UDynamicMesh* CopyFromMesh, 
		UPARAM(DisplayName = "Copy To Mesh", ref) UDynamicMesh* CopyToMesh, 
		UPARAM(DisplayName = "Copy To Mesh") UDynamicMesh*& CopyToMeshOut, 
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
