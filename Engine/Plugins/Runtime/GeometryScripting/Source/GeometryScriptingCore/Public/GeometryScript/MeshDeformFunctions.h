// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshDeformFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


USTRUCT(BlueprintType)
struct FGeometryScriptBendWarpOptions
{
	GENERATED_BODY()
public:
	/** Symmetric extents are [-BendExtent,BendExtent], if disabled, then [-LowerExtent,BendExtent] is used  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSymmetricExtents = true;

	/** Lower extent used when bSymmetricExtents = false */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "bSymmetricExtents == false"))
	float LowerExtent = 10;

	/** If true, the Bend is "centered" at the Origin, ie the regions on either side of the extents are rigidly transformed. If false, the Bend begins at the start of the Lower Extents, and the "lower" region is not affected. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bBidirectional = true;
};



USTRUCT(BlueprintType)
struct FGeometryScriptTwistWarpOptions
{
	GENERATED_BODY()
public:
	/** Symmetric extents are [-BendExtent,BendExtent], if disabled, then [-LowerExtent,BendExtent] is used  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSymmetricExtents = true;

	/** Lower extent used when bSymmetricExtents = false */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "bSymmetricExtents == false"))
	float LowerExtent = 10;

	/** If true, the Twist is "centered" at the Origin, ie the regions on either side of the extents are rigidly transformed. If false, the Twist begins at the start of the Lower Extents, and the "lower" region is not affected. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bBidirectional = true;
};

UENUM(BlueprintType)
enum class EGeometryScriptFlareType : uint8
{
	//Displaced by sin(pi x) with x in 0 to 1
	SinMode = 0,

	//Displaced by sin(pi x)*sin(pi x) with x in 0 to 1. This provides a smooth normal transition.
	SinSquaredMode = 1,

	// Displaced by piecewise-linear trianglular mode
	TriangleMode = 2
};

USTRUCT(BlueprintType)
struct FGeometryScriptFlareWarpOptions
{
	GENERATED_BODY()
public:
	/** Symmetric extents are [-BendExtent,BendExtent], if disabled, then [-LowerExtent,BendExtent] is used  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSymmetricExtents = true;

	/** Lower extent used when bSymmetricExtents = false */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "bSymmetricExtents == false"))
	float LowerExtent = 10;

	/** Determines the profile used as a displacement */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptFlareType FlareType = EGeometryScriptFlareType::SinMode;
};



USTRUCT(BlueprintType)
struct FGeometryScriptPerlinNoiseLayerOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Magnitude = 5.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Frequency = 0.25;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector FrequencyShift = FVector::Zero();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int RandomSeed = 0;
};



UENUM(BlueprintType)
enum class EGeometryScriptMathWarpType : uint8
{
	SinWave1D = 0,
	SinWave2D = 1,
	SinWave3D = 2
};


USTRUCT(BlueprintType)
struct FGeometryScriptMathWarpOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Magnitude = 5.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Frequency = 0.25f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float FrequencyShift = 0.0;
};


UENUM(BlueprintType)
enum class EGeometryScriptPerVertexNormalSource : uint8
{
	// Compute new per-vertex normals from the mesh geometry
	Computed,
	// Average existing overlay normals to compute per-vertex normals
	AverageFromOverlay
};


USTRUCT(BlueprintType)
struct FGeometryScriptPerlinNoiseOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptPerlinNoiseLayerOptions BaseLayer;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bApplyAlongNormal = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta=(EditCondition="bApplyAlongNormal"))
	EGeometryScriptPerVertexNormalSource NormalSource = EGeometryScriptPerVertexNormalSource::Computed;

	/** EmptyBehavior Defines how an empty input selection should be interpreted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptEmptySelectionBehavior EmptyBehavior = EGeometryScriptEmptySelectionBehavior::FullMeshSelection;
};


USTRUCT(BlueprintType)
struct FGeometryScriptIterativeMeshSmoothingOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int NumIterations = 10;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Alpha = 0.2f;

	/** EmptyBehavior Defines how an empty input selection should be interpreted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptEmptySelectionBehavior EmptyBehavior = EGeometryScriptEmptySelectionBehavior::FullMeshSelection;
};




USTRUCT(BlueprintType)
struct FGeometryScriptDisplaceFromTextureOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Magnitude = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UVScale = FVector2D(1,1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UVOffset = FVector2D(0,0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Center = 0.5;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int ImageChannel = 0;

	/** EmptyBehavior Defines how an empty input selection should be interpreted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptEmptySelectionBehavior EmptyBehavior = EGeometryScriptEmptySelectionBehavior::FullMeshSelection;
};

UENUM(BlueprintType)
enum class EGeometryScriptTessellationErrorMode : uint8
{
	/**
	* Target error specified in scene-units.
	*/
	Absolute,

	/**
	* Scale-invariant tessellation detail, relative to object bounds.
	*/
	Relative
};

USTRUCT(BlueprintType)
struct FGeometryScriptAdaptiveTessellationOptions
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableTessellation = false;

	/**
	 * Primary parameter to controlling tessellation detail when using adaptive tessellation (lower values produce finer meshes). When mode is absolute, correspond to world units error.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ErrorTolerance = 5.f;

	/**
	 * Control whether error criterion is related to world units (absolute) or object bounds (relative).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptTessellationErrorMode ErrorMode = EGeometryScriptTessellationErrorMode::Relative;

	/**
	 * Value between 0 and 1 to control the amount of curvature sensitive refinement for approximation of high-frequency details. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float FeatureSensitivity = 0.5f;

	/**
	 * Whether to apply some mesh regularization as a post-process, to counter stretching or skinny triangles.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bMeshOptimization = true;

	/**
	 *  Value between 0 and 1 to control how coplanar faces adjacent to an edge need to be to allow edge flips. This should rarely need to be modified. 
	 */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Coplanarity = 0.8f;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshDeformers"))
class UGeometryScriptLibrary_MeshDeformFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Applies a Bend Warp around an axis defined by the Bend Orientation transform.  
	* The extents of the affected region can be controlled by the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyBendWarpToMesh(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptBendWarpOptions Options,
		FTransform BendOrientation,
		float BendAngle = 45,
		float BendExtent = 50,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Applies a twist warp around an axis defined by the Twist Orientation transform.
	 * The extents of the affected region can be controlled by the Options.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyTwistWarpToMesh(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptTwistWarpOptions Options,
		FTransform TwistOrientation,
		float TwistAngle = 45,
		float TwistExtent = 50,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Applies a Flare/Bulge warp around an axis defined by the Flare Orientation transform.
	* The amount of flare in the perpendicular directions can be controlled by FlarePercentX and FlarePercentY
	* and the extents of the affected region can be controlled by the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyFlareWarpToMesh(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptFlareWarpOptions Options,
		FTransform FlareOrientation,
		float FlarePercentX = 0,
		float FlarePercentY = 0,
		float FlareExtent = 50,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Applies various simple math-function-based warps around an axis defined by the Warp Orientation transform, 
	* currently a 1D or 2D sine-wave with arbitrary orientation may be selected by WarpType.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMathWarpToMesh(  
		UDynamicMesh* TargetMesh, 
		FTransform WarpOrientation,
		EGeometryScriptMathWarpType WarpType,
		FGeometryScriptMathWarpOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Applies 3D Perlin noise displacement to the Target Mesh.
	* @param Selection if non-empty, only the vertices identified by the selection will be displaced, otherwise
	* the Options' EmptyBehavior will be followed.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta = (ScriptMethod, DisplayName = "Apply Perlin Noise to Mesh", HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyPerlinNoiseToMesh2(
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptPerlinNoiseOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute Perlin Noise at the requested positions
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Noise Values") FGeometryScriptScalarList
	ComputePerlinNoise(FGeometryScriptVectorList Positions, FGeometryScriptPerlinNoiseOptions Options);

	/**
	* Applies a number of iterations of mesh smoothing to a Dynamic Mesh.
	* @param Selection if non-empty, only the vertices identified by the selection will be subject to smoothing,  otherwise
	* the Options' EmptyBehavior will be followed.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyIterativeSmoothingToMesh(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptIterativeMeshSmoothingOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Applies a displacement to a Dynamic Mesh based on a Texture2D and a UV Channel.
	 * @param Selection if non-empty, only the vertices identified by the selection will be subject to displacement,  otherwise
	 * the Options' EmptyBehavior will be followed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyDisplaceFromTextureMap(  
		UDynamicMesh* TargetMesh, 
		UTexture2D* Texture,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptDisplaceFromTextureOptions Options,
		FGeometryScriptAdaptiveTessellationOptions TessellationOptions,
		UPARAM(DisplayName = "UV Channel") int32 UVLayer = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Add the vectors in VectorList, scaled by Magnitude, to the vertex positions in Target Mesh.
	 * VectorList Length must be >= the MaxVertexID of the Target Mesh.
	 * @param Selection if non-empty, only the vertices identified by the selection will be displaced. The VectorList must still be the same size as the whole mesh, this is just a filter on which vertices are updated.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Deformations", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyDisplaceFromPerVertexVectors(  
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		const FGeometryScriptVectorList& VectorList, 
		float Magnitude = 5.0,
		UGeometryScriptDebug* Debug = nullptr);

	//---------------------------------------------
	// Backwards-Compatibility implementations
	//---------------------------------------------
	// Deprecated versions of functions are kept below
	// 

	/**
	* Applies 3D Perlin noise displacement to the Target Mesh, with Frequency parameter incorrectly squared.
	* @param Selection if non-empty, only the vertices identified by the selection will be displaced, otherwise
	* the Options' EmptyBehavior will be followed.
	* @warning It is strongly recommended that callers of this function update to the current ApplyPerlinNoiseToMesh function!
	*/
	UE_DEPRECATED(5.7, "Use 'ApplyPerlinNoiseToMesh2' instead. This method is deprecated because it incorrectly squared the frequency parameter.")
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Compatibility", 
		meta = (ScriptMethod, DeprecatedFunction, HidePin = "Debug",
			DeprecationMessage = "Use 'Apply Perlin Noise to Mesh' instead. This node is deprecated because it incorrectly squared the frequency parameter.", 
			DisplayName="Apply Perlin Noise to Mesh Compatibility 5.6 (Deprecated)"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyPerlinNoiseToMesh(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptPerlinNoiseOptions Options,
		UGeometryScriptDebug* Debug = nullptr);


};

#undef UE_API
