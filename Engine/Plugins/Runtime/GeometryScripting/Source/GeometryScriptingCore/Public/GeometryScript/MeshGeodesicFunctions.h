// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"

#include "MeshGeodesicFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshGeodesic"))
class UGeometryScriptLibrary_MeshGeodesicFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:



	/**
	* Computes a vertex list that represents the shortest path constrained to travel along mesh triangle edges between the prescribed start and end vertex.
	* This can fail if the Start and End points are within separate connected components of the mesh.
	* @param TargetMesh defines the surface where the path is computed.
	* @param StartVertexID  indicates ID of mesh vertex that defines the starting point of the path.
	* @param EndVertexID  indicates ID of the mesh vertex that defined the end point of the path.
	* @param VertexIDList, if found this will hold on return a list of mesh vertex IDs that define the path from StartVertexID to EndVertexID.
	* @param bFoundErrors will be false on success.  
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshGeodesic", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetShortestVertexPath(
		UDynamicMesh* TargetMesh,
		int32 StartVertexID,
		int32 EndVertexID,
		FGeometryScriptIndexList& VertexIDList,
		bool& bFoundErrors,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Computes a PolyPath that represents the shortest mesh surface path between two prescribed points on the provided mesh.
	* This can fail if the Start and End points are within separate connected components of the mesh.
	* @param TargetMesh defines the surface where the path is computed.
	* @param StartTriangleID the ID of mesh Triangle that contains the start point of the path.
	* @param StartBaryCoords indicates the location of start point within the start triangle, in terms of barycentric coordinates.
	* @param EndTriangleID the ID of mesh Triangle that contains the end point of the path.
	* @param EndBaryCoords indicates the location of the end point within the end triangle, in terms of barycentric coordinates.
	* @param ShortestPath, if found this will hold on return a PolyPath that defines the shortest path along the mesh surface connecting the start and end points.
	* @param bFoundErrors, will be false on success.  
	* Note, Barycentric coordinates are of the form (a,b,c) where each entry is positive and a + b + c = 1.  
	*       If the provided coordinates are invalid, the value (1/3, 1/3, 1/3) will be used.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshGeodesic", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetShortestSurfacePath(
		UDynamicMesh* TargetMesh,
		int32 StartTriangleID,
		FVector StartBaryCoords,
		int32 EndTriangleID,
		FVector EndBaryCoords,
		FGeometryScriptPolyPath& ShortestPath,
		bool& bFoundErrors,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Computes a PolyPath that represents a "straight" surface path starting at the prescribed point on the mesh, and continuing 
	* in the indicated direction until reaching the requested path length or encountering a mesh boundary, whichever comes first.
	* @param TargetMesh defines the surface where the path is computed.
	* @param Direction is a three-dimensional vector that is projected onto the mesh surface to determine the path direction. 
	* @param StartTriangleID the ID of mesh Triangle that contains the start point of the path.
	* @param StartBaryCoords indicates the location of start point within the start triangle, in terms of barycentric coordinates.
	* @param MaxPathLength sets the maximal length of the path, but the actual path may be shorter as it automatically terminates when encountering a mesh boundary edge. 
	* @param SurfacePath holds on return a PolyPath that forms a "straight" path along the mesh surface from the start position.
	* @param bFoundErrors, will be false on success.  
	* Note, Barycentric coordinates are of the form (a,b,c) where each entry is positive and a + b + c = 1.  
	*       If the provided coordinate is invalid, the value (1/3, 1/3, 1/3) will be used.
	*       Also, if the direction vector is nearly zero, the up-vector will be used. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshGeodesic", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CreateSurfacePath(
		UDynamicMesh* TargetMesh,
		FVector Direction,
		int32 StartTriangleID,
		FVector StartBaryCoords,
		float MaxPathLength,
		FGeometryScriptPolyPath& SurfacePath,
		bool& bFoundErrors,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
