// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshMaterialFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Materials"))
class UGeometryScriptLibrary_MeshMaterialFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Max ID") int GetMaxMaterialID( UDynamicMesh* TargetMesh, bool& bHasMaterialIDs );

	/**
	* Enables per-triangle Material IDs on a mesh and initializes the values to 0. 
	* If Target Mesh already has Material IDs, this function will do nothing.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	EnableMaterialIDs( UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug = nullptr );

	/**
	* Resets all Material IDs on a mesh to the given ClearValue, or 0 if no ClearValue is provided.
	* If Material IDs are not already enabled on the Target Mesh, this function will first enable them and then set the value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ClearMaterialIDs( UDynamicMesh* TargetMesh, int ClearValue = 0, UGeometryScriptDebug* Debug = nullptr );

	/**
	* For all triangles with a Material ID matching the given value (From Material ID), update the Material ID to the new value (To Material ID).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RemapMaterialIDs( 
		UDynamicMesh* TargetMesh, 
		int FromMaterialID,
		int ToMaterialID,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	 * Remap the Material IDs of the TargetMesh to a new set of Material IDs based on a 'From'/Current Material List, and a New Material List.
	 * For each triangle, the current Material is determined as FromMaterialList[MaterialID], and then the first index of this Material is found
	 * in the ToMaterialList, and this index is used as the new MaterialID 
	 * 
	 * If a Material cannot be found in ToMaterialList, a warning will be printed and the MaterialID left unmodified, 
	 * unless MissingMaterialID is set to a value >= 0, in which case MissingMaterialID will be assigned
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RemapToNewMaterialIDsByMaterial( 
		UDynamicMesh* TargetMesh, 
		const TArray<UMaterialInterface*>& FromMaterialList,
		const TArray<UMaterialInterface*>& ToMaterialList,
		int MissingMaterialID = -1,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Remap material IDs to be consistent with a Required Materials list.
	 * The Target Mesh material IDs will be remapped to reference the Combined Materials list, 
	 * which will always start with the Required Materials.
	 *
	 * If a Material cannot be found in CurrentMeshMaterials, a warning will be printed and the MaterialID left unmodified, 
	 * unless RemapInvalidMaterialID is set to a value >= 0, in which case RemapInvalidMaterialID will be assigned
	 *
	 * @param TargetMesh Mesh to update
	 * @param TargetMeshMaterials Initial materials used by the TargetMesh
	 * @param RequiredMaterials Materials that must be used, unchanged, in the output
	 * @param CombinedMaterials Final materials used by the TargetMesh after remapping. Always starts with the RequiredMaterials.
	 * @param RemapInvalidMaterialID If >= 0, automatically remap invalid input material IDs to this value
	 * @param bCompactDuplicateMaterials If true, materials from TargetMeshMaterials will only be added if they are not already in RequiredMaterials. If false, all TargetMeshMaterials are appended.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	RemapAndCombineMaterials(
		UDynamicMesh* TargetMesh,
		const TArray<UMaterialInterface*>& TargetMeshMaterials,
		const TArray<UMaterialInterface*>& RequiredMaterials,
		TArray<UMaterialInterface*>& CombinedMaterials,
		int RemapInvalidMaterialID = -1,
		bool bCompactDuplicateMaterials = true,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Returns the current Material ID for a Triangle.  
	 * If the mesh does not have Material IDs enabled or if the Triangle ID is not an element of the mesh, the value 0 will be returned and bIsValidTriangle will be false.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Materials", meta=(ScriptMethod, DisplayName="Get Material ID Of Triangle"))
	static UE_API UPARAM(DisplayName = "Material ID") int32
	GetTriangleMaterialID( UDynamicMesh* TargetMesh, int TriangleID, bool& bIsValidTriangle );

	/**
	* This populates the MaterialIDList with Material IDs for each triangle in the TriangleIDList.  
	* If a triangle is not present in the Target Mesh the number -1 will be used for the corresponding Material ID.  
	* If Material IDs are not enabled on the TargetMesh no Material IDs will be added to the result list.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMaterialIDsOfTriangles(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "Triangle ID List") FGeometryScriptIndexList TriangleIDList,
		UPARAM(DisplayName = "Material ID List") FGeometryScriptIndexList& MaterialIDList,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	* Returns an Index List of all triangle Material IDs, constructed with one entry for each consecutive Triangle ID.
	* If Material IDs are not enabled on the mesh, bHasMaterialsIDs will be set to false on return and nothing will be added to the Material ID List.
	* @warning if the mesh is not Triangle-Compact (eg GetHasTriangleIDGaps == false) then the returned list will also have the same gaps where the number -1 will be recorded for any missing Triangle IDs.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Materials", meta=(ScriptMethod, DisplayName="Get Material IDs Of All Triangles"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllTriangleMaterialIDs( UDynamicMesh* TargetMesh, FGeometryScriptIndexList& MaterialIDList, bool& bHasMaterialIDs );

	/**
	 * Populates Triangle ID List with the Triangle IDs of triangles that share the specified Material ID in the Target Mesh.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Selection", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetTrianglesByMaterialID( 
		UDynamicMesh* TargetMesh, 
		int MaterialID,
		UPARAM(DisplayName = "Triangle ID List") FGeometryScriptIndexList& TriangleIDList,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Assigns the specified triangle the given Material ID.  
	* If the Target Mesh does not have Material IDs enabled, or if the Triangle ID is not an element of the Target Mesh then bIsValidTriangle will be set to false.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, DisplayName="Set Material ID On Triangle"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetTriangleMaterialID( 
		UDynamicMesh* TargetMesh, 
		int TriangleID, 
		int MaterialID,
		bool& bIsValidTriangle, 
		bool bDeferChangeNotifications = false );

	/**
	* Sets the Material ID of all triangles in a mesh to the values in an input Index List.
	* @param TriangleMateralIDList the list of Material IDs, the length of this list should be the same as the Max Triangle ID for this mesh.
	* @param bDeferChangeNotifications if true, the UDynamicMesh does not emit a change event/signal for this modification.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Set Material IDs On All Triangles"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetAllTriangleMaterialIDs(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "Triangle Material ID List") FGeometryScriptIndexList TriangleMaterialIDList,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Assigns the Material ID to all the triangles specified by the Triangle ID List.
	* @param TriangleIDList the triangles in the target mesh that will be updated with the new Material ID
	* @param MaterialID the ID to be assigned to each triangle in the input list.
	* @param bDeferChangeNotifications if true, the UDynamicMesh does not emit a change event/signal for this modification.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Set Material ID On Triangles"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMaterialIDOnTriangles(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "Triangle ID List") FGeometryScriptIndexList TriangleIDList,
		int MaterialID,
        bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Set a new MaterialID on all the triangles of the given Selection.
	* @param MaterialID new Material ID to set
	* @param bDeferChangeNotifications if true, the UDynamicMesh does not emit a change event/signal for this modification
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Set Material ID For Mesh Selection"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMaterialIDForMeshSelection( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelection Selection,
		int MaterialID,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Set a new MaterialID on all the triangles of TargetMesh with the given PolyGroup.
	* @param GroupLayer PolyGroup Layer to use as basis for PolyGroups
	* @param PolyGroupID PolyGroup ID that specifies Triangles to set to new MaterialID
	* @param MaterialID explicit new MaterialID to set
	* @param bDeferChangeNotifications if true, the UDynamicMesh does not emit a change event/signal for this modification
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Set PolyGroup Material ID"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetPolygroupMaterialID( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		UPARAM(DisplayName = "PolyGroup ID") int PolygroupID,
		int MaterialID,
		UPARAM(DisplayName = "Is Valid PolyGroup ID") bool& bIsValidPolygroupID,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Delete all triangles in TargetMesh with the given MaterialID
	 * @param NumDeleted number of deleted triangles is returned here
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteTrianglesByMaterialID( 
		UDynamicMesh* TargetMesh, 
		int MaterialID,
		int& NumDeleted,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compact the MaterialIDs of the TargetMesh, ie remove any un-used MaterialIDs and remap the remaining
	 * N in-use MaterialIDs to the range [0,N-1]. Optionally compute a Compacted list of Materials.
	 * @param SourceMaterialList Input Material list, assumption is that SourceMaterialList.Num() == number of MaterialIDs on mesh at input
	 * @param CompactedMaterialList new Compacted Material list, one-to-one with new compacted MaterialIDs
	 * @param bRemoveDuplicateMaterials Whether to also remove duplicate materials from the compacted list
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Materials", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CompactMaterialIDs( 
		UDynamicMesh* TargetMesh, 
		TArray<UMaterialInterface*> SourceMaterialList,
		TArray<UMaterialInterface*>& CompactedMaterialList,
		bool bRemoveDuplicateMaterials = false,
		UGeometryScriptDebug* Debug = nullptr);



};

#undef UE_API
