// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshModelingFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

UENUM(BlueprintType)
enum class EGeometryScriptMeshEditPolygroupMode : uint8
{
	PreserveExisting = 0,
	AutoGenerateNew = 1,
	SetConstant = 2
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshEditPolygroupOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptMeshEditPolygroupMode GroupMode = EGeometryScriptMeshEditPolygroupMode::PreserveExisting;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int ConstantGroup = 0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshOffsetOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float OffsetDistance = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFixedBoundary = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SolveSteps = 5;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SmoothAlpha = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bReprojectDuringSmoothing = false;

	// should not be > 0.9
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float BoundaryAlpha = 0.2f;
};


UENUM(BlueprintType)
enum class EGeometryScriptPolyOperationArea : uint8
{
	EntireSelection = 0,
	PerPolygroup = 1 UMETA(DisplayName = "Per PolyGroup"),
	PerTriangle = 2
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshExtrudeOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ExtrudeDistance = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector ExtrudeDirection = FVector(0,0,1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVScale = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSolidsToShells = true;
};


UENUM(BlueprintType)
enum class EGeometryScriptLinearExtrudeDirection : uint8
{
	FixedDirection = 0,
	AverageFaceNormal = 1
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshLinearExtrudeOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Distance = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptLinearExtrudeDirection DirectionMode = EGeometryScriptLinearExtrudeDirection::FixedDirection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector Direction = FVector(0,0,1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolyOperationArea AreaMode = EGeometryScriptPolyOperationArea::EntireSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptMeshEditPolygroupOptions GroupOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVScale = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSolidsToShells = true;
};


UENUM(BlueprintType)
enum class EGeometryScriptOffsetFacesType : uint8
{
	VertexNormal = 0,
	FaceNormal = 1,
	ParallelFaceOffset = 2
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshOffsetFacesOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Distance = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptOffsetFacesType OffsetType = EGeometryScriptOffsetFacesType::ParallelFaceOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolyOperationArea AreaMode = EGeometryScriptPolyOperationArea::EntireSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptMeshEditPolygroupOptions GroupOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVScale = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSolidsToShells = true;
};




USTRUCT(BlueprintType)
struct FGeometryScriptMeshInsetOutsetFacesOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Distance = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bReproject = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bBoundaryOnly = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Softness = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float AreaScale = 1.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolyOperationArea AreaMode = EGeometryScriptPolyOperationArea::EntireSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptMeshEditPolygroupOptions GroupOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float UVScale = 1.0f;
};



USTRUCT(BlueprintType)
struct FGeometryScriptMeshBevelOptions
{
	GENERATED_BODY()
public:
	/** Distance that each beveled mesh edge is inset from it's initial position*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float BevelDistance = 1.0;

	/** If true, when faces on either side of a beveled mesh edges have the same Material ID, beveled edge will be set to that Material ID. Otherwise SetMaterialID is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bInferMaterialID = false;

	/** Material ID to set on the new faces introduced by bevel operation, unless bInferMaterialID=true and non-ambiguous MaterialID can be inferred from adjacent faces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SetMaterialID = 0;

	/** Number of edge loops added along the bevel faces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int Subdivisions = 0;

	/** Roundness of the bevel. Ignored if Subdivisions = 0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float RoundWeight = 1.0;

	/**
	 * If true the set of beveled PolyGroup edges is limited to those that 
	 * are fully or partially contained within the (transformed) FilterBox
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FilterShape, AdvancedDisplay)
	bool bApplyFilterBox = false;

	/**
	 * Bounding Box used for edge filtering
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FilterShape, AdvancedDisplay)
	FBox FilterBox = FBox(EForceInit::ForceInit);

	/**
	 * Transform applied to the FilterBox
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FilterShape, AdvancedDisplay)
	FTransform FilterBoxTransform = FTransform::Identity;

	/**
	 * If true, then only PolyGroup edges that are fully contained within the filter box will be beveled,
	 * otherwise the edge will be beveled if any vertex is within the filter box.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FilterShape, AdvancedDisplay)
	bool bFullyContained = true;
};


/**
 * Mode passed to ApplyMeshBevelSelection to control how the input Selection should
 * be interpreted as selecting an area of the mesh to Bevel
 */
UENUM(BlueprintType)
enum class EGeometryScriptMeshBevelSelectionMode : uint8
{
	/** Convert the selection to Triangles and bevel the boundary edge loops of the triangle set */
	TriangleArea = 0,
	/** Convert the selection to PolyGroups and bevel all the PolyGroup Edges of the selected PolyGroups */
	AllPolygroupEdges = 1 UMETA(DisplayName = "All PolyGroup Edges"),
	/** Convert the selection to PolyGroups and bevel all the PolyGroup Edges that are between selected PolyGroups */
	SharedPolygroupEdges = 2 UMETA(DisplayName = "Shared PolyGroup Edges"),
	/** Convert the selection to Edges (if needed) and bevel them */
	SelectedEdges = 3
};


USTRUCT(BlueprintType)
struct FGeometryScriptMeshBevelSelectionOptions
{
	GENERATED_BODY()
public:
	/** Distance that each beveled mesh edge is inset from it's initial position*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float BevelDistance = 1.0;

	/** If true, when faces on either side of a beveled mesh edges have the same Material ID, beveled edge will be set to that Material ID. Otherwise SetMaterialID is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bInferMaterialID = false;

	/** Material ID to set on the new faces introduced by bevel operation, unless bInferMaterialID=true and non-ambiguous MaterialID can be inferred from adjacent faces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SetMaterialID = 0;

	/** Number of edge loops added along the bevel faces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int Subdivisions = 0;

	/** Roundness of the bevel. Ignored if Subdivisions = 0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float RoundWeight = 1.0;
};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshModeling"))
class UGeometryScriptLibrary_MeshModelingFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Disconnect the triangles of TargetMesh identified by the Selection.
	 * The input Selection will still identify the same geometric elements after Disconnecting.
	 * @param bAllowBowtiesInOutput if false, any bowtie vertices created by the operation will be automatically split
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshDisconnectFaces(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		bool bAllowBowtiesInOutput = true,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Disconnect triangles of TargetMesh along the edges of the Selection.
	 * The input Selection will still identify the same geometric elements after Disconnecting.
	 * 
	 * @param TargetMesh Mesh to operate on
	 * @param Selection Which edges to operate on. Non-edge selections will be interpreted as edge selections -- i.e., all selected triangles' edges, or all selected vertices' one-ring edges.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyMeshDisconnectFacesAlongEdges(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Duplicate the triangles of TargetMesh identified by the Selection
	 * @param NewTriangles a Mesh Selection of the duplicate triangles is returned here (with type Triangles)
	 * @param bAllowBowtiesInOutput if false, any bowtie vertices resulting created in the Duplicate area will be disconnected into unique vertices
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshDuplicateFaces(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptMeshSelection& NewTriangles,
		FGeometryScriptMeshEditPolygroupOptions GroupOptions = FGeometryScriptMeshEditPolygroupOptions(),
		UGeometryScriptDebug* Debug = nullptr );


	/**
	 * Offset the vertices of TargetMesh from their initial positions based on averaged vertex normals.
	 * This function is intended for high-res meshes, for polymodeling-style offsets, ApplyMeshOffsetFaces will produce better results.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshOffset(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshOffsetOptions Options,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Create a thickened shell from TargetMesh by offsetting the vertex positions along averaged vertex normals, inwards or outwards.
	 * Similar to ApplyMeshOffset but also includes the initial mesh (possibly flipped, if the offset is positive)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshShell(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshOffsetOptions Options,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	 * Apply Linear Extrusion (ie extrusion in a single direction) to the triangles of TargetMesh identified by the Selection.
	 * The input Selection will still identify the same geometric elements after the Extrusion
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshLinearExtrudeFaces(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshLinearExtrudeOptions Options,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Apply an Offset to the faces of TargetMesh identified by the Selection, or all faces if the Selection is empty.
	 * The Offset direction at each vertex can be derived from the averaged vertex normals or per-triangle normals.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshOffsetFaces(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshOffsetFacesOptions Options,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Apply an Inset or Outset to the faces of TargetMesh identified by the Selection, or all faces if the Selection is empty.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshInsetOutsetFaces(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshInsetOutsetFacesOptions Options,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Apply a Mesh Bevel operation to parts of TargetMesh using the BevelOptions settings.
	 * @param Selection specifies which mesh edges to Bevel
	 * @param BevelOptions settings for the Bevel Operation
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyMeshBevelEdgeSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptMeshBevelSelectionOptions BevelOptions,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Apply a Mesh Bevel operation to parts of TargetMesh using the BevelOptions settings, with additional options to handle region selections
	 * @param Selection specifies which mesh edges to Bevel
	 * @param BevelMode specifies whether Selection should be optionally converted to a Triangle Region or set of PolyGroup Edges
	 * @param BevelOptions settings for the Bevel Operation
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply Mesh Bevel Region Selection"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshBevelSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		EGeometryScriptMeshBevelSelectionMode BevelMode,
		FGeometryScriptMeshBevelSelectionOptions BevelOptions,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	 * Apply a Mesh Bevel operation to all PolyGroup Edges.
     */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Modeling", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply Mesh PolyGroup Bevel"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshPolygroupBevel(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshBevelOptions Options,
		UGeometryScriptDebug* Debug = nullptr );




	//---------------------------------------------
	// Backwards-Compatibility implementations
	//---------------------------------------------
	// These are versions/variants of the above functions that were released
	// in previous UE 5.x versions, that have since been updated. 
	// To avoid breaking user scripts, these previous versions are currently kept and 
	// called via redirectors registered in GeometryScriptingCoreModule.cpp.
	// 
	// These functions may be deprecated in future UE releases.
	//


	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Compatibility", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyMeshExtrude_Compatibility_5p0(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshExtrudeOptions Options,
		UGeometryScriptDebug* Debug = nullptr );

};

#undef UE_API
