// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshNormalsFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

USTRUCT(BlueprintType)
struct FGeometryScriptCalculateNormalsOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAngleWeighted = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAreaWeighted = true;
};



USTRUCT(BlueprintType)
struct FGeometryScriptSplitNormalsOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSplitByOpeningAngle = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float OpeningAngleDeg = 15.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSplitByFaceGroup = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptGroupLayer GroupLayer;
};


UENUM(BlueprintType)
enum class EGeometryScriptTangentTypes : uint8
{
	FastMikkT = 0,
	PerTriangle = 1,
	// Standard MikkT-algorithm tangents. Available for windows/mac/linux platforms; will fall back to FastMikkT if unavailable.
	StandardMikkT
};


USTRUCT(BlueprintType)
struct FGeometryScriptTangentsOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptTangentTypes Type = EGeometryScriptTangentTypes::FastMikkT;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int UVLayer = 0;
};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Normals"))
class UGeometryScriptLibrary_MeshNormalsFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Flip/Invert the normal vectors of TargetMesh by multiplying them by -1, as well as reversing the mesh triangle orientations, ie triangle (a,b,c) becomes (b,a,c)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	FlipNormals( 
		UDynamicMesh* TargetMesh, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Flip/Invert the normal vectors of the triangles in a given selection of TargetMesh. For edge or vertex selections, normals for the triangles touching the selected edges or vertices will be flipped.
	 * 
	 * @param bFlipTriangleOrientation If true, triangles are flipped by reversing the mesh triangle orientations, ie triangle (a,b,c) becomes (b,a,c)
	 * @param bFlipNormalDirection If true, normals of vertices on the selected triangles are flipped by multiplying them by -1
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	FlipTriangleSelectionNormals(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		bool bFlipTriangleOrientation = true,
		bool bFlipNormalDirection = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Attempt to repair inconsistent normals in TargetMesh. Currently this is done in two passes. In the first pass, triangles with
	 * reversed orientation from their neighours are incrementally flipped until each connected component has a consistent orientation,
	 * if this is possible (note that this is not always globally possible, eg for a mobius-strip topology there is no consistent orientation).
	 * In the second pass, the "global" orientation is detected by casting rays from outside the mesh. This may produce incorrect results for
	 * meshes that are not closed. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AutoRepairNormals( 
		UDynamicMesh* TargetMesh, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute the normals of TargetMesh by averaging the triangle/face normals around each vertex, using combined area and angle weighting.
	 * Each vertex will have a single normal, ie there will be no hard edges.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Set Mesh To Per Vertex Normals (Computed)"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetPerVertexNormals( 
		UDynamicMesh* TargetMesh, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute the normals of TargetMesh by setting the normals of each triangle vertex to the triangle/face normal.
	 * Each vertex will have a unique normal in each triangle, ie there will be hard edges / split normals at every mesh edge
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Set Mesh To Facet Normals"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetPerFaceNormals( 
		UDynamicMesh* TargetMesh, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute the normals of TargetMesh using the given CalculateOptions. This method will preserve any existing hard
	 * edges, ie each shared triangle-vertex normal is recomputed by averaging the face normals of triangles that reference
	 * that shared triangle-vertex normal
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RecomputeNormals(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptCalculateNormalsOptions CalculateOptions,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute the normals of TargetMesh on all the triangles/vertices of the given Selection using the given CalculateOptions. 
	 * This method will preserve any existing hard edges, ie each shared triangle-vertex normal is recomputed by averaging 
	 * the face normals of triangles that reference that shared triangle-vertex normal
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RecomputeNormalsForMeshSelection(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptCalculateNormalsOptions CalculateOptions,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute hard edges / split-normals for TargetMesh based on the provided SplitOptions, and then 
	 * recompute the new shared triangle-vertex normals using the given CalculateOptions. 
	 * The normal recomputation is identical to calling RecomputeNormals.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeSplitNormals( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptSplitNormalsOptions SplitOptions,
		FGeometryScriptCalculateNormalsOptions CalculateOptions,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Set or remove split normals (aka sharp normals) for all edges in the Selection
	 * @param TargetMesh The mesh to update
	 * @param Selection Which edges to operate on
	 * @param bSplit Whether to split normals along the selected edges; if false, they will be merged instead
	 * @param bRecalculateNormals Whether to recalculate normals along edges where they were split/merged
	 * @param CalculateOptions Options for computing the normals, if bRecalculateNormals is true
	 * @param bDeferChangeNotifications If true, no mesh change notification will be sent. Set to true if performing many changes in a loop.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetSplitNormalsAlongSelectedEdges(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		bool bSplit = true,
		bool bRecalculateNormals = true,
		FGeometryScriptCalculateNormalsOptions CalculateOptions = FGeometryScriptCalculateNormalsOptions(),
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Set the triangle-vertex normals for the given TriangleID on the TargetMesh. This will
	 * create unique triangle-vertex normals, ie it will create hard edges / split normals in 
	 * the normal overlay for each edge of the triangle. 
	 * @param bIsValidTriangle will be returned as false if TriangleID does not refer to a valid triangle
	 * @param bDeferChangeNotifications if true, no mesh change notification will be sent. Set to true if changing many normals in a loop.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshTriangleNormals( 
		UDynamicMesh* TargetMesh, 
		int TriangleID, 
		FGeometryScriptTriangle Normals,
		bool& bIsValidTriangle, 
		bool bDeferChangeNotifications = false );

	/**
	 * Set all normals in the TargetMesh Normals Overlay to the specified per-vertex normals
	 * @param VertexNormalList per-vertex normals. Size must be equal to the MaxVertexID of TargetMesh  (ie non-compact TargetMesh is supported)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Set Mesh To Per Vertex Normals (From List)"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshPerVertexNormals(
		UDynamicMesh* TargetMesh,
		FGeometryScriptVectorList VertexNormalList,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Get a list of single normal vectors for each mesh vertex in the TargetMesh, derived from the Normals Overlay.
	 * The Normals Overlay may store multiple normals for a single vertex (ie split normals)
	 * In such cases the normals can either be averaged, or the last normal seen will be used, depending on the bAverageSplitVertexValues parameter.
	 * @param NormalList output normal list will be stored here. Size will be equal to the MaxVertexID of TargetMesh  (not the VertexCount!)
	 * @param bIsValidNormalSet will be set to true if the Normal Overlay was valid
	 * @param bHasVertexIDGaps will be set to true if some vertex indices in TargetMesh were invalid, ie MaxVertexID > VertexCount 
	 * @param bAverageSplitVertexValues control how multiple normals at the same vertex should be interpreted
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshPerVertexNormals( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptVectorList& NormalList, 
		bool& bIsValidNormalSet,
		bool& bHasVertexIDGaps,
		bool bAverageSplitVertexValues = true);



	/**
	 * Check if the TargetMesh has a Tangents Attribute Layer enabled
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshHasTangents( 
		UDynamicMesh* TargetMesh,
		bool& bHasTangents,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Remove any existing Tangents Attribute Layer from the TargetMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DiscardTangents( 
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Recompute Tangents for the TargetMesh, using the method and settings specified by FGeometryScriptTangentsOptions
	 * @note If recomputing Tangents for use with a DynamicMeshComponent, it is also necessary to set the Tangents Type on the Component to "Externally Provided"
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeTangents( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptTangentsOptions Options,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Set all tangents in the TargetMesh Tangents Overlays to the specified per-vertex tangents
	 * @param TangentXList per-vertex tangent vectors. Size must be equal to the MaxVertexID of TargetMesh  (ie non-compact TargetMesh is supported)
	 * @param TangentYList per-vertex bitangent/binormal vectors. Size must be equal to TangentXList
	 * @note If setting Tangents for use with a DynamicMeshComponent, it is also necessary to set the Tangents Type on the Component to "Externally Provided"
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshPerVertexTangents(
		UDynamicMesh* TargetMesh,
		FGeometryScriptVectorList TangentXList,
		FGeometryScriptVectorList TangentYList,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get a list of single tangent vectors for each mesh vertex in the TargetMesh, derived from the Tangents Overlays.
	 * The Tangents Overlay may store multiple tangents for a single vertex (ie split tangents)
	 * In such cases the tangents can either be averaged, or the last tangent seen will be used, depending on the bAverageSplitVertexValues parameter.
	 * @param TangentXList output Tangent "X" vectors list will be stored here. Size will be equal to the MaxVertexID of TargetMesh  (not the VertexCount!)
	 * @param TangentYList output Tangent "Y" vectors (Binormal/Bitangent) list will be stored here. Size will be equal to TangentXList
	 * @param bIsValidTangentSet will be set to true if the Tangent Overlay was valid
	 * @param bHasVertexIDGaps will be set to true if some vertex indices in TargetMesh were invalid, ie MaxVertexID > VertexCount 
	 * @param bAverageSplitVertexValues control how multiple tangents at the same vertex should be interpreted
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Normals", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshPerVertexTangents( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptVectorList& TangentXList, 
		FGeometryScriptVectorList& TangentYList,
		bool& bIsValidTangentSet,
		bool& bHasVertexIDGaps,
		bool bAverageSplitVertexValues = true);


	/**
	 * Update the Normals and/or Tangents at VertexID of TargetMesh. Note that the specified vertex may have "split normals"
	 * or "split tangents", ie in the case of hard/crease normals, UV seams, and so on. In these situations, by default
	 * each of the unique normals/tangents at the vertex will be updated, but they will not be "merged", ie they will remain split.
	 * However if bMergeSplitValues=true, then the vertex will be "un-split", ie after the function call the vertex will have
	 * a single unique shared normal and/or tangents. 
	 * 
	 * Note that this function requires that some normals/tangents already exist on the TargetMesh. If this is not the case, 
	 * functions like SetPerVertexNormals and ComputeTangents can be used to initialize the normals/tangents first.
	 * 	
	 * @param bUpdateNormal if true (default) then the normals overlay is updated
	 * @param NewNormal the new normal vector. This vector will not be normalized, it must be normalized by the calling code.
	 * @param bUpdateTangents if true then the tangents overlay will be updated. If the tangents overlay does not exist, this function returns an error.
	 * @param NewTangentX the new tangent vector. This vector will not be normalized, it must be normalized by the calling code.
	 * @param NewTangentY the new bitangent/binormal vector. This vector will not be normalized, it must be normalized by the calling code.
	 * @param bIsValidVertex will be set to true on return if the VertexID was valid, ie had valid normals and tangents
	 * @param bMergeSplitValues if true, any split normals/tangents at the vertex will be cleared, and a unique normal/tangent element will be set in the connected triangles
	 * @param bDeferChangeNotifications if true, no mesh change notification will be sent. Set to true if changing many normals in a loop.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod, bUpdateNormal="true"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	UpdateVertexNormal( 
		UDynamicMesh* TargetMesh, 
		int VertexID, 
		bool bUpdateNormal,
		FVector NewNormal, 
		bool bUpdateTangents,
		FVector NewTangentX,
		FVector NewTangentY,
		bool& bIsValidVertex, 
		bool bMergeSplitValues,
		bool bDeferChangeNotifications = false );

};

#undef UE_API
