// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshPolygroupFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_PolyGroups"))
class UGeometryScriptLibrary_MeshPolygroupFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	* Enables the standard PolyGroup Layer on the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Enable PolyGroups"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	EnablePolygroups( UDynamicMesh* TargetMesh, UGeometryScriptDebug* Debug = nullptr );

	/**
	* Sets the number of extended PolyGroup Layers on a Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Set Num Extended PolyGroup Layers"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetNumExtendedPolygroupLayers( 
		UDynamicMesh* TargetMesh, 
		int NumLayers,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Add an extended polygroup layer with the given name. If a layer with that name is already on the mesh, that existing layer will be returned and no new layer will be added.
	 * 
	 * @param TargetMesh Mesh to update
	 * @param LayerName Name of the PolyGroup layer
	 * @param GroupLayer PolyGroup layer with the requested name
	 * @param bAlreadyExisted If true, the layer already existed and did not need to be added (and the output GroupLayer will refer to that existing layer)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Add Named PolyGroup Layer"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddNamedPolygroupLayer( 
		UDynamicMesh* TargetMesh, 
		FName LayerName,
		FGeometryScriptGroupLayer& GroupLayer,
		bool& bAlreadyExisted,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Find an extended PolyGroup layer by its name. If there are multiple layers with the same name, returns the first such layer.
	 * 
	 * @param TargetMesh Mesh to search
	 * @param LayerName Name of the PolyGroup layer to find
	 * @param GroupLayer PolyGroup layer with the requested name (if Outcome is Success)
	 * @param Outcome Whether the requested layer was found
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Find Extended PolyGroup Layer By Name", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Target Mesh") const UDynamicMesh*
	FindExtendedPolygroupLayerByName(
		const UDynamicMesh* TargetMesh,
		FName LayerName,
		FGeometryScriptGroupLayer &GroupLayer,
		EGeometryScriptSearchOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);



	/**
	* Resets the triangle PolyGroup assignments within a PolyGroup Layer to the given Clear Value (or 0 if no Clear Value is specified).
	* Note, this will have no effect if PolyGroups have not been enabled on the mesh, or if the requested Group Layer does not exist. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Clear PolyGroups"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ClearPolygroups( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		int ClearValue = 0,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Copies the triangle PolyGroup assignments from one layer on the Target Mesh to another.
	* Note, this will have no effect if PolyGroups have not been enabled on the mesh, or if one of the requested Group Layers does not exist.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Copy PolyGroups Layer"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CopyPolygroupsLayer( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer FromGroupLayer,
		FGeometryScriptGroupLayer ToGroupLayer,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Creates and assigns a new PolyGroup for each disconnected UV island of a Mesh.
	* Note, this will have no effect if either the requested UV Layer or Group Layer does not exist.
	* @param GroupLayer indicates PolyGroup Layer that will be populated with unique values for each UV island.
	* @param UVLayer specifies the UV Layer used to construct the PolyGroups.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Convert UV Islands To PolyGroups"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertUVIslandsToPolygroups( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		int UVLayer = 0,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Creates and assigns a new PolyGroup for each disconnected island (component) of a Mesh. Regions of a mesh are disconnected they do not have a triangle in common.
	* Note, this will have no effect if the Group Layer does not exist.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", KeyWords = "Components",
		DisplayName = "Convert Connected Islands To PolyGroups"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertComponentsToPolygroups( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Sets PolyGroups by partitioning the mesh based on an edge crease/opening-angle.
	* Note, this will have no effect if the Group Layer does not exist.
	* @param GroupLayer indicates the PolyGroup Layer that will be populated.
	* @param CreaseAngle measured in degrees and used when comparing adjacent faces.
	* @param MinGroupSize the minimum number of triangles in each PolyGroup.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Compute PolyGroups From Angle Threshold"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputePolygroupsFromAngleThreshold( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		float CreaseAngle = 15,
		int MinGroupSize = 2,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Sets PolyGroups by identifying adjacent triangles that form reasonable quads. Note any triangles that do not neatly pair to form quads will receive their own PolyGroup.  
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Compute PolyGroups From Polygon Detection"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputePolygroupsFromPolygonDetection( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		bool bRespectUVSeams = true,
		bool bRespectHardNormals = false,
		double QuadAdjacencyWeight = 1.0,
		double QuadMetricClamp = 1.0,
		int MaxSearchRounds = 1,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Gets the PolyGroup ID associated with the specified Triangle ID and stored in the Group Layer.
	* If the Group Layer or the Triangle does not exist, the value 0 will be returned and bIsValidTriangle set to false.
	* @param GroupLayer indicates the layer on the Target Mesh to query.
	* @param TriangleID identifies a triangle in the Target Mesh.
	* @param bIsValidTriangle will be populated on return with false if either the Group Layer or the Triangle does not exist.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get Triangle PolyGroup ID"))
	static UE_API UPARAM(DisplayName = "PolyGroup ID") int32
	GetTrianglePolygroupID( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer, 
		int TriangleID, 
		bool& bIsValidTriangle );

	/**
	 * Deletes all triangles from the Target Mesh that have a particular PolyGroup ID, in the specific Group Layer.
	 * @param GroupLayer specifies the PolyGroup Layer to query.
	 * @param PolyGroup ID identifies the triangles in the Target Mesh to delete.
	 * @param NumDeleted on return will contain the number of triangles deleted from the Target Mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Delete Triangles In PolyGroup"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	DeleteTrianglesInPolygroup( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer,
		UPARAM(DisplayName = "PolyGroup ID") int PolygroupID,
		int& NumDeleted,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Create list of per-triangle PolyGroup IDs for the PolyGroup in the Mesh
	* @warning if the mesh is not Triangle-Compact (eg GetHasTriangleIDGaps == false) then the returned list will also have the same gaps
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get All Triangle PolyGroup IDs"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllTrianglePolygroupIDs( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer, 
		UPARAM(ref, DisplayName="PolyGroup IDs Out") FGeometryScriptIndexList& PolygroupIDsOut );

	/**
	* Create list of all unique PolyGroup IDs that exist in the PolyGroup Layer in the Mesh
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get PolyGroup IDs In Mesh"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetPolygroupIDsInMesh( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer, 
		UPARAM(ref, DisplayName="PolyGroup IDs Out") FGeometryScriptIndexList& PolygroupIDsOut );

	/**
	 * Compute the bounds of a PolyGroup
	 * 
	 * @param GroupLayer The PolyGroup layer to query
	 * @param GroupID The PolyGroup to query
	 * @param Bounds The bounds of the PolyGroups's triangles. Can be invalid if the PolyGroup was not found.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get PolyGroup Bounding Box"))
	static UE_API UPARAM(DisplayName = "Target Mesh") const UDynamicMesh*
	GetPolyGroupBoundingBox(const UDynamicMesh* TargetMesh,
		FGeometryScriptGroupLayer GroupLayer,
		int32 GroupID,
		FBox& Bounds);

	/**
	 * Compute the UV bounds of a PolyGroup
	 * 
	 * @param GroupLayer The PolyGroup layer to query
	 * @param GroupID The PolyGroup to query
	 * @param UVChannel The UV channel to query
	 * @param UVBounds The bounds of the PolyGroups's UV triangles. Can be invalid if the PolyGroup was not found or did not have UVs in the requested channel.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get PolyGroup UV Bounding Box"))
	static UE_API UPARAM(DisplayName = "Target Mesh") const UDynamicMesh*
	GetPolyGroupUVBoundingBox(const UDynamicMesh* TargetMesh,
		FGeometryScriptGroupLayer GroupLayer,
		int32 GroupID,
		UPARAM(DisplayName = "UV Channel") int32 UVChannel,
		UPARAM(DisplayName = "UV Bounds") FBox2D& UVBounds);

	/**
	 * Compute the UV centroid of a PolyGroup
	 * 
	 * @param GroupLayer The PolyGroup layer to query
	 * @param GroupID The PolyGroup to query
	 * @param UVChannel The UV channel to query
	 * @param Centroid The centroid of the polygroup's UV triangles
	 * @param bIsValid True if a valid centroid was found, false otherwise. Can be false if the PolyGroup was not found or did not have UVs in the requested channel.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get PolyGroup UV Centroid"))
	static UE_API UPARAM(DisplayName = "Target Mesh") const UDynamicMesh*
	GetPolyGroupUVCentroid(const UDynamicMesh* TargetMesh,
		FGeometryScriptGroupLayer GroupLayer,
		int32 GroupID,
		UPARAM(DisplayName = "UV Channel") int32 UVChannel,
		FVector2D& Centroid,
		bool& bIsValid);

	/**
	 * Create list of all triangles with the given PolyGroup ID in the given GroupLayer (not necessarily a single connected-component / island)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Get Triangles In PolyGroup"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetTrianglesInPolygroup( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer, 
		UPARAM(DisplayName = "PolyGroup ID") int PolygroupID, 
		UPARAM(ref, DisplayName="Triangle IDs Out") FGeometryScriptIndexList& TriangleIDsOut );


	/**
	 * Set a new PolyGroup on all the triangles of the given Selection, for the given GroupLayer.
	 * @param SetPolygroupID explicit new PolyGroupID to set
	 * @param bGenerateNewPolygroup if true, SetPolyGroupID is ignored and a new unique PolyGroupID is generated
	 * @param SetPolygroupIDOut the PolyGroupID that was set on the triangles is returned here (whether explicit or auto-generated)
	 * @param bDeferChangeNotifications if true, the UDynamicMesh does not emit a change event/signal for this modification
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyGroups", meta = (ScriptMethod, DisplayName = "Set PolyGroup For Mesh Selection"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetPolygroupForMeshSelection( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptGroupLayer GroupLayer, 
		FGeometryScriptMeshSelection Selection,
		UPARAM(DisplayName = "Set PolyGroup ID Out") int& SetPolygroupIDOut,
		UPARAM(DisplayName = "Set PolyGroup ID") int SetPolygroupID = 0,
		UPARAM(DisplayName = " Generate New PolyGroup") bool bGenerateNewPolygroup = false,
		bool bDeferChangeNotifications = false);
};

#undef UE_API
