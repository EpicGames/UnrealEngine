// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshPoolFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMeshPool;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshPoolUtility"))
class UGeometryScriptLibrary_MeshPoolFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Access a global compute mesh pool (created on first access) */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshPool")
	static UE_API UPARAM(DisplayName = "Mesh Pool") UDynamicMeshPool* 
	GetGlobalMeshPool();

	/** Fully clear/destroy the current global mesh pool, allowing it and all its meshes to be garbage collected */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshPool")
	static UE_API void DiscardGlobalMeshPool();
};

#undef UE_API
