// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshPrimitiveFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


UENUM(BlueprintType)
enum class EGeometryScriptPrimitivePolygroupMode : uint8
{
	SingleGroup = 0,
	PerFace = 1,
	PerQuad = 2
};

UENUM(BlueprintType)
enum class EGeometryScriptPrimitiveOriginMode : uint8
{
	Center = 0,
	Base = 1
};

UENUM(BlueprintType)
enum class EGeometryScriptPrimitiveUVMode : uint8
{
	Uniform = 0,
	ScaleToFill = 1
};

UENUM(BlueprintType)
enum class EGeometryScriptPolygonFillMode : uint8
{
	// Keep all triangles, regardless of whether they were enclosed by constrained edges
	All = 0,
	// Fill everything inside the outer boundaries of constrained edges, ignoring edge orientation and any internal holes
	Solid = 1,
	// Fill where the 'winding number' is positive
	PositiveWinding = 2,
	// Fill where the 'winding number' is not zero
	NonZeroWinding = 3,
	// Fill where the 'winding number' is negative
	NegativeWinding = 4,
	// Fill where the 'winding number' is an odd number
	OddWinding = 5
};


USTRUCT(BlueprintType)
struct FGeometryScriptPrimitiveOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName = "PolyGroup Mode"))
	EGeometryScriptPrimitivePolygroupMode PolygroupMode = EGeometryScriptPrimitivePolygroupMode::PerFace;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFlipOrientation = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPrimitiveUVMode UVMode = EGeometryScriptPrimitiveUVMode::Uniform;

	// Material ID to set on primitive mesh triangles
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 MaterialID = 0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptRevolveOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float RevolveDegrees = 360.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float DegreeOffset = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bReverseDirection = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHardNormals = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float HardNormalAngle = 30.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bProfileAtMidpoint = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bFillPartialRevolveEndcaps = true;
};

USTRUCT(BlueprintType)
struct FGeometryScriptVoronoiOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float BoundsExpand = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FBox Bounds = FBox(EForceInit::ForceInit);

	/** Optional list of cells to create meshes for.  If empty, create all cells. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<int32> CreateCells;

	/** Whether to include the bordering Voronoi cells (which extend 'infinitely' to any boundary) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bIncludeBoundary = true;
};


USTRUCT(BlueprintType)
struct FGeometryScriptConstrainedDelaunayTriangulationOptions
{
	GENERATED_BODY()
public:
	/** How to decide which parts of the shape defined by constrained edges should be filled with triangles */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolygonFillMode ConstrainedEdgesFillMode = EGeometryScriptPolygonFillMode::All;

	/**
	 * Whether the triangulation should be considered a failure if it doesn't include the requested Constrained Edges.
	 * (Edges may be missing e.g. due to intersecting edges in the input.) 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bValidateEdgesInResult = true;

	/** Whether to remove duplicate vertices from the output.  If false, duplicate vertices will not be used in any triangles, but will remain in the output mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRemoveDuplicateVertices = false;

};


USTRUCT(BlueprintType)
struct FGeometryScriptPolygonsTriangulationOptions
{
	GENERATED_BODY()
public:

	/** Whether to still append the triangulation in error cases -- typically, cases where the input contained intersecting edges. Resulting triangulation likely will appear correct except at the intersecting edges. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bStillAppendOnTriangulationError = true;

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Primitives"))
class UGeometryScriptLibrary_MeshPrimitiveFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Appends a 3D box to the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendBox( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		float DimensionZ = 100,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D box to the Target Mesh
	 * Also creates matching simple collision
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendBoxWithCollision(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision &SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		float DimensionZ = 100,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D box to the Target Mesh with dimensions and origin taken from the input Box
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendBoundingBox( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FBox Box,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D oriented box to the Target Mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendOrientedBox(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FOrientedBox Box,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D oriented box to the Target Mesh
	 * Also creates matching simple collision
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendOrientedBoxWithCollision(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FOrientedBox Box,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D box to the Target Mesh with dimensions and origin taken from the input Box
	 * Also creates matching simple collision
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendBoundingBoxWithCollision( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FBox Box,
		int32 StepsX = 0,
		int32 StepsY = 0,
		int32 StepsZ = 0,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Appends a 3D Sphere triangulated using latitude/longitude topology to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSphereLatLong( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		int32 StepsPhi = 10,
		int32 StepsTheta = 16,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Center,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	* Appends a 3D Sphere triangulated using latitude/longitude topology to the Target Mesh.
	* Also creates matching simple collision. Note: If transform has non-uniform scale, collision shape will be a uniform-scale approximation
	* matching the behavior of the physics system -- specifically, it will scale the sphere radius by the smallest axis scale.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSphereLatLongWithCollision( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		int32 StepsPhi = 10,
		int32 StepsTheta = 16,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Center,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a 3D sphere triangulated using box topology to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSphereBox( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		int32 StepsX = 6,
		int32 StepsY = 6,
		int32 StepsZ = 6,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Center,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D sphere triangulated using box topology to the Target Mesh.
	 * Also creates matching simple collision. Note: If transform has non-uniform scale, collision shape will be a uniform-scale approximation
	 * matching the behavior of the physics system -- specifically, it will scale the sphere radius by the smallest axis scale.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendSphereBoxWithCollision(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		int32 StepsX = 6,
		int32 StepsY = 6,
		int32 StepsZ = 6,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Center,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a 3D Capsule to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendCapsule( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 30,
		float LineLength = 75,
		int32 HemisphereSteps = 5,
		int32 CircleSteps = 8,
		int32 SegmentSteps = 0,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Appends a 3D Capsule to the Target Mesh.
	 * Also creates matching simple collision. Note: If transform has non-uniform scale, collision shape will be a uniform-scale approximation
	 * matching the behavior of the physics system -- specifically, it will scale the radius by the larger of the X, Y axis scales, and the length by the Z axis scale
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendCapsuleWithCollision( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 30,
		float LineLength = 75,
		int32 HemisphereSteps = 5,
		int32 CircleSteps = 8,
		int32 SegmentSteps = 0,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	// Version of append capsule without SegmentSteps parameter
	UE_DEPRECATED(5.5, "Use AppendCapsule with the SegmentSteps parameter, instead")
	static UDynamicMesh* AppendCapsule(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 30,
		float LineLength = 75,
		int32 HemisphereSteps = 5,
		int32 CircleSteps = 8,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr)
	{
		return AppendCapsule(TargetMesh, PrimitiveOptions, Transform, Radius, LineLength, HemisphereSteps, CircleSteps, 0, Origin, Debug);
	}

	/**
	* Appends a 3D Cylinder (with optional end caps) to the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendCylinder( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		float Height = 100,
		int32 RadialSteps = 12,
		int32 HeightSteps = 0,
		bool bCapped = true,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a 3D cone to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendCone( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float BaseRadius = 50,
		float TopRadius = 5,
		float Height = 100,
		int32 RadialSteps = 12,
		int32 HeightSteps = 4,
		bool bCapped = true,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a 3D torus (donut) or partial torus to the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendTorus( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FGeometryScriptRevolveOptions RevolveOptions,
		float MajorRadius = 50,
		float MinorRadius = 25,
		int32 MajorSteps = 16,
		int32 MinorSteps = 8,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * In the coordinate system of the revolve polygon, +X is towards the "outside" of the revolve donut, and +Y is "up" (ie +Z in local space)
	 * Polygon should be oriented counter-clockwise to produce a correctly-oriented shape, otherwise it will be inside-out
	 * Polygon endpoint is not repeated.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRevolvePolygon( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		FGeometryScriptRevolveOptions RevolveOptions,
		float Radius = 100,
		int32 Steps = 8,
		UGeometryScriptDebug* Debug = nullptr);

    /**
	* Revolves a 2D polygon on a helical path, like one used to create a vertical spiral, appending the result to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendSpiralRevolvePolygon(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		FGeometryScriptRevolveOptions RevolveOptions,
		float Radius = 100,
		int Steps = 18,
		float RisePerRevolution = 50,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Revolves an open 2D path, with optional top and bottom end caps, appending the result to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRevolvePath( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PathVertices,
		FGeometryScriptRevolveOptions RevolveOptions,
		int32 Steps = 8,
		bool bCapped = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Sweep the given 2D PolylineVertices along the SweepPath specified as a set of FTransforms
	 * If the 2D vertices are (U,V), then in the coordinate space of the FTransform, X points "along" the path,
	 * Y points "right" (U) and Z points "up" (V).
	 * @param PolylineVertices vertices of the open 2D path that will be swept along the SweepPath
	 * @param SweepPath defines the 3D sweep path curve as a 3D poly-path, with rotation and scaling at each polypath vertex taken from the Transform
	 * @param PolylineTexParamU defines U coordinate value for each element in PolylineVertices. Must be same length as PolylineVertices (ignored if length=0).
	 * @param SweepPathTexParamV defines V coordinate value for each element in SweepPath. Must be same length as SweepPath if bLoop=false, length+1 if bLoop=true, and ignored if length=0.
	 * @param bLoop if true, SweepPath is considered to be a Loop and a section connecting the end and start of the path is added (bCapped is ignored)
	 * @param StartScale uniform scaling applied to the 2D polygon at the start of the path. Interpolated via arc length to EndScale at the end of the path.
	 * @param EndScale uniform scaling applied to the 2D polygon at the end of the path
	 * @param RotationAngleDeg Rotation applied to the 2D Polygon. Positive rotation rotates clockwise, ie Up/+Z/+V towards Right/+Y/+U. This Rotation is applied before any rotation in the SweepPath Transforms.
	 * @param MiterLimit If > 1, maximum scaling to apply at sharp turns in the curve path to keep the apparent swept profile from shrinking, and sweep path frames will be aligned to the path direction
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug", AutoCreateRefTerm="PolylineTexParamU, SweepPathTexParamV"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSweepPolyline( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolylineVertices,
		const TArray<FTransform>& SweepPath,
		const TArray<float>& PolylineTexParamU,
		const TArray<float>& SweepPathTexParamV,
		bool bLoop = false,
		float StartScale = 1.0f,
		float EndScale = 1.0f,
		float RotationAngleDeg = 0.0f,
		float MiterLimit = 1.0f,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Polygon should be oriented counter-clockwise to produce a correctly-oriented shape, otherwise it will be inside-out
	 * Polygon endpoint is not repeated.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSimpleExtrudePolygon( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		float Height = 100,
		int32 HeightSteps = 0,
		bool bCapped = true,
		EGeometryScriptPrimitiveOriginMode Origin = EGeometryScriptPrimitiveOriginMode::Base,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Sweeps a 2D polygon along an arbitrary 3D path, appending the result to the Target Mesh.
	 * @param PolygonVertices vertices of the closed 2D polyon that will be swept along the SweepPath
	 * @param SweepPath defines the 3D sweep path curve
	 * @param bLoop if true, SweepPath is considered to be a Loop and a section connecting the end and start of the path is added (bCapped is ignored)
	 * @param bCapped if true the open ends of the swept generalized cylinder are triangulated
	 * @param StartScale uniform scaling applied to the 2D polygon at the start of the path. Interpolated via arc length to EndScale at the end of the path.
	 * @param EndScale uniform scaling applied to the 2D polygon at the end of the path
	 * @param RotationAngleDeg Rotation applied to the 2D Polygon. Positive rotation rotates clockwise, ie Up/+Z/+V towards Right/+Y/+U
	 * @param MiterLimit If > 1, maximum scaling to apply at sharp turns in the curve path to keep the apparent swept profile from shrinking
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSimpleSweptPolygon( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		const TArray<FVector>& SweepPath,
		bool bLoop = false,
		bool bCapped = true,
		float StartScale = 1.0f,
		float EndScale = 1.0f,
		float RotationAngleDeg = 0.0f,
		float MiterLimit = 1.0f,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Sweep the given 2D PolygonVertices along the SweepPath specified as a set of FTransforms
	 * If the 2D vertices are (U,V), then in the coordinate space of the FTransform, X points "along" the path,
	 * Y points "right" (U) and Z points "up" (V).
	 * @param PolygonVertices vertices of the closed 2D polyon that will be swept along the SweepPath
	 * @param SweepPath defines the 3D sweep path curve as a 3D poly-path, with rotation and scaling at each polypath vertex taken from the Transform
	 * @param bLoop if true, SweepPath is considered to be a Loop and a section connecting the end and start of the path is added (bCapped is ignored)
	 * @param bCapped if true the open ends of the swept generalized cylinder are triangulated
	 * @param StartScale uniform scaling applied to the 2D polygon at the start of the path. Interpolated via arc length to EndScale at the end of the path.
	 * @param EndScale uniform scaling applied to the 2D polygon at the end of the path
	 * @param RotationAngleDeg Rotation applied to the 2D Polygon. Positive rotation rotates clockwise, ie Up/+Z/+V towards Right/+Y/+U. This Rotation is applied before any rotation in the SweepPath Transforms.
	 * @param MiterLimit If > 1, maximum scaling to apply at sharp turns in the curve path to keep the apparent swept profile from shrinking, and sweep path frames will be aligned to the path direction
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendSweepPolygon( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		const TArray<FTransform>& SweepPath,
		bool bLoop = false,
		bool bCapped = true,
		float StartScale = 1.0f,
		float EndScale = 1.0f,
		float RotationAngleDeg = 0.0f,
		float MiterLimit = 1.0f,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Appends a planar Rectangle to a Dynamic Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRectangleXY( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		int32 StepsWidth = 0,
		int32 StepsHeight = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a planar Rectangle with Rounded Corners (RoundRect) to the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRoundRectangleXY( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		float CornerRadius = 5,
		int32 StepsWidth = 0,
		int32 StepsHeight = 0,
		int32 StepsRound = 4,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a planar disc to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendDisc( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float Radius = 50,
		int32 AngleSteps = 16,
		int32 SpokeSteps = 0,
		float StartAngle = 0,
		float EndAngle = 360,
		float HoleRadius = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a Triangulated Polygon to the Target Mesh.
	* Polygon should be oriented counter-clockwise to produce a correctly-oriented shape, otherwise it will be inside-out
	* Polygon endpoint is not repeated.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendTriangulatedPolygon( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& PolygonVertices,
		bool bAllowSelfIntersections = true,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Appends a Triangulated Polygon (with vertices specified in 3D) to the Target Mesh.
	* Uses Ear Clipping-based triangulation. Output vertices will always be 1:1 with input vertices.
	* Polygon endpoint is not repeated.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendTriangulatedPolygon3D(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector>& PolygonVertices3D,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a linear staircase to the Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendLinearStairs( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float StepWidth = 100.0f,
		float StepHeight = 20.0f,
		float StepDepth = 30.0f,
		int NumSteps = 8,
		bool bFloating = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Appends a rising circular staircase to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendCurvedStairs( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float StepWidth = 100.0f,
		float StepHeight = 20.0f,
		float InnerRadius = 150.0f,
		float CurveAngle = 90.0f,
		int NumSteps = 8,
		bool bFloating = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Generates triangulated Voronoi Cells from the provided Voronoi Sites, identifying each with PolyGroups, and appends to the Target Mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendVoronoiDiagram2D(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& VoronoiSites,
		FGeometryScriptVoronoiOptions VoronoiOptions,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Generates a Delaunay Triangulation of the provided vertices, and appends it to the Target Mesh.
	* If optional Constrained Edges are provided, will generate a Constrained Delaunay Triangulation which connects the specified vertices with edges.
	* On success, all vertices are always appended to the output mesh, though duplicate vertices will not be used in any triangles and may optionally be removed.
	* Use PositionsToVertexIDs to map indices in the input VertexPositions array to vertex IDs in the Dynamic Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (ScriptMethod, HidePin = "Debug", AutoCreateRefTerm = "ConstrainedEdges"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendDelaunayTriangulation2D(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const TArray<FVector2D>& VertexPositions,
		const TArray<FIntPoint>& ConstrainedEdges,
		FGeometryScriptConstrainedDelaunayTriangulationOptions TriangulationOptions,
		TArray<int32>& PositionsToVertexIDs,
		bool& bHasDuplicateVertices,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Intended for use with AppendDelaunayTriangulation2D:
	 * Create a loop of edges through sequential vertices
	 * e.g., a Loop(3,0) will construct edges (2,0), (0,1) and (1,2)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives")
	static UE_API TArray<FIntPoint> CreateConstrainedEdgesLoop(int32 NumVertices, int32 Start = 0);

	/**
	 * Intended for use with AppendDelaunayTriangulation2D:
	 * Create a chain of edges through sequential vertices
	 * e.g., a Chain(3,0) will construct edges (0,1) and (1,2)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives")
	static UE_API TArray<FIntPoint> CreateConstrainedEdgesChain(int32 NumVertices, int32 Start = 0);


	/**
	* Generates a Delaunay Triangulation of the provided Polygon List, and appends it to the Target Mesh.
	* @param bTriangulationError Reports whether the triangulation contains errors, typically due to intersecting edges in the input. Consider pre-processing the PolygonsList with PolygonsUnion to resolve intersections and prevent this error.
	* @param bStillAppendOnError Whether to still append a best-effort triangulation in error cases. Often this will be a triangulation that does not quite match the polygon shape near intersecting edges in the input, but otherwise is as-expected.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendPolygonListTriangulation(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		FGeometryScriptGeneralPolygonList PolygonList,
		FGeometryScriptPolygonsTriangulationOptions TriangulationOptions,
		bool& bTriangulationError,
		UGeometryScriptDebug* Debug = nullptr);

	
	/**
	 * Appends Simple Collision shapes to the Target Mesh, triangulated as specified by Triangulation Options
	 * @param Transform Transform to be applied to simple collision shapes, following the method by which simple collision shapes are transformed -- so, e.g., spheres will not be non-uniformly scaled
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Append Simple Collision Shapes to Mesh", Category = "GeometryScript|Primitives", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendSimpleCollisionShapes(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const FGeometryScriptSimpleCollision& SimpleCollision,
		FGeometryScriptSimpleCollisionTriangulationOptions TriangulationOptions,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	 * Appends the spheres in the Sphere Covering to the Target Mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Primitives", meta = (HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	AppendSphereCovering(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		const FGeometryScriptSphereCovering& SphereCovering,
		int32 StepsX = 6,
		int32 StepsY = 6,
		int32 StepsZ = 6,
		UGeometryScriptDebug* Debug = nullptr
	);


	//---------------------------------------------
	// Backwards-Compatibility implementations
	//---------------------------------------------
	// These are versions/variants of the above functions that were released
	// in previous UE 5.x versions, that have since been updated. 
	// To avoid breaking user scripts, these previous versions are currently kept and 
	// called via redirectors registered in GeometryScriptingCoreModule.cpp.
	// 
	// These functions may be deprecated in future UE releases.
	//


	/**
	 * 5.0 Preview 1 Compatibility version of AppendRectangleXY. Incorrectly interprets the input dimensions.
	 * Incorrectly divides the input DimensionX and DimensionY by 2.
	 * @warning It is strongly recommended that callers of this function update to the current AppendRectangleXY function!
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Compatibility", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRectangle_Compatibility_5_0( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		int32 StepsWidth = 0,
		int32 StepsHeight = 0,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * 5.0 Preview 1 Compatibility version of AppendRoundRectangleXY. 
	 * Incorrectly divides the input DimensionX and DimensionY by 2.
	 * @warning It is strongly recommended that callers of this function update to the current AppendRoundRectangleXY function!
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Compatibility", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AppendRoundRectangle_Compatibility_5_0( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPrimitiveOptions PrimitiveOptions,
		FTransform Transform,
		float DimensionX = 100,
		float DimensionY = 100,
		float CornerRadius = 5,
		int32 StepsWidth = 0,
		int32 StepsHeight = 0,
		int32 StepsRound = 4,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
