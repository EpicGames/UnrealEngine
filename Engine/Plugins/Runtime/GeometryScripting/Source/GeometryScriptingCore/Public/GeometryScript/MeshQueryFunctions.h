// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshQueryFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshQueries"))
class UGeometryScriptLibrary_MeshQueryFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Returns information about the Target Mesh, such as the vertex and triangle count as well as some attribute information.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API FString GetMeshInfoString( UDynamicMesh* TargetMesh );

	/**
	* Returns true if the mesh is dense. For example, no gaps in Vertex IDs or Triangle IDs.
	* Note if a mesh is not dense, the Compact Mesh node can be used to removed the gaps.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool GetIsDenseMesh( UDynamicMesh* TargetMesh );

	/**
	* Returns true if the Target Mesh has attributes enabled.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool GetMeshHasAttributeSet( UDynamicMesh* TargetMesh );

	/**
	* Computes the bounding box of the mesh vertices in the local space of the mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Bounding Box") FBox GetMeshBoundingBox( UDynamicMesh* TargetMesh );

	/**
	* Computes the volume and area of the mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API void GetMeshVolumeArea( UDynamicMesh* TargetMesh, float& SurfaceArea, float& Volume );

	/**
	 * Computes the volume, area and center-of-mass of the mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod))
	static UE_API void GetMeshVolumeAreaCenter(UDynamicMesh* TargetMesh, float& SurfaceArea, float& Volume, FVector& CenterOfMass);

	/**
	* Returns true if the mesh is closed, such as no topological boundary edges.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool GetIsClosedMesh( UDynamicMesh* TargetMesh );

	/**
	* Returns the number of open border loops, such as "holes" in the mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Num Loops") int32 GetNumOpenBorderLoops( UDynamicMesh* TargetMesh, bool& bAmbiguousTopologyFound );

	/**
	* Returns the number of topological boundary edges in the mesh, i.e counts edges that only have one adjacent triangle.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Num Edges") int32 GetNumOpenBorderEdges( UDynamicMesh* TargetMesh );

	/**
	* Returns the number of separate connected islands (components) in the mesh, such as "triangle patches" internally connected by shared edges.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, DisplayName = "Get Num Connected Islands", KeyWords = "Components Parts"))
	static UE_API UPARAM(DisplayName = "Num Islands") int32 GetNumConnectedComponents( UDynamicMesh* TargetMesh );

	// UDynamicMesh already has this function
	//UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	//static UPARAM(DisplayName = "Triangle Count") int32 GetTriangleCount( UDynamicMesh* TargetMesh );

	/**
	* Gets the number of Triangle IDs in the mesh. This may be larger than the Triangle Count if the mesh is not dense, even after deleting triangles.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Num Triangle IDs") int32 GetNumTriangleIDs( UDynamicMesh* TargetMesh );

	/**
	* Returns true if there are gaps in the Triangle ID list, such that Get Num Triangle IDs is greater than Get Triangle Count.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool GetHasTriangleIDGaps( UDynamicMesh* TargetMesh );

	/**
	* Returns true if Triangle ID refers to a valid Triangle in the Target Mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool IsValidTriangleID( UDynamicMesh* TargetMesh, int32 TriangleID );

	/**
	* Returns an Index List of all Triangle IDs in a mesh. 
	* @param bHasTriangleIDGaps will be true on return if there are breaks in the sequential numeration of Triangle IDs, as would happen after deleting triangles.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllTriangleIDs( UDynamicMesh* TargetMesh, FGeometryScriptIndexList& TriangleIDList, bool& bHasTriangleIDGaps );

	/**
	* Returns the Vertex ID triplet for the specified Triangle.
	* @param TriangleID indicates the triangle to query on the Target Mesh.
	* @param bIsValidTriangle will be false on return if the Triangle ID does not exist in the Target Mesh, in that case the returned vertex triplet will be (-1, -1, -1).
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Indices") FIntVector GetTriangleIndices( UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle );

	/*
	* Returns a TriangleList of all Triangle Vertex ID triplets in a mesh.
	* @param bSkipGaps if false there will be a one-to-one correspondence between Triangle ID and entries in the triangle list and invalid triplets of (-1,-1,-1) will correspond to Triangle IDs not found in the Target Mesh.
	* @param bHasTriangleIDGaps will be false on return if the mesh had no gaps in Triangle IDs or if bSkipGaps was set to true.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllTriangleIndices( UDynamicMesh* TargetMesh, FGeometryScriptTriangleList& TriangleList, bool bSkipGaps, bool& bHasTriangleIDGaps);

	/*
	* Returns the 3D positions (in the mesh local space) of the three vertices of the requested triangle.
	* If the Triangle ID is not an element of the Target Mesh, all three vertices will be returned as (0, 0, 0) and bIsValidTriangle will be set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API void GetTrianglePositions( UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle, FVector& Vertex1, FVector& Vertex2, FVector& Vertex3 );

	/**
	 * Compute the interpolated Position (A*Vertex1 + B*Vertex2 + C*Vertex3), where (A,B,C)=BarycentricCoords and the Vertex positions are taken
	 * from the specified TriangleID of the TargetMesh. 
	 * @param bIsValidTriangle will be returned true if TriangleID exists in TargetMesh, and otherwise will be returned false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetInterpolatedTrianglePosition( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FVector BarycentricCoords, 
		bool& bIsValidTriangle,
		FVector& InterpolatedPosition );


	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Normal") FVector GetTriangleFaceNormal( UDynamicMesh* TargetMesh, int32 TriangleID, bool& bIsValidTriangle );

	/**
	 * Compute the barycentric coordinates (A,B,C) of the Point relative to the specified TriangleID of the TargetMesh.
	 * The properties of barycentric coordinates are such that A,B,C are all positive, A+B+C=1.0, and A*Vertex1 + B*Vertex2 + C*Vertex3 = Point.
	 * So, the barycentric coordinates can be used to smoothly interpolate/blend any other per-triangle-vertex quantities.
	 * The Point must lie in the plane of the Triangle, otherwise the coordinates are somewhat meaningless (but clamped to 0-1 range to avoid catastrophic errors)
	 * The Positions of the Triangle Vertices are also returned for convenience (similar to GetTrianglePositions)
	 * @param bIsValidTriangle will be returned true if TriangleID exists in TargetMesh, and otherwise will be returned false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeTriangleBarycentricCoords( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		bool& bIsValidTriangle, 
		FVector Point, 
		FVector& Vertex1, 
		FVector& Vertex2, 
		FVector& Vertex3, 
		FVector& BarycentricCoords );


	/**
	* Gets the number of vertices in the mesh. Note this may be less than the number of Vertex IDs used as some vertices may have been deleted.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Vertex Count") int32 GetVertexCount( UDynamicMesh* TargetMesh );

	/**
	* Gets the number of Vertex IDs in the mesh, which may be larger than the Vertex Count, if the mesh is not dense (e.g.  after deleting vertices).
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Num Vertex IDs") int32 GetNumVertexIDs( UDynamicMesh* TargetMesh );

	/**
	* Returns true if there are gaps in the Vertex ID list. For example, Get Number of Vertex IDs is greater than Get Vertex Count.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool GetHasVertexIDGaps( UDynamicMesh* TargetMesh );

	/**
	* Returns true if Vertex ID refers to a valid vertex in the Target Mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API bool IsValidVertexID( UDynamicMesh* TargetMesh, int32 VertexID );

	/**
	* Returns an IndexList of all Vertex IDs in mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllVertexIDs( UDynamicMesh* TargetMesh, FGeometryScriptIndexList& VertexIDList, bool& bHasVertexIDGaps );

	/**
	* Gets the 3D position of a mesh vertex in the mesh local space, by Vertex ID.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Position") FVector GetVertexPosition( UDynamicMesh* TargetMesh, int32 VertexID, bool& bIsValidVertex );

	/**
	* Returns a Vector List of all the mesh vertex 3D positions (possibly large!).
	* @param bSkipGaps if false there will be a one-to-one correspondence between Vertex ID and entries in the Position List
	* where a zero vector (0,0,0) will correspond to Vertex IDs not found in the Target Mesh.
	* @param bHasVertexIDGaps will be false if the mesh had no gaps in Vertex IDs or if bSkipGaps was set to true.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllVertexPositions( UDynamicMesh* TargetMesh, FGeometryScriptVectorList& PositionList, bool bSkipGaps, bool& bHasVertexIDGaps );

	/**
	 * Returns the vertex positions for each edge in the given index list.
	 * 
	 * @param TargetMesh The mesh to query
	 * @param EdgeIDs The edge IDs to query
	 * @param Start The output list of start vertex positions
	 * @param End The output list of end vertex positions
	 * @return The target mesh that was queried
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	GetAllVertexPositionsAtEdges( UDynamicMesh* TargetMesh, const FGeometryScriptIndexList& EdgeIDs, FGeometryScriptVectorList& Start, FGeometryScriptVectorList& End);

	/**
	 * Return array of Triangle IDs connected to the given VertexID, ie the triangle one-ring
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetVertexConnectedTriangles( UDynamicMesh* TargetMesh, int32 VertexID, TArray<int32>& Triangles);

	/**
	 * Return array of Vertex IDs connected via a mesh edge to the given VertexID, ie the vertex one-ring
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetVertexConnectedVertices( UDynamicMesh* TargetMesh, int32 VertexID, TArray<int32>& Vertices);

	//
	// UV Queries
	//

	/**
	* Gets the number of UV Channels on the Target Mesh.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod), DisplayName = "Get Num UV Channels")
	static UE_API UPARAM(DisplayName = "Num UV Channels") int32 GetNumUVSets( UDynamicMesh* TargetMesh );

	/**
	* Gets the 2D bounding box of all UVs in the UV Channel.  If the UV Channel does not exist, or if the UV Channel is empty, the resulting box will be invalid.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Bounding Box") FBox2D GetUVSetBoundingBox( UDynamicMesh* TargetMesh, 
																			UPARAM(DisplayName = "UV Channel") int UVSetIndex, 
																			UPARAM(DisplayName = "Is Valid UV Channel") bool& bIsValidUVSet, 
																			UPARAM(DisplayName = "UV Channel Is Empty") bool& bUVSetIsEmpty );

	/**
	 * Gets the area of triangles in UV space for the given UV Channel.
	 * 
	 * @param TargetMesh The mesh to query.
	 * @param UVChannel The UV channel to query
	 * @param bIsValidUVChannel True, if the mesh has UVs for the given UVSetIndex.
	 * @return The number of UV islands
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod), DisplayName = "Get Mesh UV Area")
	static UE_API UPARAM(DisplayName = "UV Area") double GetMeshUVArea(UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "UV Channel") int UVChannel,
		UPARAM(DisplayName = "Is Valid UV Channel") bool& bIsValidUVChannel);

	/**
	* Returns the UV values associated with the three vertices of the triangle in the specified UV Channel.
	* If the Triangle does not exist in the mesh or if no UVs are set in the specified UV Channel for the triangle, the resulting values will be (0,0) and bHaveValidUVs will be set to false.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API void GetTriangleUVs( UDynamicMesh* TargetMesh, UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, int32 TriangleID, FVector2D& UV1, FVector2D& UV2, FVector2D& UV3, bool& bHaveValidUVs );


	/**
	* Returns the unique UV element IDs and values associated with the mesh vertex, in the specified UV Channel.
	* If the Vertex or UV channel does not exist, the arrays will be empty and bHaveValidUVs will be set to false.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetAllSplitUVsAtVertex( UDynamicMesh* TargetMesh, UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, int32 VertexID, TArray<int32>& ElementIDs, TArray<FVector2D>& ElementUVs, bool& bHaveValidUVs);

	/**
	 * Compute the interpolated UV (A*UV1+ B*UV2+ C*UV3), where (A,B,C)=BarycentricCoords and the UV positions are taken
	 * from the specified TriangleID in the specified UVSet of the TargetMesh. 
	 * @param bIsValidTriangle bTriHasValidUVs be returned true if TriangleID exists in TargetMesh and is set to valid UVs in the UV Set, and otherwise will be returned false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetInterpolatedTriangleUV( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, 
		int32 TriangleID, 
		FVector BarycentricCoords, 
		bool& bTriHasValidUVs, 
		FVector2D& InterpolatedUV );


	/**
	 * Returns all edge element IDs that are UV seam edges for a given UV channel.
	 * 
	 * @param TargetMesh The mesh to query.
	 * @param UVSetIndex The UV channel to query
	 * @param bHaveValidUVs True, if the mesh has UVs for the given UVSetIndex.
	 * @param ElementIDs The returned edge element IDs 
	 * @return The target mesh that was queried. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	GetAllUVSeamEdges( UDynamicMesh* TargetMesh, UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, bool& bHaveValidUVs, FGeometryScriptIndexList& ElementIDs);

	/**
	 * Returns the number of UV islands in a given UV channel.
	 *
	 * @param TargetMesh The mesh to query.
	 * @param UVChannel The UV channel to query
	 * @param bIsValidUVChannel True, if the mesh has UVs for the given UVSetIndex.
	 * @return The number of UV islands
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod), DisplayName = "Get Num UV Islands")
	static UE_API UPARAM(DisplayName = "Num UV Islands") int32 GetNumUVIslands(
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int32 UVChannel, 
		UPARAM(DisplayName = "Is Valid UV Channel") bool& bIsValidUVChannel);

	
	//
	// Normal queries
	//

	/**
	 * @return true if the TargetMesh has the Normals Attribute enabled (which allows for storing split normals)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Enabled") bool GetHasTriangleNormals( UDynamicMesh* TargetMesh );

	/**
	 * For the specified TriangleID of the Target Mesh, get the Normal vectors at each vertex of the Triangle.
	 * These Normals will be taken from the Normal Overlay, i.e. they will potentially be split-normals.
	 * @param bTriHasValidNormals will be returned true if TriangleID exists in TargetMesh and has Normals set.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetTriangleNormals( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FVector& Normal1, 
		FVector& Normal2, 
		FVector& Normal3, 
		bool& bTriHasValidNormals );

	/**
	 * Compute the interpolated Normal (A*Normal1 + B*Normal2 + C*Normal3), where (A,B,C)=BarycentricCoords and the Normals are taken
	 * from the specified TriangleID in the Normal layer of the TargetMesh. 
	 * @param bTriHasValidNormals will be returned true if TriangleID exists in TargetMesh and has Normals set, and otherwise will be returned false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetInterpolatedTriangleNormal( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FVector BarycentricCoords, 
		bool& bTriHasValidNormals,
		FVector& InterpolatedNormal );

	/**
	 * For the specified Triangle ID of the TargetMesh, get the Normal and Tangent vectors at each vertex of the Triangle.
	 * These Normals/Tangents will be taken from the Normal and Tangents Overlays, i.e. they will potentially be split-normals.
	 * @param bTriHasValidElements will be returned true if TriangleID exists in TargetMesh and has Normals and Tangents set.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetTriangleNormalTangents( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		bool& bTriHasValidElements,
		FGeometryScriptTriangle& Normals,
		FGeometryScriptTriangle& Tangents,
		FGeometryScriptTriangle& BiTangents);

	/**
	 * Compute the interpolated Normal and Tangents for the specified specified TriangleID in the Normal and Tangent attributes of the TargetMesh. 
	 * @param bTriHasValidElements will be returned true if TriangleID exists in TargetMesh and has Normals and Tangents set, and otherwise will be returned false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetInterpolatedTriangleNormalTangents( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FVector BarycentricCoords, 
		bool& bTriHasValidElements,
		FVector& InterpolatedNormal, 
		FVector& InterpolatedTangent,
		FVector& InterpolatedBiTangent);


	//
	// Vertex Color Queries
	//


	/**
	 * @return true if the TargetMesh has the Vertex Colors attribute enabled
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Enabled") bool GetHasVertexColors( UDynamicMesh* TargetMesh );

	/**
	 * For the specified TriangleID of the TargetMesh, get the Vertex Colors at each vertex of the Triangle.
	 * These Colors will be taken from the Vertex Color Attribute, ie they will potentially be split-colors.
	 * @param bTriHasValidVertexColors will be returned true if TriangleID exists in TargetMesh and has Vertex Colors set
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetTriangleVertexColors( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FLinearColor& Color1, 
		FLinearColor& Color2, 
		FLinearColor& Color3, 
		bool& bTriHasValidVertexColors );

	/**
	 * Compute the interpolated Vertex Color (A*Color1 + B*Color2 + C*Color3), where (A,B,C)=BarycentricCoords and the Colors are taken
	 * from the specified TriangleID in the Vertex Color layer of the TargetMesh. 
	 * @param bTriHasValidVertexColors will be returned true if TriangleID exists in TargetMesh and has Colors set, and otherwise will be returned false
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetInterpolatedTriangleVertexColor( 
		UDynamicMesh* TargetMesh, 
		int32 TriangleID, 
		FVector BarycentricCoords, 
		FLinearColor DefaultColor,
		bool& bTriHasValidVertexColors,
		FLinearColor& InterpolatedColor );


	//
	// Material Queries
	//

	/**
	 * Returns true if the mesh has Material IDs available/enabled.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Enabled") bool GetHasMaterialIDs( UDynamicMesh* TargetMesh );


	//
	// Polygroup Queries
	//

	/**
	* Returns true if the mesh has a standard PolyGroup Layer.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod, DisplayName = "Get Has PolyGroups"))
	static UE_API UPARAM(DisplayName = "Enabled") bool GetHasPolygroups( UDynamicMesh* TargetMesh );

	/**
	* Returns the count of extended PolyGroup Layers.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|MeshQueries", meta=(ScriptMethod, DisplayName = "Get Num Extended PolyGroup Layers"))
	static UE_API UPARAM(DisplayName = "Num Layers") int GetNumExtendedPolygroupLayers( UDynamicMesh* TargetMesh );


	//
	// Weight Map Attribute Layer Queries
	//

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, DisplayName = "Get Weight Map Handle By Name", ExpandEnumAsExecs = "FoundWeights"))
	static UE_API UPARAM(DisplayName = "Weight Map Handle") FGeometryScriptWeightMapHandle GetWeightMapHandle(UDynamicMesh* TargetMesh, FName Name, EGeometryScriptSearchOutcomePins& FoundWeights);

};

#undef UE_API
