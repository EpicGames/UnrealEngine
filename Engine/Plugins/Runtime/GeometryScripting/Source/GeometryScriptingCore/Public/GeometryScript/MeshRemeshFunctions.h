// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshRemeshFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


/** Goal types for Uniform Remeshing */
UENUM(BlueprintType)
enum class EGeometryScriptUniformRemeshTargetType : uint8
{
	/** Approximate Desired Triangle Count. This is used to compute a Target Edge Length, and is not an explicit target */
	TriangleCount = 0,
	/** Attempt to Remesh such that all edges have approximately this length */
	TargetEdgeLength = 1
};

/** Types of edge constraints, specified for different mesh attributes */
UENUM(BlueprintType)
enum class EGeometryScriptRemeshEdgeConstraintType : uint8
{
	/** Constrained edges cannot be flipped, split or collapsed, and vertices will not move */
	Fixed = 0,
	/** Constrained edges can be split, but not flipped or collapsed. Vertices will not move. */
	Refine = 1,
	/** Constrained edges cannot be flipped, but otherwise are free to move */
	Free = 2,
	/** Edges are not constrained, ie the Attribute used to derive the Constraints will not be considered */
	Ignore = 3
};

/** The Vertex Smoothing strategy used in a Remeshing operation */
UENUM(BlueprintType)
enum class EGeometryScriptRemeshSmoothingType : uint8
{
	/** Vertices move towards their 3D one-ring centroids, UVs are ignored. This produces the most regular mesh possible. */
	Uniform = 0,
	/** Vertices move towards the projection of their one-ring centroids onto their normal vectors, preserving UVs  */
	UVPreserving = 1,
	/** Similar to UV Preserving, but allows some tangential drift (causing UV distortion) when vertices would otherwise be "stuck" */
	Mixed = 2
};


/**
 * Standard Remeshing Options
 */
USTRUCT(Blueprintable)
struct FGeometryScriptRemeshOptions
{
	GENERATED_BODY()
public:

	/** When enabled, all mesh attributes are discarded, so UV and Normal Seams can be ignored. New per-vertex normals are computed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDiscardAttributes = false;

	/** When enabled, mesh vertices are projected back onto the input mesh surface during Remeshing, preserving the shape */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bReprojectToInputMesh = true;

	/** Type of 3D Mesh Smoothing to apply during Remeshing. Disable by setting SmoothingRate = 0 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemeshSmoothingType SmoothingType = EGeometryScriptRemeshSmoothingType::Mixed;

	/** Smoothing Rate/Speed. Faster Smoothing results in a more regular mesh, but also more potential for undesirable 3D shape change and UV distortion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1))
	float SmoothingRate = 0.25f;

	/** Constraints on the open mesh boundary/border edges */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemeshEdgeConstraintType MeshBoundaryConstraint = EGeometryScriptRemeshEdgeConstraintType::Free;

	/** Constraints on the mesh boundary/border edges between different PolyGroups of the Mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemeshEdgeConstraintType GroupBoundaryConstraint = EGeometryScriptRemeshEdgeConstraintType::Free;

	/** Constraints on the mesh boundary/border edges between different Material Results of the Mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemeshEdgeConstraintType MaterialBoundaryConstraint = EGeometryScriptRemeshEdgeConstraintType::Free;

	/** Enable/Disable Edge Flips during Remeshing. Disabling flips will significantly reduce the output mesh quality */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowFlips = true;

	/** Enable/Disable Edge Splits during Remeshing. Disabling Splits will prevent the mesh density from increasing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowSplits = true;

	/** Enable/Disable Edge Collapses during Remeshing. Disabling Collapses will prevent the mesh density from decreasing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowCollapses = true;

	/** When Enabled, Flips and Collapses will be skipped if they would flip any triangle face normals */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreventNormalFlips = true;

	/** When Enabled, Flips and Collapses will be skipped if they would create tiny degenerate triangles */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreventTinyTriangles = true;

	/** By default, remeshing is accelerated by tracking a queue of edges that need to be processed. This is signficantly faster but can produce a lower quality output. Enable this option to use a more expensive strategy that guarantees maximum quality.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUseFullRemeshPasses = false;

	/** Maximum Number of iterations of the Remeshing Strategy to apply to the Mesh. More iterations are generally more expensive (much moreso with bUseFullRemeshPasses = true) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, ClampMin = 0))
	int32 RemeshIterations = 20;

	/** If enabled, the output mesh is automatically compacted to remove gaps in the index space. This is expensive and can be disabled by advanced users. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoCompact = true;
};


/**
 * Uniform Remeshing Options
 */
USTRUCT(Blueprintable)
struct FGeometryScriptUniformRemeshOptions
{
	GENERATED_BODY()
public:
	/** Method used to define target/goal of Uniform Remeshing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptUniformRemeshTargetType TargetType = EGeometryScriptUniformRemeshTargetType::TriangleCount;

	/** Approximate Target Triangle Count, combined with mesh surface area to derive a TargetEdgeLength */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, ClampMin = 0, EditCondition = "TargetType == EGeometryScriptUniformRemeshTargetType::TriangleCount"))
	int32 TargetTriangleCount = 5000;

	/** Explicit Target Edge Length that is desired in the output uniform mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, ClampMin = 0, EditCondition = "TargetType == EGeometryScriptUniformRemeshTargetType::TargetEdgeLength"))
	float TargetEdgeLength = 1.0f;
};

/**
 * Adaptive Remeshing Options
 */
USTRUCT(Blueprintable)
struct FGeometryScriptAdaptiveRemeshOptions
{
	GENERATED_BODY()
public:
	/** Method used to define target/goal of Adaptive Remeshing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptUniformRemeshTargetType TargetType = EGeometryScriptUniformRemeshTargetType::TriangleCount;

	/** Approximate Target Triangle Count, combined with mesh surface area to derive a TargetEdgeLength */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, ClampMin = 0, EditCondition = "TargetType == EGeometryScriptUniformRemeshTargetType::TriangleCount"))
	int32 TargetTriangleCount = 5000;

	/** Explicit Target Edge Length that is desired in the output uniform mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, ClampMin = 0, EditCondition = "TargetType == EGeometryScriptUniformRemeshTargetType::TargetEdgeLength"))
	float TargetEdgeLength = 1.0f;

	/** Amount to adjust density in regions where the vertex weight map is greater than zero. A positive value here results in a denser mesh, a negative value results in a coarser one */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = "-2.0", UIMax = "2.0"))
	float RelativeDensity = 0.f;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Remeshing"))
class UGeometryScriptLibrary_RemeshingFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Apply Uniform Remeshing to the TargetMesh. 
	 * @warning this function can be quite expensive. The results may be non-deterministic, and are expected to change in future versions.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyUniformRemesh(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptRemeshOptions RemeshOptions,
		FGeometryScriptUniformRemeshOptions UniformOptions,
		UGeometryScriptDebug* Debug = nullptr);

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyAdaptiveRemesh(
		UDynamicMesh* TargetMesh,
		FGeometryScriptRemeshOptions RemeshOptions,
		FGeometryScriptAdaptiveRemeshOptions AdaptiveOptions,
		FGeometryScriptWeightMapHandle WeightMapHandle,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
