// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshRepairFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

USTRUCT(BlueprintType)
struct FGeometryScriptWeldEdgesOptions
{
	GENERATED_BODY()
public:
	/** Edges are coincident if both pairs of endpoint vertices, and their midpoint, are closer than this distance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Tolerance = 1e-06f;

	/** Only merge unambiguous pairs that have unique duplicate-edge matches */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOnlyUniquePairs = true;
};



USTRUCT(BlueprintType)
struct FGeometryScriptResolveTJunctionOptions
{
	GENERATED_BODY()
public:
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Tolerance = 1e-03f;
};

USTRUCT(BlueprintType)
struct FGeometryScriptSnapBoundariesOptions
{
	GENERATED_BODY()
public:
	/** Snapping tolerance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Tolerance = 1e-03f;

	/** Whether to snap vertices to open edges. If false, will only snap together vertices */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSnapToEdges = true;

	/** Maximum number of iterations of boundary snapping to apply. Will stop earlier if an iteration applies no snapping. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxIterations = 5;
};

UENUM(BlueprintType)
enum class EGeometryScriptFillHolesMethod : uint8
{
	Automatic = 0,
	MinimalFill = 1,
	PolygonTriangulation = 2,
	TriangleFan = 3,
	PlanarProjection = 4
};

USTRUCT(BlueprintType)
struct FGeometryScriptFillHolesOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptFillHolesMethod FillMethod = EGeometryScriptFillHolesMethod::Automatic;

	/** Delete floating, disconnected triangles, as they produce a "hole" that cannot be filled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDeleteIsolatedTriangles = true;
};



USTRUCT(BlueprintType)
struct FGeometryScriptRemoveSmallComponentOptions
{
	GENERATED_BODY()
public:
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MinVolume = 0.0001f;

	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MinArea = 0.0001f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MinTriangleCount = 1;
};




UENUM(BlueprintType)
enum class EGeometryScriptRemoveHiddenTrianglesMethod : uint8
{
	FastWindingNumber = 0,
	RaycastOcclusionTest = 1
};


USTRUCT(BlueprintType)
struct FGeometryScriptRemoveHiddenTrianglesOptions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemoveHiddenTrianglesMethod Method = EGeometryScriptRemoveHiddenTrianglesMethod::FastWindingNumber;

	// add triangle samples per triangle (in addition to TriangleSamplingMethod)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SamplesPerTriangle = 0;

	// once triangles to remove are identified, do iterations of boundary erosion, ie contract selection by boundary vertex one-rings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int ShrinkSelection = 0;

	// use this as winding isovalue for WindingNumber mode
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float WindingIsoValue = 0.5;

	// random rays to add beyond +/- major axes, for raycast sampling
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int RaysPerSample = 0;


	/** Nudge sample points out by this amount to try to counteract numerical issues */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float NormalOffset = 1e-6f;

	/** Whether to compact the resulting mesh after triangles are deleted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCompactResult = true;
};

USTRUCT(BlueprintType)
struct FGeometryScriptSelectHiddenTrianglesFromOutsideOptions
{
	GENERATED_BODY()
public:

	// Approximate spacing between samples on triangle faces used for determining visibility
	// Set a smaller value to increase the number of visibility samples per triangle, which can give more accurate visibility results
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double SampleSpacing = 1.0;

	// Whether to treat faces as double-sided when determining visibility
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDoubleSided = false;

	// Number of directions to test for visibility. Note: RestrictToViewDirections will filter the view directions after they are sampled from a full sphere, so the actual number of sampled directions will be smaller if restricted.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 NumSearchDirections = 128;

	// Whether to decide visibility on a per-polygroup basis, rather than per-triangle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPerPolyGroup = false;

	// If not empty, only allow views along these directions (or a direction within the ViewDirectionAngleRange cone of these directions)
	// To be specific: A mesh sample P is visible from given direction D if a ray starting at P-D*(Mesh Diameter), cast in direction D, reaches P
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> RestrictToViewDirections;

	// Field of view to use for the RestrictViewDirections (if any), in degrees
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double ViewDirectionAngleRange = 45;
};


UENUM(BlueprintType)
enum class EGeometryScriptRepairMeshMode : uint8
{
	DeleteOnly = 0,
	RepairOrDelete = 1,
	RepairOrSkip = 2
};



USTRUCT(BlueprintType)
struct FGeometryScriptDegenerateTriangleOptions
{
	GENERATED_BODY()
public:
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRepairMeshMode Mode = EGeometryScriptRepairMeshMode::RepairOrDelete;

	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double MinTriangleArea = 0.001;

	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double MinEdgeLength = 0.0001;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bCompactOnCompletion = true;
};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshRepair"))
class UGeometryScriptLibrary_MeshRepairFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Compacts the mesh's vertices and triangles to remove any "holes" in the Vertex ID or Triangle ID lists.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CompactMesh(  
		UDynamicMesh* TargetMesh, 
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Remove vertices that are not used by any triangles. Note: Does not update the IDs of any remaining vertices; use CompactMesh to do so.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	RemoveUnusedVertices(
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Snap vertices on open edges to the closest compatible open boundary, if found within the tolerance distance
	 * Unlike ResolveMeshTJunctions, does not introduce new vertices to the mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SnapMeshOpenBoundaries(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSnapBoundariesOptions SnapOptions,
		UGeometryScriptDebug* Debug = nullptr
	);

	/**
	* Attempts to resolve T-Junctions in the mesh by addition of vertices and welding.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ResolveMeshTJunctions(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptResolveTJunctionOptions ResolveOptions,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Welds any open boundary edges of the mesh together if possible in order to remove "cracks."
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	WeldMeshEdges(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptWeldEdgesOptions WeldOptions,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Tries to fill all open boundary loops (such as holes in the geometry surface) of a mesh.
	* @param FillOptions specifies the method used to fill the holes.
	* @param NumFilledHoles reports the number of holes filled by the function.
	* @param NumFailedHolesFills reports the detected holes that were unable to be filled.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	FillAllMeshHoles(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptFillHolesOptions FillOptions,
		int32& NumFilledHoles,
		int32& NumFailedHoleFills,
		UGeometryScriptDebug* Debug = nullptr);

	/*
	* Removes connected islands (components) of the mesh that have a volume, area, or triangle count below a threshold as specified by the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug",
		DisplayName = "Remove Small Connected Islands", KeyWords = "Delete Components Parts"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RemoveSmallComponents(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptRemoveSmallComponentOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Removes any triangles in the mesh that are not visible from the exterior view, under various definitions of "visible" and "outside"
	* as specified by the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug", KeyWords = "Jacket Occluded"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RemoveHiddenTriangles(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptRemoveHiddenTrianglesOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/*
	* Select mesh triangles hidden from outside views, with optional filtering on view directions
	*
	* @param IgnoreSelection Optionally indicate parts of the mesh that should not be included in the ResultSelection, so do not need to be tested for occlusion
	* @param TransparentSelection Optionally indicate parts of the mesh that should not be considered occluding, e.g. transparent triangles
	* @param ResultSelection The fully occluded triangles of the target mesh (based on the options / transparencies, and excluding any IgnoreSelection triangles)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta = (ScriptMethod, HidePin = "Debug", KeyWords = "Remove Jacket Occluded"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectHiddenTrianglesFromOutside(
		UDynamicMesh* TargetMesh, 
		FGeometryScriptSelectHiddenTrianglesFromOutsideOptions Options,
		FGeometryScriptMeshSelection IgnoreSelection,
		FGeometryScriptMeshSelection TransparentSelection,
		FGeometryScriptMeshSelection& ResultSelection,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Splits Bowties in the mesh and/or the attributes.  A Bowtie is formed when a single vertex is connected to more than two boundary edges, 
	* and splitting duplicates the shared vertex so each triangle will have a unique copy.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SplitMeshBowties(  
		UDynamicMesh* TargetMesh, 
		bool bMeshBowties = true,
		bool bAttributeBowties = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Removes triangles that have area or edge length below specified amounts depending on the Options requested.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Repair", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RepairMeshDegenerateGeometry(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptDegenerateTriangleOptions Options,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
