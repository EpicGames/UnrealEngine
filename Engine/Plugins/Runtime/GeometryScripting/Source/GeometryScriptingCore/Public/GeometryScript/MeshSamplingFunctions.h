// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshSamplingFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


USTRUCT(BlueprintType)
struct FGeometryScriptMeshPointSamplingOptions
{
	GENERATED_BODY()
public:
	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float SamplingRadius = 10.0;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxNumSamples = 0;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int RandomSeed = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double SubSampleDensity = 10;

	/**
	 * If < 0, the latest, recommended sampling methods will be used. Otherwise, requests a specific sampling method version.
	 * Set this for more consistent results across UE versions, at the risk of worse performance or quality.
	 * Valid versions are:
	 * Method 0: UE 5.5 and earlier sampling method. Slower initial (dense) point sampling, less robust to degenerate triangles.
	 * Method 1: Point downsampling is performed in a fixed order regardless of seed, which gives overly-similar sample patterns across multiple runs
	 * Method 2: Currently the default method.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, AdvancedDisplay)
	int SamplingMethodVersion = INDEX_NONE;
};

/** Control how active Weights are used to affect point radius */
UENUM(BlueprintType)
enum class EGeometryScriptSamplingWeightMode : uint8
{
	/** 
	 * Weights are clamped to [0,1] and used to interpolate Min/Max Radius. This is a "hard constraint", ie if the weight
	 * at a point is 1, only a "max radius" sample may be placed there, otherwise no samples at all (so no "filling in" smaller samples between large ones)
	 */
	WeightToRadius = 0,
	/** 
	 * Weights are clamped to [0,1] and used to interpolate Min/Max Radius, with decay, so that smaller-radius samples will infill between large ones.
	 * So areas with large weight may still end up with some variable-radius samples, but areas with 0 weight will only ever have min-radius samples.
	 */
	FilledWeightToRadius = 1,
	/** 
	 * Weight is used to create nonuniform random sampling, ie it nudges the random sample-radius distribution but does not directly control it.
	 * So samples with any radius can still appear at any location, but if weight=1 then max-radius samples are more likely, etc.
	 */
	WeightedRandom = 2
};

/** Controls the distribution of sample radii */
UENUM(BlueprintType)
enum class EGeometryScriptSamplingDistributionMode : uint8
{
	/** Uniform distribution of sizes, ie all equally likely */
	Uniform = 0,
	/** Distribution is weighted towards smaller points */
	Smaller = 1,
	/** Distribution is weighted towards larger points */
	Larger = 2
};


USTRUCT(BlueprintType)
struct FGeometryScriptNonUniformPointSamplingOptions
{
	GENERATED_BODY()
public:
	/** If MaxSampleRadius > SampleRadius, then output sample radius will be in range [SampleRadius, MaxSampleRadius] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MaxSamplingRadius = 0.0;

	/** SizeDistribution setting controls the distribution of sample radii */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptSamplingDistributionMode SizeDistribution = EGeometryScriptSamplingDistributionMode::Uniform;

	/** SizeDistributionPower is used to control how extreme the Size Distribution shift is. Valid range is [1,10] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (UIMin=1, UIMax=10, EditCondition ="(SizeDistribution != EGeometryScriptSamplingDistributionMode::Uniform)" ))
	double SizeDistributionPower = 2.0;

	/** WeightMode controls how any active Weight scheme is used to affect sample radius */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptSamplingWeightMode WeightMode = EGeometryScriptSamplingWeightMode::WeightedRandom;

	/** If true, weight values are inverted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bInvertWeights = false;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSampling"))
class UGeometryScriptLibrary_MeshSamplingFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Compute a 'uniform random' (not 'uniform spacing') point sampling over the mesh surface
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") const UDynamicMesh*
	ComputeUniformRandomPointSampling(
		const UDynamicMesh* TargetMesh,
		int32 NumSamples,
		TArray<FTransform>& Samples,
		FGeometryScriptIndexList& TriangleIDs,
		int RandomSeed = 0,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute a set of sample points lying on the surface of TargetMesh based on the provided sampling Options.
	 * Samples are approximately uniformly distributed, and non-overlapping relative to the provided Options.SamplingRadius,
	 * ie the distance between any pair of samples if >= 2*SamplingRadius.
	 * @param Samples output list of sample points. Transform Location is sample position, Rotation orients Z with the triangle normal
	 * @param TriangleIDs TriangleID that contains each sample point. Length is the same as Samples array.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputePointSampling( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshPointSamplingOptions Options,
		TArray<FTransform>& Samples,
		FGeometryScriptIndexList& TriangleIDs,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute a set of sample points lying on the surface of TargetMesh based on the provided sampling Options and NonUniformOptions.
	 * The sample points have radii in the range [Options.SamplingRadius, NonUniformOptions.MaxSamplingRadius], and
	 * are non-overlapping, ie the distance between two points is always larger than the sum of their respective radii.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeNonUniformPointSampling( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshPointSamplingOptions Options,
		FGeometryScriptNonUniformPointSamplingOptions NonUniformOptions,
		TArray<FTransform>& Samples,
		TArray<double>& SampleRadii,
		FGeometryScriptIndexList& TriangleIDs,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute a set of sample points lying on the surface of TargetMesh based on the provided sampling Options and NonUniformOptions.
	 * The sample points have radii in the range [Options.SamplingRadius, NonUniformOptions.MaxSamplingRadius], and
	 * are non-overlapping, ie the distance between two points is always larger than the sum of their respective radii.
	 * @param VertexWeights defines a per-vertex weight in range [0,1], these are interpolated to create a scalar field over the mesh triangles which is used to weight the sampling radii
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeVertexWeightedPointSampling( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptMeshPointSamplingOptions Options,
		FGeometryScriptNonUniformPointSamplingOptions NonUniformOptions,
		FGeometryScriptScalarList VertexWeights,
		TArray<FTransform>& Samples,
		TArray<double>& SampleRadii,
		FGeometryScriptIndexList& TriangleIDs,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute a set of Render Capture Cameras to capture a scene within the given Box
	 * @param Cameras Output Cameras with view frustums that contain the Box while maintaining the desired FOV
	 * @param Box     Bounding Box containing the scene to be captured
	 * @param Options Defines the Camera viewing directions into the box and other Camera parameters
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta = (HidePin = "Debug", AutoCreateRefTerm = "Options"))
	static UE_API void
	ComputeRenderCaptureCamerasForBox(
		TArray<FGeometryScriptRenderCaptureCamera>& Cameras,
		FBox Box,
		const FGeometryScriptRenderCaptureCamerasForBoxOptions& Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute oriented sample points on the visible surfaces of the given Actors
	 * The Samples are computed using Render Capture from the given virtual Cameras
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSampling", meta = (HidePin = "Debug"))
	static UE_API void
	ComputeRenderCapturePointSampling(
		TArray<FTransform>& Samples,
		const TArray<AActor*>& Actors,
		const TArray<FGeometryScriptRenderCaptureCamera>& Cameras,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
