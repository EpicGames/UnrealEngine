// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshSculptLayersFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

USTRUCT(BlueprintType)
struct FGeometryScriptSculptLayerUpdateOptions
{
	GENERATED_BODY()
public:
	/** Whether to recompute normals when changes to sculpt layers may have changed the mesh vertex positions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRecomputeNormals = true;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSculptLayers"))
class UGeometryScriptLibrary_MeshSculptLayersFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/// Enable Sculpt Layers on the Target Mesh, if not already enabled, with at least the requested number of layers. Note: Will never decrease the number of enabled layers.
	/// @param NumLayers The amount of layers that must be enabled on the mesh. Note: If this many layers (or more) are already enabled, the mesh will not be changed.
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	EnableSculptLayers(
		UDynamicMesh* TargetMesh,
		int32 NumLayers,
		UGeometryScriptDebug* Debug = nullptr);

	/// Set the requested LayerIndex as the current active sculpt layer, if possible.
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetActiveSculptLayer(
		UDynamicMesh* TargetMesh,
		int32 LayerIndex,
		UGeometryScriptDebug* Debug = nullptr);

	/// Set the weight of the layer at LayerIndex to the requested Weight
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetSculptLayerWeight(
		UDynamicMesh* TargetMesh,
		int32 LayerIndex,
		double Weight,
		FGeometryScriptSculptLayerUpdateOptions Options = FGeometryScriptSculptLayerUpdateOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/// Set the weights of multiple layers to match the given Weights array.
	/// Note: If the Weights array length is larger than the number of layers, will just set the weights of the existing layers. Will not add or remove layers.
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetSculptLayerWeightsArray(
		UDynamicMesh* TargetMesh,
		TArray<double> Weights,
		FGeometryScriptSculptLayerUpdateOptions Options = FGeometryScriptSculptLayerUpdateOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/// Get the weights of all sculpt layers on the mesh
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Weights")
	TArray<double> GetSculptLayerWeightsArray(UDynamicMesh* TargetMesh);

	/// Get the number of sculpting layers active on the mesh
	UFUNCTION(BlueprintPure, Category = "GeometryScript|SculptLayers")
	static UE_API UPARAM(DisplayName = "Num Layers") int32 GetNumSculptLayers(const UDynamicMesh* TargetMesh);

	/// Get the current sculpting layers active on the mesh, or -1 if the mesh does not have sculpting layers
	UFUNCTION(BlueprintPure, Category = "GeometryScript|SculptLayers")
	static UE_API UPARAM(DisplayName = "Layer Index") int32 GetActiveSculptLayer(const UDynamicMesh* TargetMesh);

	/// Discard all sculpt layer data, leaving current vertex positions unchanged
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug", Keywords="Delete Remove"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	DiscardSculptLayers(
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/// Merge a range of sculpt layers together. May change the Active Sculpt Layer.
	/// @param OutActiveLayer The active layer after layers have been merged.
	/// @param MergeLayerStart The first layer to merge
	/// @param MergeLayerNum The number of layers to merge. Note: If larger than the number of existing layers after MergeLayerStart, will be clamped as needed.
	/// @param bUseWeights If true, layers will be merged based on their current weights -- keeping the mesh shape unchanged. Otherwise, layers will be merged with weight 1, which may change the shape of the mesh.
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SculptLayers", meta = (ScriptMethod, HidePin = "Debug", Keywords = "Combine"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	MergeSculptLayers(
		UDynamicMesh* TargetMesh,
		int32& OutActiveLayer,
		int32 MergeLayerStart,
		int32 MergeLayerNum,
		bool bUseWeights = true,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
