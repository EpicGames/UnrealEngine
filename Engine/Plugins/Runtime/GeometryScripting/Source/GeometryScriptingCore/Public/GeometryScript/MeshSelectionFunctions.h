// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshSelectionFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


/**
 * Types of connection between adjacent Mesh Elements (vertices/triangles/polygoups)
 */
UENUM(BlueprintType)
enum class EGeometryScriptTopologyConnectionType : uint8
{
	Geometric,
	Polygroup UMETA(DisplayName = "PolyGroup"),
	MaterialID
};




UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSelection"))
class UGeometryScriptLibrary_MeshSelectionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Query information about a Mesh Selection
	 * Note that NumSelected may double count some polygroups and (non-border) edges due to their internal representation
	 * Use GetMeshUniqueSelectionInfo for an accurate count of unique mesh elements
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API void
	GetMeshSelectionInfo(
		FGeometryScriptMeshSelection Selection,
		EGeometryScriptMeshSelectionType& SelectionType,
		int& NumSelected);

	/**
	 * Query information about a Mesh Selection, and get a count of unique selected elements
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod))
	static UE_API void
	GetMeshUniqueSelectionInfo(
		const UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		EGeometryScriptMeshSelectionType& SelectionType,
		int& NumSelected);

	/**
	 * Print information about the Mesh Selection to the Output Log
	 * @param bDisable if true, printing will be disabled (useful for debugging)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API void
	DebugPrintMeshSelection(
		FGeometryScriptMeshSelection Selection,
		bool bDisable = false);

	/**
	 * Create a Selection of the given SelectionType that contains all the mesh elements of TargetMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CreateSelectAllMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles );

	/**
	 * Create a Selection of the SelectionType that contains all mesh elements referencing triangles with the given Material ID
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshElementsByMaterialID(
		UDynamicMesh* TargetMesh,
		int MaterialID,
		FGeometryScriptMeshSelection& Selection,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles );

	/**
	 * Create a Selection of the SelectionType that contains all mesh elements referencing triangles with the given PolyGroup ID in the given GroupLayer
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshElementsByPolygroup(
		UDynamicMesh* TargetMesh,
		FGeometryScriptGroupLayer GroupLayer,
		UPARAM(DisplayName = "PolyGroup ID") int PolygroupID,
		FGeometryScriptMeshSelection& Selection,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles );

	/**
	 * Convert a Mesh Selection to a different Type (eg Vertices to Triangles, etc)
	 * By default, Vertices map to Triangle one-rings, and Triangles to all contained vertices.
	 * If bAllowPartialInclusion is disabled, then more restrictive conversions are performed, as follows:
	 *   For To-Vertices, only include vertices where all one-ring triangles or edges are included in FromSelection.
	 *   For To-Edges, only include edges where all adjacent triangles or vertices are included in FromSelection
	 *   For To-Triangles, only include triangles where all tri vertices or edges are included in FromSelection.
	 *   For To-PolyGroups, only include groups where all group triangles are touched by FromSelection
	 * (Note: The To-PolyGroups rule allows vertex and edge selections that miss some vertices or edges, as long as they touch all the polygroup triangles.)
	 * @param bAllowPartialInclusion if false, perform more limited selection conversion as described above
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection FromSelection,
		FGeometryScriptMeshSelection& ToSelection,
		EGeometryScriptMeshSelectionType NewType = EGeometryScriptMeshSelectionType::Triangles,
		bool bAllowPartialInclusion = true );

	/**
	 * Combine two Mesh Selections into a new Mesh Selection.
	 * The two inputs SelectionA and SelectionB must have the same Type.
	 * @param CombineMode specifies how the selection elements in SelectionA and SelectionB are interpreted/combined.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API void
	CombineMeshSelections(
		FGeometryScriptMeshSelection SelectionA,
		FGeometryScriptMeshSelection SelectionB,
		FGeometryScriptMeshSelection& ResultSelection,
		EGeometryScriptCombineSelectionMode CombineMode = EGeometryScriptCombineSelectionMode::Add );


	/**
	 * Create a Mesh Selection from the IndexArray. 
	 * @param SelectionType type of indices specified in the selection
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertIndexArrayToMeshSelection(
		UDynamicMesh* TargetMesh,
		const TArray<int32>& IndexArray,
		EGeometryScriptMeshSelectionType SelectionType,
		FGeometryScriptMeshSelection& Selection);

	/**
	 * Create a Mesh Selection from the IndexSet. 
	 * @param SelectionType type of indices specified in the selection
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertIndexSetToMeshSelection(
		UDynamicMesh* TargetMesh,
		const TSet<int32>& IndexSet,
		EGeometryScriptMeshSelectionType SelectionType,
		FGeometryScriptMeshSelection& Selection);

	/**
	 * Convert a Mesh Selection to an Index Array
	 * @param ConvertToType optional parameter specifying the type of Index List to convert to. If Set to Any, no conversion will be performed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertMeshSelectionToIndexArray(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection, 
		TArray<int32>& IndexArray,
		EGeometryScriptMeshSelectionType& SelectionType);

	/**
	 * Create a Mesh Selection from the Index List. For cases where the IndexList Type does not match
	 * the SelectionType, ConvertMeshSelection with bAllowPartialInclusion=true is used to convert.
	 * @param SelectionType type of indices desired in the Output selection
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertIndexListToMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptIndexList IndexList,
		EGeometryScriptMeshSelectionType SelectionType,
		FGeometryScriptMeshSelection& Selection);

	/**
	 * Convert a Mesh Selection to an Index List
	 * @param ConvertToType optional parameter specifying the type of Index List to convert to. If Set to Any, no conversion will be performed.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ConvertMeshSelectionToIndexList(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection, 
		FGeometryScriptIndexList& IndexList,
		EGeometryScriptIndexType& ResultListType,
		EGeometryScriptIndexType ConvertToType = EGeometryScriptIndexType::Any);


	/**
	 * Create a new Mesh Selection of the SelectionType for the TargetMesh by finding all elements contained in the Box.
	 * @param bInvert return a selection of all elements not in the Box
	 * @param MinElementVertices number of vertices of a triangle or edge that must be in the box for it to be selected (clamped to 1-3 for tri; 1-2 for edge)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod, AdvancedDisplay="MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsInBox(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection, 
		FBox Box,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles,
		bool bInvert = false,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3);

	/**
	* Create a new Mesh Selection of the SelectionType for the TargetMesh by finding all elements contained in the Sphere.
	* @param SphereOrigin center point of the Sphere
	* @param SphereRadius radius of the Sphere
	* @param bInvert return a selection of all elements not in the Sphere
	* @param MinElementVertices number of vertices of a triangle or edge that must be in the Sphere for it to be selected (clamped to 1-3 for tri; 1-2 for edge)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod, AdvancedDisplay="MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsInSphere(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection, 
		FVector SphereOrigin = FVector::ZeroVector,
		double SphereRadius = 100.0,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles,
		bool bInvert = false,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3);

	/**
	* Create a new Mesh Selection of the SelectionType for the TargetMesh by finding all elements on the "positive" side of a Plane
	* @param PlaneOrigin center point of the Plane
	* @param PlaneNormal normal vector for the Plane
	* @param bInvert return a selection of all elements on the other (negative) side of the Plane
	* @param MinElementVertices number of vertices of a triangle or edge that must be on the positive Plane side to be selected (clamped to 1-3 for tri; 1-2 for edge)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod, AdvancedDisplay="MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsWithPlane(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection, 
		FVector PlaneOrigin = FVector::ZeroVector,
		FVector PlaneNormal = FVector::UpVector,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles,
		bool bInvert = false,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3);

	/**
	* Create a new Mesh Selection of the SelectionType for the TargetMesh by finding all elements that have a normal
	* vector that is within an angular deviation threshold from the given Normal. 
	* For Triangle and PolyGroup selections the triangle facet normal is used, for Vertex selections the per-vertex averaged normal is used.
	* @param Normal normal/direction vector to measure against
	* @param MaxAngleDeg maximum angular deviation from Normal, in degrees
	* @param bInvert return a selection of all elements not within the given deviation
	* @param MinElementVertices number of vertices of a triangle or edge that must be within the angular deviation for it to be selected  (clamped to 1-3 for tri; 1-2 for edge)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod, AdvancedDisplay="MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsByNormalAngle(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection, 
		FVector Normal = FVector::UpVector,
		double MaxAngleDeg = 1.0,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles,
		bool bInvert = false,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3);

	/**
	 * Create a new Selection, for the TargetMesh, of all 'sharp' edges where the edge's adjacent triangle normals differ by at least MinAngleDeg
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshSharpEdges(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection,
		double MinAngleDeg = 20.0
	);

	/**
	 * Create a new Selection, for the TargetMesh, of all edges seams in the mesh normal topology -- i.e., where normals can be different across the edge
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshSplitNormalEdges(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection
	);

	/**
	 * Create a new Selection, for the TargetMesh, of all mesh boundary edges
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshBoundaryEdges(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection& Selection
	);

	/**
	 * Create a new BoundarySelection, for the TargetMesh, of the edges on the boundary of another Selection
	 * @param bExcludeMeshBoundaryEdges If true, do not include Selection boundary edges if they are also mesh boundary edges
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectSelectionBoundaryEdges(
		UDynamicMesh* TargetMesh,
		const FGeometryScriptMeshSelection& Selection,
		FGeometryScriptMeshSelection& BoundarySelection,
		bool bExcludeMeshBoundaryEdges = false
	);

	
	/**
	 * Create a new Mesh Selection of UV seam edges for the TargetMesh
	 * @param UVChannel Seams will be found with respect to this UV Channel
	 * @param bHaveValidUVs If false, the requested UV Channel was not present on the TargetMesh
	 * @param bExcludeMeshBoundaryEdges If true, do not include mesh boundary edges in the selection; otherwise, they will be included.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod, DisplayName = "Select Mesh UV Seam Edges"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshUVSeamEdges(
		UDynamicMesh* TargetMesh, FGeometryScriptMeshSelection& Selection,
		UPARAM(DisplayName = "UV Channel") int32 UVChannel, bool& bHaveValidUVs,
		bool bExcludeMeshBoundaryEdges = false
	);
	
	/**
	 * Create a new Mesh Selection of PolyGroup boundary edges for the TargetMesh
	 * @param GroupLayer PolyGroup layer to reference
	 * @param bExcludeMeshBoundaryEdges If true, do not include mesh boundary edges in the selection; otherwise, they will be included.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta = (ScriptMethod, DisplayName = "Select Mesh PolyGroup Boundary Edges"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SelectMeshPolyGroupBoundaryEdges(
		UDynamicMesh* TargetMesh, FGeometryScriptMeshSelection& Selection,
		FGeometryScriptGroupLayer GroupLayer,
		bool bExcludeMeshBoundaryEdges = false
	);


	/**
	* Create a new Mesh Selection of the SelectionType for the TargetMesh by finding all elements inside a second SelectionMesh
	* For Triangle and PolyGroup selections the triangle facet normal is used, for Vertex selections the per-vertex averaged normal is used.
	* @param SelectionMeshTransform Transform applied to SelectionMesh for inside/outside testing
	* @param bInvert return a selection of all elements not within the given deviation
	* @param ShellDistance If > 0, points within this distance from SelectionMesh will also be considered "inside"
	* @param WindingThreshold Threshold used for Fast Mesh Winding Number inside/outside test (range is [0,1], with 1 being "inside")
	* @param MinElementVertices number of vertices of a triangle or edge that must be within the selection mesh for it to be selected  (clamped to 1-3 for tri; 1-2 for edge)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod, AdvancedDisplay="MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsInsideMesh(
		UDynamicMesh* TargetMesh,
		UDynamicMesh* SelectionMesh,
		FGeometryScriptMeshSelection& Selection, 
		FTransform SelectionMeshTransform,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Triangles,
		bool bInvert = false,
		double ShellDistance = 0.0,
		double WindingThreshold = 0.5,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3);


	/**
	 * Invert the Selection on the TargetMesh, ie select what is not currently selected
	 * @param bOnlyToConnected if true, the inverse is limited to mesh areas geometrically connected to the Selection, instead of the entire mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	InvertMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection, 
		FGeometryScriptMeshSelection& NewSelection,
		bool bOnlyToConnected = false );


	/**
	 * Expand the Selection on the TargetMesh to connected regions and return the NewSelection
	 * @param ConnectionType defines what "connected" means, ie purely geometric connection, or some additional constraint like same MaterialIDs/etc 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ExpandMeshSelectionToConnected(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection, 
		FGeometryScriptMeshSelection& NewSelection,
		EGeometryScriptTopologyConnectionType ConnectionType = EGeometryScriptTopologyConnectionType::Geometric );


	/**
	 * Grow or Shrink the Selection on the TargetMesh to connected neighbours.
	 * For Vertex selections, Expand includes vertices in one-ring of selected vertices, and Contract removes any vertices with a one-ring neighbour that is not selected
	 * For Triangle selections, Add/Remove Triangles connected to selected Triangles
	 * For PolyGroup selections, Add/Remove PolyGroups connected to selected PolyGroups
	 * @param Iterations number of times to Expand/Contract the Selection. Valid range is [0,100] where 0 is a no-op.
	 * @param bContract if true selection contracts instead of growing
	 * @param bOnlyExpandToFaceNeighbours if true, only adjacent Triangles/PolyGroups directly connected by an edge are added, vs connected to any selected vertex
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshSelection", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ExpandContractMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection, 
		FGeometryScriptMeshSelection& NewSelection,
		int32 Iterations = 1,
		bool bContract = false,
		bool bOnlyExpandToFaceNeighbours = false );


};

#undef UE_API
