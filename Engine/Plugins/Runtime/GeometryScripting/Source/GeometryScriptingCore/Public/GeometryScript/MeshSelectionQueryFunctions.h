// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshSelectionQueryFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSelectionQueries"))
class UGeometryScriptLibrary_MeshSelectionQueryFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Get the 3D Bounding Box of a Mesh Selection, ie bounding box of vertices contained in the Selection
	 * @param bIsEmpty will return as true if the selection was empty (the box will be initialized to 0 in this case)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SelectionQueries", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshSelectionBoundingBox(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FBox& SelectionBounds,
		bool& bIsEmpty,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Compute the set of Vertex Loops bordering a Mesh Selection. Both the 3D polylines and lists of vertex indices are returned for each Loop.
	 * Note that for a Vertex selection this will function return the border loops around the set of vertex triangle one-rings.
	 * 
	 * @param IndexLoops for each discovered Loop, the IndexList of mesh vertex indices around the loop is returned here
	 * @param PathLoops for each discovered Loop, the PolyPath of mesh vertex positions around the loop is returned here. The ordering for each loop is the same as IndexLoops.
	 * @param NumLoops number of loops found is returned here
	 * @param bFoundErrors true is returned here if topological errors were found during loop computation. In this case the Loop set may be incomplete.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SelectionQueries", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshSelectionBoundaryLoops(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		TArray<FGeometryScriptIndexList>& IndexLoops,
		TArray<FGeometryScriptPolyPath>& PathLoops,
		int& NumLoops,
		bool& bFoundErrors,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
