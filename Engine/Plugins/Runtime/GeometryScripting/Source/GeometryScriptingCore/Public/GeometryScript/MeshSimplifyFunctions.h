// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshSimplifyFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

USTRUCT(BlueprintType)
struct FGeometryScriptPlanarSimplifyOptions
{
	GENERATED_BODY()
public:
	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float AngleThreshold = 0.001f;

	/** If enabled, the simplified mesh is automatically compacted to remove gaps in the index space. This is expensive and can be disabled by advanced users. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoCompact = true;
};

USTRUCT(BlueprintType)
struct FGeometryScriptPolygroupSimplifyOptions
{
	GENERATED_BODY()
public:
	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float AngleThreshold = 0.001f;

	/** If enabled, the simplified mesh is automatically compacted to remove gaps in the index space. This is expensive and can be disabled by advanced users. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoCompact = true;
};


UENUM(BlueprintType)
enum class EGeometryScriptRemoveMeshSimplificationType : uint8
{
	/** classic quadric error metric without volume preservation */
	StandardQEM = 0 UMETA(DisplayName = "Standard QEM"),      

	/** classic quadric error metric with volume preservation */
	VolumePreserving = 1 UMETA(DisplayName = "Volume Preserving"), 

	/** QEM with volume preservation, account for vertex normals in optimization. This used to be called Attribute Aware prior to 5.8. */
	AttributeAware = 2 UMETA(DisplayName = "Normals Aware, Volume Preserving"),   

	/** QEM with volume preservation, accounting for normals/tangents/bitangents/color/texture coordinates and weight channels with attribute seams. */
	AttributeAwareV2 = 3 UMETA(DisplayName = "Attribute Aware, Volume Preserving") 
};

UENUM(BlueprintType)
enum class EGeometryScriptMeshSimplificationQuadricVariant : uint8
{
	PlaneQuadric = 0,
	TriangleQuadric = 1
};

// Specify a weight map for controlling relative density of a mesh
USTRUCT(BlueprintType)
struct FGeometryScriptWeightMapDensity
{
	GENERATED_BODY()
public:
	
	/** The weight map to reference */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptWeightMapHandle Handle;

	/** 
	 * Controls the effect of the weight map:
	 * Positive values increase local mesh density for larger weights, negative values decrease density, and zero gives no effect
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = "-3.0", UIMax = "3.0"))
	float RelativeDensity = 0.f;
};

USTRUCT(Blueprintable)
struct FGeometryScriptSimplifyMeshOptions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRemoveMeshSimplificationType Method = EGeometryScriptRemoveMeshSimplificationType::AttributeAware;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowSeamCollapse = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowSeamSmoothing = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowSeamSplits = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreserveVertexPositions = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRetainQuadricMemory = false;

	/** Optional weight map to scale measured edge lengths */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptWeightMapDensity EdgeLengthWeightMap;

	/** Optional weight map to scale measured geometric errors */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptWeightMapDensity GeometricToleranceWeightMap;

	/** Optional weight map to scale measured quadric error */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptWeightMapDensity QuadricErrorWeightMap;

	/** Setting a very small RegularizeWeight value (e.g., 0.000001) can improve triangle quality and the effect of the QuadricErrorWeightMap in flat regions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMax = ".01"))
	float RegularizeWeight = 0.000001f;

	/** If enabled, the simplified mesh is automatically compacted to remove gaps in the index space. This is expensive and can be disabled by advanced users. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoCompact = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptMeshSimplificationQuadricVariant QuadricVariant = EGeometryScriptMeshSimplificationQuadricVariant::PlaneQuadric;

	/** Control the influence of the normals to the simplification metric (for attribute-aware quadrics) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float NormalAttributeWeight = 16.f;

	/** Control the influence of tangents and bitangents to the simplification metric (for attribute-aware quadrics) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float TangentAttributeWeight = 0.1f;

	/** Control the influence of color channels to the simplification metric. alpha is ignored but preserved. (for attribute-aware quadrics) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ColorAttributeWeight = 0.1f;

	/** Control the influence of texture coordinates the simplification metric (all available channels). (for attribute-aware quadrics) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float TexCoordAttributeWeight = 0.5f;

	/** The behavior of the simplifier is not scale invariant: scaling an object by X will impact the geometry terms X^2 times more than
	 *  it will impact the attributes and change the relative impact. To facilitate rebalancing, while working with the same attribute weights,
	 *  this setting can be used. Its meaning corresponds to the scale of the object relative to the scale at which the attribute weights
	 *  were initially calibrated for. This means, setting this value to X results in the same behavior as if the object was scaled down 
	 *  by a factor of X and rescaled back up after simplification. Only supported for attribute aware method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ScaleCorrection = 1.f;

};

UENUM(BlueprintType)
enum class EGeometryScriptClusterSimplifyConstraintLevel : uint8
{
	// Edge will be kept as-is in final output
	Fixed,
	// Attempt to preserve 'paths' of constrained edges, but may simplify along the path
	// Vertices where paths converge will be kept in final output
	Constrained,
	// Edge can be simplified without constraint
	Free
};

USTRUCT(BlueprintType)
struct FGeometryScriptClusterSimplifyEdgeConstraintOptions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel MeshBoundary = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel GroupBoundary = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel MaterialBoundary = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel UVSeam = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel ColorSeam = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConstraintOptions)
	EGeometryScriptClusterSimplifyConstraintLevel NormalSeam = EGeometryScriptClusterSimplifyConstraintLevel::Constrained;
};

USTRUCT(BlueprintType)
struct FGeometryScriptClusterSimplifyMeshOptions
{
	GENERATED_BODY()
public:

	// Whether to discard attributes (materials, UV/normal/tangent/color attributes)
	// Note that attribute seam edges will not be constrained if attributes are discarded
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDiscardAttributes = false;

	// If > 0, boundary vertices w/ incident boundary edge angle greater than this (in degrees) will be kept in the output
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double FixBoundaryAngleTolerance = 45;

	// Options to constrain simplification of different mesh edge types, to optionally help preserve mesh boundaries and seams
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptClusterSimplifyEdgeConstraintOptions EdgeConstraints;

	// Optional weight map to control simplification error tolerances and/or edge lengths
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptWeightMapHandle WeightMapHandle;

	/** Amount to adjust density in regions where the vertex weight map is greater than zero. A positive value results in less simplification in weighted regions, while a negative value results in more */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = "-2.0", UIMax = "2.0"))
	float RelativeDensity = 0.f;

};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSimplification"))
class UGeometryScriptLibrary_MeshSimplifyFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Simplifies planar areas of the mesh that have more triangles than necessary. Note that it does not change the 3D shape of the mesh.
	* Planar regions are identified by comparison of face normals using a Angle Threshold in the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToPlanar(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPlanarSimplifyOptions Options,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	* Simplifies the mesh down to the PolyGroup Topology. For example, the high-level faces of the mesh PolyGroups. 
	* Another example would be on a default Box-Sphere where simplifying to PolyGroup topology produces a box.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply Simplify To PolyGroup Topology"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToPolygroupTopology(  
		UDynamicMesh* TargetMesh, 
		FGeometryScriptPolygroupSimplifyOptions Options,
		FGeometryScriptGroupLayer GroupLayer,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Simplifies the mesh until a target triangle count is reached. Behavior can be additionally controlled with the Options. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToTriangleCount(  
		UDynamicMesh* TargetMesh, 
		int32 TriangleCount,
		FGeometryScriptSimplifyMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Simplifies the mesh until a target vertex count is reached. Behavior can be additionally controlled with the Options. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToVertexCount(  
		UDynamicMesh* TargetMesh, 
		int32 VertexCount,
		FGeometryScriptSimplifyMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Simplifies the mesh until a target triangle count is reached, using the UE Editor's standard mesh simplifier. Editor only.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyEditorSimplifyToTriangleCount(  
		UDynamicMesh* TargetMesh, 
		int32 TriangleCount,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Simplifies the mesh until a target vertex count is reached, using the UE Editor's standard mesh simplifier. Editor only.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyEditorSimplifyToVertexCount(  
		UDynamicMesh* TargetMesh, 
		int32 VertexCount,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Simplifies the mesh to a target geometric tolerance. Stops when any further simplification would result in a deviation from the input mesh larger than the tolerance.
	* Behavior can be additionally controlled with the Options. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToTolerance(  
		UDynamicMesh* TargetMesh, 
		float Tolerance,
		FGeometryScriptSimplifyMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Simplifies the mesh to a target edge length, using error-based edge collapse.
	 * Note: Not intended to create uniform edge lengths. Result may have edge lengths much larger than the target value.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySimplifyToEdgeLength(  
		UDynamicMesh* TargetMesh, 
		double EdgeLength,
		FGeometryScriptSimplifyMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Simplifies the mesh to a target edge length, using a fast cluster-based method (not error-based).
	 * Note: Lengths are approximated via 'graph distance' on the original mesh; will not exactly match the target.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Simplification", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyClusterSimplifyToEdgeLength(
		UDynamicMesh* TargetMesh,
		double EdgeLength,
		FGeometryScriptClusterSimplifyMeshOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
