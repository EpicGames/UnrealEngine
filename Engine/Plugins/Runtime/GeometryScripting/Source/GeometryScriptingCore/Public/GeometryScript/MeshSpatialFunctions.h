// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshSpatialFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


USTRUCT(BlueprintType, meta = (DisplayName = "BVH Query Options"))
struct FGeometryScriptSpatialQueryOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float MaxDistance = 0;

	/** When true, allows the provided BHV to be used even if the mesh has been updated */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowUnsafeModifiedQueries = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float WindingIsoThreshold = 0.5;
};





USTRUCT(BlueprintType, meta = (DisplayName = "Ray Hit Result"))
struct FGeometryScriptRayHitResult
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHit = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float RayParameter = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int HitTriangleID = -1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector HitPosition = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector HitBaryCoords = FVector::ZeroVector;
};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSpatial"))
class UGeometryScriptLibrary_MeshSpatial : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Reset the Bounding Volume Hierarchy (BVH) by clearing all the internal data.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta = (ScriptMethod))
	static UE_API void ResetBVH(UPARAM(ref) FGeometryScriptDynamicMeshBVH& ResetBVH);
	
	/**
	* Builds a Bounding Volume Hierarchy (BVH) object for a mesh that can be used with multiple spatial queries.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Build BVH For Mesh"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	BuildBVHForMesh(
		UDynamicMesh* TargetMesh,
		FGeometryScriptDynamicMeshBVH& OutputBVH,
		UGeometryScriptDebug* Debug = nullptr );
	
	/**
	* Checks if the provided Bounding Volume Hierarchy (BVH) can still be used with the Mesh — it generally returns false if the mesh has been changed.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Is BVH Valid For Mesh"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	IsBVHValidForMesh(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) const FGeometryScriptDynamicMeshBVH& TestBVH,
		bool& bIsValid,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Rebuilds the Bounding Volume Hierarchy (BVH) for the mesh in-place, which can reduce memory allocations, compared to building a new BVH.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", DisplayName="Rebuild BVH For Mesh"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RebuildBVHForMesh(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) FGeometryScriptDynamicMeshBVH& UpdateBVH,
		bool bOnlyIfInvalid = true,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Finds the nearest point (Nearest Result) on the Target Mesh to a given 3D point (Query Point) by using the Query BVH.
	* @param QueryBVH a BVH associated with the Target Mesh
	* @param QueryPoint a 3D location relative to the local space of the mesh
	* @param NearestResult on return, holds the nearest point on the mesh to the QueryPoint
	* @param Outcome will be either Found or Not Found depending on the success of the query.
	* Note NearestResult.bValid will be false if the query failed. 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	FindNearestPointOnMesh(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
		FVector QueryPoint,
		FGeometryScriptSpatialQueryOptions Options,
		FGeometryScriptTrianglePoint& NearestResult,
		EGeometryScriptSearchOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Finds the nearest intersection of a 3D ray with the mesh by using the Query BVH.  
	* Note, depending on the Ray Origin and Ray Direction, there is the possibility that the ray might not intersect with the Target Mesh.  
	* Should the ray miss, the HitResult.bHit will be false and the Outcome  will be Not Found.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	FindNearestRayIntersectionWithMesh(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
		FVector RayOrigin,
		FVector RayDirection,
		FGeometryScriptSpatialQueryOptions Options,
		FGeometryScriptRayHitResult& HitResult,
		EGeometryScriptSearchOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Tests if a point is inside the mesh using the Fast Winding Number query and data stored in the BVH.
	* @param QueryBVH is an acceleration structure previously built with this mesh.
	* @param QueryPoint the point in the mesh's 3D local space.
	* @param Options control the fast winding number threshold 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	IsPointInsideMesh(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
		FVector QueryPoint,
		FGeometryScriptSpatialQueryOptions Options,
		bool& bIsInside,
		EGeometryScriptContainmentOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	* Create Mesh Selection of mesh elements in TargetMesh contained by QueryBox, using QueryBVH to accellerate the computation.
	* Triangles and Edges are selected if Min Element Vertices (clamped to a 1-3 or 1-2 range, for triangles or edges respectively) or more are inside the box. 
	* PolyGroups are selected if any of their triangles are inside the box
	* 
	* Note that this method cannot select mesh elements that cut through the query box without having any vertices in the query box.
	* 
	* @param QueryBVH is an acceleration structure previously built with TargetMesh.
	* @param QueryPoint the point in the mesh's 3D local space.
	* @param Options control the fast winding number threshold 
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Spatial", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "MinNumTrianglePoints"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SelectMeshElementsInBoxWithBVH(
		UDynamicMesh* TargetMesh,
		UPARAM(ref) const FGeometryScriptDynamicMeshBVH& QueryBVH,
		FBox QueryBox,
		FGeometryScriptSpatialQueryOptions Options,
		FGeometryScriptMeshSelection& Selection,
		EGeometryScriptMeshSelectionType SelectionType = EGeometryScriptMeshSelectionType::Vertices,
		UPARAM(DisplayName = "Min Element Vertices") int MinNumTrianglePoints = 3,
		UGeometryScriptDebug* Debug = nullptr );

};

#undef UE_API
