// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshSubdivideFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

//
// PN Tessellate options
//
USTRUCT(BlueprintType, meta = (DisplayName = "PN Tessellate Options"))
struct FGeometryScriptPNTessellateOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRecomputeNormals = true;
};


//
// Selective Tessellate options
//
UENUM(BlueprintType)
enum class ESelectiveTessellatePatternType : uint8
{
	ConcentricRings = 0
};


USTRUCT(BlueprintType, meta = (DisplayName = "Selective Tessellate Options"))
struct FGeometryScriptSelectiveTessellateOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableMultithreading = true;

	/** EmptyBehavior Defines how an empty input selection should be interpreted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptEmptySelectionBehavior EmptyBehavior = EGeometryScriptEmptySelectionBehavior::FullMeshSelection;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshSubdivide"))
class UGeometryScriptLibrary_MeshSubdivideFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:


	
	/**
	 * Apply PN Tessellation to the Target Mesh as controlled by the Tessellation Level and the Options.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Subdivide", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply PN Tessellation"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyPNTessellation(
		UDynamicMesh* TargetMesh,
		FGeometryScriptPNTessellateOptions Options,
		int TessellationLevel = 3,
		UGeometryScriptDebug* Debug = nullptr );
	
	/**
	* Apply Uniform Tessellation to the Target Mesh as controlled by the Tessellation Level and the Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Subdivide", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyUniformTessellation(
		UDynamicMesh* TargetMesh,
		int TessellationLevel = 3,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Selectively Tessellate a Selection of the Target Mesh or possibly the entire mesh as controlled by 
	* the Options.
	* @param Selection selects the triangles of the mesh to be tessellated.
	* @param Options controls the behavior of the tessellation if the Selection is empty.
	* @param TessellationLevel determines the amount of tessellation
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Subdivide", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplySelectiveTessellation(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FGeometryScriptSelectiveTessellateOptions Options,
		int TessellationLevel = 1,
		ESelectiveTessellatePatternType PatternType = ESelectiveTessellatePatternType::ConcentricRings,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
