// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshTransformFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshTransforms"))
class UGeometryScriptLibrary_MeshTransformFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Applies the provided Transform to the vertices of a Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	TransformMesh(
		UDynamicMesh* TargetMesh,
		FTransform Transform,
		bool bFixOrientationForNegativeScale = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Applies the inverse of the provided Transform to the TargetMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta = (ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	InverseTransformMesh(
		UDynamicMesh* TargetMesh,
		FTransform Transform,
		bool bFixOrientationForNegativeScale = true,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Applies the provided Translation to the vertices of a Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	TranslateMesh(
		UDynamicMesh* TargetMesh,
		FVector Translation,
		UGeometryScriptDebug* Debug = nullptr );
	
	/**
	* Rotates the mesh relative to the specified Rotation Origin.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RotateMesh(
		UDynamicMesh* TargetMesh,
		FRotator Rotation,
		FVector RotationOrigin = FVector(0,0,0),
		UGeometryScriptDebug* Debug = nullptr );
	
	/**
	* Applies the provided Scale transformation relative to the Scale Origin to the vertices of a Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ScaleMesh(
		UDynamicMesh* TargetMesh,
		FVector Scale = FVector(1,1,1),
		FVector ScaleOrigin = FVector(0,0,0),
		bool bFixOrientationForNegativeScale = true,
		UGeometryScriptDebug* Debug = nullptr);



	/**
	* Applies the given Transform to the vertices identified by the Selection of the mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	TransformMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FTransform Transform,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Applies the inverse of the given Transform to the vertices identified by the Selection of the TargetMesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	InverseTransformMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FTransform Transform,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Applies the given Translation to the vertices identified by the Selection of the mesh.
	*/ 
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	TranslateMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FVector Translation,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Rotates the selected part of the mesh relative to the specified Rotation Origin.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RotateMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FRotator Rotation,
		FVector RotationOrigin = FVector(0,0,0),
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Applies the given Scale transform relative to the Scale Origin to the selection part of the mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bFixOrientationForNegativeScale"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ScaleMeshSelection(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FVector Scale = FVector(1,1,1),
		FVector ScaleOrigin = FVector(0,0,0),
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Set the Pivot Location for the Mesh. Since the Pivot of a Mesh object is always the point at (0,0,0),
	 * this function simply translates the mesh by -PivotLocation.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Transforms", meta=(ScriptMethod, HidePin = "Debug"))
	static UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	TranslatePivotToLocation(
		UDynamicMesh* TargetMesh,
		FVector PivotLocation,
		UGeometryScriptDebug* Debug = nullptr)
	{
		// note: this function is redundant, however *many* users do not intuitively understand the relationship
		// between "setting the pivot" and translation. This function is for those users.
		return TranslateMesh(TargetMesh, -PivotLocation, Debug);
	}

};

#undef UE_API
