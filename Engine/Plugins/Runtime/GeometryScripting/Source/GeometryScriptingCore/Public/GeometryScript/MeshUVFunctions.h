// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshUVFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


USTRUCT(BlueprintType)
struct FGeometryScriptRepackUVsOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int TargetImageWidth = 512;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOptimizeIslandRotation = true;
};

UENUM(BlueprintType)
enum class EGeometryScriptUVLayoutType : uint8
{
	/** Apply Scale and Translation properties to all UV values */
	Transform,
	/** Uniformly scale and translate each UV island individually to pack it into the unit square, i.e. fit between 0 and 1 with overlap */
	Stack,
	/** Uniformly scale and translate UV islands collectively to pack them into the unit square, i.e. fit between 0 and 1 with no overlap */
	Repack,
	/** Scale and translate UV islands to normalize the UV islands' area to match an average texel density. */
	Normalize
};

USTRUCT(BlueprintType)
struct FGeometryScriptLayoutUVsOptions
{
	GENERATED_BODY()
public:
	/** Type of layout applied to input UVs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptUVLayoutType LayoutType = EGeometryScriptUVLayoutType::Repack;

	/** Expected resolution of the output textures; this controls spacing left between UV islands to avoid interpolation artifacts */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int TextureResolution = 1024;

	/** Uniform scale applied to UVs after packing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Scale = 1;

	/** Translation applied to UVs after packing, and after scaling */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D Translation = FVector2D(0, 0);

	/** Force the Repack layout type to preserve existing scaling of UV islands. Note, this might lead to the packing not fitting within a unit square, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreserveScale = false;

	/** Force the Repack layout type to preserve existing rotation of UV islands. Note, this might lead to the packing not being as space efficient as possible, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreserveRotation = false;

	/** Allow the Repack layout type to flip the orientation when rotating UV islands to save space. Note that this may cause problems for downstream operations, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAllowFlips = false;

	/** Enable UDIM aware layout and keep islands within their originating UDIM tiles when laying out.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableUDIMLayout = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName = "Per UDIM Texture Resolution"))
	TMap<int32, int32> UDIMResolutions;
};

UENUM(BlueprintType)
enum class EGeometryScriptUVFlattenMethod : uint8
{
	ExpMap = 0,
	Conformal = 1,
	SpectralConformal = 2
};

UENUM(BlueprintType)
enum class EGeometryScriptUVIslandSource : uint8
{
	PolyGroups = 0,
	UVIslands = 1
};


USTRUCT(BlueprintType)
struct FGeometryScriptExpMapUVOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int NormalSmoothingRounds = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float NormalSmoothingAlpha = 0.25f;
};

USTRUCT(BlueprintType)
struct FGeometryScriptSpectralConformalUVOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bPreserveIrregularity = true;
};


USTRUCT(BlueprintType)
struct FGeometryScriptRecomputeUVsOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptUVFlattenMethod Method = EGeometryScriptUVFlattenMethod::SpectralConformal;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptUVIslandSource IslandSource = EGeometryScriptUVIslandSource::UVIslands;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptExpMapUVOptions ExpMapOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptSpectralConformalUVOptions SpectralConformalOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptGroupLayer GroupLayer;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoAlignIslandsWithAxes = true;
};





USTRUCT(BlueprintType)
struct FGeometryScriptPatchBuilderOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int InitialPatchCount = 100;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MinPatchSize = 2;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float PatchCurvatureAlignmentWeight = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float PatchMergingMetricThresh = 1.5f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float PatchMergingAngleThresh = 45.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptExpMapUVOptions ExpMapOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRespectInputGroups = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptGroupLayer GroupLayer;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoPack = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptRepackUVsOptions PackingOptions;
};




USTRUCT(BlueprintType)
struct FGeometryScriptXAtlasOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int MaxIterations = 2;
};



UENUM(BlueprintType)
enum class EGeometryScriptTexelDensityMode : uint8
{
	ApplyToIslands,
	ApplyToWhole,
	Normalize
};

USTRUCT(Blueprintable)
struct FGeometryScriptUVTexelDensityOptions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptTexelDensityMode TexelDensityMode = EGeometryScriptTexelDensityMode::ApplyToIslands;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float TargetWorldUnits = 100;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float TargetPixelCount = 1024;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float TextureResolution = 1024;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bEnableUDIMLayout = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName = "Per UDIM Texture Resolution"))
	TMap<int32, int32> UDIMResolutions;
};

USTRUCT(BlueprintType)
struct FGeometryScriptMeshProjectionSettings
{
	GENERATED_BODY()
public:

	// If selection contains no triangles, project all triangles (for both source and target)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bProcessAllIfEmptySelection = true;

	// Maximum projection distance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ProjectionRangeMax = UE_LARGE_WORLD_MAX;

	// Minimum projection distance. If negative, will also consider projection backwards, and take the closest result.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ProjectionRangeMin = -UE_LARGE_WORLD_MAX;
	
	// Whether to reset UVs for triangles where projection failed. Otherwise UVs are left as-is where projection failed.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bResetUVsForUnmatched = false;

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_UVs"))
class UGeometryScriptLibrary_MeshUVFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Set the number of UV Channels on the Target Mesh. If not already enabled, this will enable the mesh attributes.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"), DisplayName = "Set Num UV Channels")
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetNumUVSets( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "Num UV Channels") int NumUVSets,
		UGeometryScriptDebug* Debug = nullptr );

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod, HidePin = "Debug"), DisplayName = "Clear UV Channel")
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ClearUVChannel(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "UV Channel") int UVChannel,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Copy the data in one UV Channel to another UV Channel on the same Target Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"), DisplayName = "Copy UV Channel")
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	CopyUVSet( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "From UV Channel") int FromUVSet,
		UPARAM(DisplayName = "To UV Channel")   int ToUVSet,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Sets the UVs of a mesh triangle in the given UV Channel. 
	 * This function will create new UV elements for each vertex of the triangle, meaning that
	 * the triangle will become an isolated UV island.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshTriangleUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		int TriangleID, 
		FGeometryScriptUVTriangle UVs,
		bool& bIsValidTriangle, 
		bool bDeferChangeNotifications = false );


	/**
	 * Adds a new UV Element to the specified UV Channel of the Mesh and returns a new UV Element ID.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshEdits", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AddUVElementToMesh( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int32 UVSetIndex,
		FVector2D NewUVPosition, 
		UPARAM(DisplayName = "New UV Element ID") int& NewUVElementID,
		bool& bIsValidUVSet,
		bool bDeferChangeNotifications = false );

	/**
	 * Sets the UV Element IDs for a given Triangle in the specified UV Channel, ie the "UV Triangle" indices.
	 * This function does not create new UVs, the provided UV Elements must already.
	 * The UV Triangle can only be set if the resulting topology would be valid, ie the Elements cannot be shared
	 * between different base Mesh Vertices, so they must either be unused by any other triangles, or already associated
	 * with the same mesh vertex in other UV triangles. 
	 * If any conditions are not met, bIsValidTriangle will be returned as false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshTriangleUVElementIDs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		int TriangleID, 
		FIntVector TriangleUVElements,
		bool& bIsValidTriangle, 
		bool bDeferChangeNotifications = false );


	/**
	 * Returns the UV Element IDs associated with the three vertices of the triangle in the specified UV Channel.
	 * If the Triangle does not exist in the mesh or if no UVs are set in the specified UV Channel for the triangle, bHaveValidUVs will be returned as false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshTriangleUVElementIDs(
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, 
		int32 TriangleID, 
		FIntVector& TriangleUVElements,
		bool& bHaveValidUVs);


	/**
	 * Convert Selection to an Edge selection, and set or remove UV seams along all of the selected edges
	 * @param TargetMesh The mesh to update
	 * @param UVChannel The UV Channel to update
	 * @param Selection Which edges to operate on
	 * @param bInsertSeams Whether to insert new seams. If false, removes existing seams instead.
	 * @param bDeferChangeNotifications If true, no mesh change notification will be sent. Set to true if performing many changes in a loop.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod, HidePin = "Debug", DisplayName = "Set UV Seams Along Selected Edges"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetUVSeamsAlongSelectedEdges(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "UV Channel") int UVChannel,
		FGeometryScriptMeshSelection Selection,
		bool bInsertSeams = true,
		bool bDeferChangeNotifications = false,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Returns the UV Position for a given UV Element ID in the specified UV Channel.
	 * If the UV Set or Element ID does not exist, bIsValidElementID will be returned as false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshUVElementPosition(
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int32 UVSetIndex, 
		int32 ElementID, 
		FVector2D& UVPosition,
		bool& bIsValidElementID);

	/**
	 * Sets the UV position of a specific ElementID in the given UV Set/Channel
	 * If the UV Set or Element ID does not exist, bIsValidElementID will be returned as false.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshUVElementPosition( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		int ElementID, 
		FVector2D NewUVPosition,
		bool& bIsValidElementID, 
		bool bDeferChangeNotifications = false );

	/**
	* Update all selected UV values in the specified UV Channel by adding the Translation value to each.
	* If the provided Selection is empty, the Translation is applied to the entire UV Channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	TranslateMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FVector2D Translation,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );
	
	/**
	* Update all selected UV values in the specified UV Channel by Scale, mathematically the new value is given by (UV - ScaleOrigin) * Scale + ScaleOrigin
	* If the provided Selection is empty, the update is applied to the entire UV Channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ScaleMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FVector2D Scale,
		FVector2D ScaleOrigin,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Update all the selected UV values in the specified UV Channel by a rotation of Rotation Angle (in degrees) relative to the Rotation Origin.
	* If the provided Selection is empty, the update is applied to the entire UV Channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RotateMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		float RotationAngle,
		FVector2D RotationOrigin,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	 * Scale of PlaneTransform defines world-space dimension that maps to 1 UV dimension
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshUVsFromPlanarProjection( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FTransform PlaneTransform,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );


	/**
	* Using Box Projection, update the UVs in the UV Channel for an entire mesh or a subset defined by a non-empty Selection.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshUVsFromBoxProjection( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FTransform BoxTransform,
		FGeometryScriptMeshSelection Selection,
		int MinIslandTriCount = 2,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Using Cylinder Projection, update the UVs in the UV Channel for an entire mesh or a subset defined by a non-empty Selection.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	SetMeshUVsFromCylinderProjection( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FTransform CylinderTransform,
		FGeometryScriptMeshSelection Selection,
		float SplitAngle = 45.0,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Copy UVs from one mesh to another, by projecting along the requested direction.
	 * Note: This does not transfer UV seams; it assigns a single UV coordinate per vertex in the target mesh selection.
	 * By default, also searches in -ProjectionDirection and picks the closest source mesh position to copy from --
	 * set ProjectionRangeMin in Settings to a value >= 0 to only search in +ProjectionDirection.
	 *
	 * @param TargetMesh Mesh to assign new UVs
	 * @param TargetUVChannel UV channel to update on target mesh
	 * @param TargetSelection Triangles to update on the target mesh
	 * @param TargetTransform Transform of target mesh
	 * @param SourceMesh Mesh to transfer UVs from
	 * @param SourceMeshOptionalBVH Optional BVH for the source mesh
	 * @param SourceUVChannel UV channel to read from on the source mesh
	 * @param SourceSelection Triangles to read from on the source mesh
	 * @param SourceTransform Transform of the source mesh
	 * @param Settings Additional settings
	 * @param ProjectionDirection Direction to project (in the space where TargetMesh is transformed by TargetTransform, and SourceMesh is transformed by SourceTransform)
	 * @param ProjectionOffset Projection will start offset by this amount from the TargetMesh vertices
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod, HidePin = "Debug", Keywords = "Set From Mesh"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	TransferMeshUVsByProjection(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "Target UV Channel") int TargetUVChannel,
		FGeometryScriptMeshSelection TargetSelection,
		FTransform TargetTransform,
		const UDynamicMesh* SourceMesh,
		FGeometryScriptDynamicMeshBVH SourceMeshOptionalBVH,
		UPARAM(DisplayName = "Source UV Channel") int SourceUVChannel,
		FGeometryScriptMeshSelection SourceSelection,
		FTransform SourceTransform,
		FGeometryScriptMeshProjectionSettings Settings,
		FVector ProjectionDirection = FVector(0,0,-1),
		double ProjectionOffset = 0,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	* Rescales UVs in the UV Channel for a Mesh to match a specified texel density, described by the options passed in. Supports
	* processing on a subset of UVs via a non-empty Selection.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "UDIMResolutions"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyTexelDensityUVScaling( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptUVTexelDensityOptions Options,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Recomputes UVs in the UV Channel for a Mesh based on different types of well-defined UV islands, such as existing UV islands, PolyGroups, 
	* or a subset of the mesh based on a non-empty Selection.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RecomputeMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptRecomputeUVsOptions Options,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Packs the existing UV islands in the specified UV Channel into standard UV space based on the Repack Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	RepackMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptRepackUVsOptions RepackOptions,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Packs the existing UV islands in the specified UV Channel into standard UV space based on the Repack Options.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	LayoutMeshUVs(
		UDynamicMesh* TargetMesh,
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptLayoutUVsOptions LayoutOptions,
		FGeometryScriptMeshSelection Selection,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	* Computes new UVs for the specified UV Channel using PatchBuilder method in the Options, and optionally packs.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AutoGeneratePatchBuilderMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptPatchBuilderOptions Options,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	* Computes new UVs for the specified UV Channel using XAtlas, and optionally packs.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	AutoGenerateXAtlasMeshUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptXAtlasOptions Options,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Compute information about dimensions and areas for a UV Set of a Mesh, with an optional Mesh Selection
	 * @param UVSetIndex index of UV Channel to query
	 * @param Selection subset of triangles to process, whole mesh is used if selection is not provided
	 * @param MeshArea output 3D area of queried triangles
	 * @param UVArea output 2D UV-space area of queried triangles
	 * @param MeshBounds output 3D bounding box of queried triangles
	 * @param UVBounds output 2D UV-space bounding box of queried triangles
	 * @param bIsValidUVSet output flag set to false if UV Channel does not exist on the target mesh. In this case Areas and Bounds are not initialized.
	 * @param bFoundUnsetUVs output flag set to true if any of the queried triangles do not have valid UVs set
	 * @param bOnlyIncludeValidUVTris if true, only triangles with valid UVs are included in 3D Mesh Area/Bounds
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Copy From Mesh") UDynamicMesh* 
	GetMeshUVSizeInfo(  
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptMeshSelection Selection,
		double& MeshArea,
		double& UVArea,
		FBox& MeshBounds,
		FBox2D& UVBounds,
		bool& bIsValidUVSet,
		bool& bFoundUnsetUVs,
		bool bOnlyIncludeValidUVTris = true,
		UGeometryScriptDebug* Debug = nullptr);	


	/**
	 * Get a list of single vertex UVs for each mesh vertex in the TargetMesh, derived from the specified UV Channel.
	 * The UV Channel may store multiple UVs for a single vertex (along UV seams)
	 * In such cases an arbitrary UV will be stored for that vertex, and bHasSplitUVs will be returned as true
	 * @param UVSetIndex index of UV Channel to read
	 * @param UVList output UV list will be stored here. Size will be equal to the MaxVertexID of TargetMesh  (not the VertexCount!)
	 * @param bIsValidUVSet will be set to true if the UV Channel was valid
	 * @param bHasVertexIDGaps will be set to true if some vertex indices in TargetMesh were invalid, ie MaxVertexID > VertexCount 
	 * @param bHasSplitUVs will be set to true if there were split UVs in the UV Channel
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshPerVertexUVs( 
		UDynamicMesh* TargetMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		FGeometryScriptUVList& UVList, 
		bool& bIsValidUVSet,
		bool& bHasVertexIDGaps,
		bool& bHasSplitUVs,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Copy the 2D UVs from the given UV Channel in CopyFromMesh to the 3D vertex positions in CopyToUVMesh,
	 * with the triangle mesh topology defined by the UV Channel. Generally this "UV Mesh" topology will not
	 * be the same as the 3D mesh topology. PolyGroup IDs and Material IDs are preserved in the UVMesh.
	 * 
	 * 2D UV Positions are copied to 3D as (X, Y, 0) 
	 * 
	 * CopyMeshToMeshUVChannel will copy the 3D UV Mesh back to the UV Channel. This pair of functions can
	 * then be used to implement UV generation/editing via other mesh functions.
	 * 
	 * @param bInvalidTopology will be returned true if any topological issues were found
	 * @param bIsValidUVSet will be returned false if UVSetIndex is not available
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"), DisplayName="Copy Mesh UV Channel To Mesh")
	static UE_API UPARAM(DisplayName = "Copy From Mesh") UDynamicMesh* 
	CopyMeshUVLayerToMesh(  
		UDynamicMesh* CopyFromMesh, 
		UPARAM(DisplayName = "UV Channel") int UVSetIndex,
		UPARAM(DisplayName = "Copy To UV Mesh", ref) UDynamicMesh* CopyToUVMesh, 
		UPARAM(DisplayName = "Copy To UV Mesh") UDynamicMesh*& CopyToUVMeshOut,
		bool& bInvalidTopology,
		bool& bIsValidUVSet,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Transfer the 3D vertex positions and triangles of CopyFromUVMesh to the given UV Channel identified by ToUVChannel of CopyToMesh.
	 * 3D positions (X,Y,Z) will be copied as UV positions (X,Y), ie Z is ignored.
	 * 
	 * bOnlyUVPositions controls whether only UV positions will be updated, or if the UV topology will be fully replaced.
	 * When false, CopyFromUVMesh must currently have a MaxVertexID <= that of the UV Channel MaxElementID
	 * When true, CopyFromUVMesh must currently have a MaxTriangleID <= that of CopyToMesh
	 * 
	 * @param bInvalidTopology will be returned true if any topological inconsistencies are found (but the operation will generally continue)
	 * @param bIsValidUVSet will be returned false if To UV Channel is not available
	 * @param bOnlyUVPositions if true, only (valid, matching) UV positions are updated, a full new UV topology is created
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug"), DisplayName="Copy Mesh To Mesh UV Channel")
	static UE_API UPARAM(DisplayName = "Copy From Mesh") UDynamicMesh* 
	CopyMeshToMeshUVLayer(  
		UDynamicMesh* CopyFromUVMesh, 
		UPARAM(DisplayName = "To UV Channel")  int ToUVSetIndex,
		UPARAM(DisplayName = "Copy To Mesh", ref) UDynamicMesh* CopyToMesh, 
		UPARAM(DisplayName = "Copy To Mesh") UDynamicMesh*& CopyToMeshOut,
		bool& bFoundTopologyErrors,
		UPARAM(DisplayName = "Is Valid UV Channel") bool& bIsValidUVSet,
		bool bOnlyUVPositions = true,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Compute local UV parameterization on TargetMesh vertices around the given CenterPoint / Triangle. This method
	 * uses a Discrete Exponential Map parameterization, which unwraps the mesh locally based on geodesic distances and angles.
	 * The CenterPoint will have UV value (0,0), and the computed vertex UVs will be such that Length(UV) == geodesic distance.
	 * 
	 * @param CenterPoint the center point of the parameterization. This point must lie on the triangle specified by CenterPointTriangleID
	 * @param CenterPointTriangleID the ID of the Triangle that contains CenterPoint
	 * @param Radius the parameterization will be computed out to this geodesic radius
	 * @param bUseInterpolatedNormal if true (default false), the normal frame used for the parameterization will be taken from the normal overlay, otherwise the CenterPointTriangleID normal will be used
	 * @param VertexIDs output list of VertexIDs that UVs have been computed for, ie are within geodesic distance Radius from the CenterPoint
	 * @param VertexUVs output list of Vertex UVs that corresponds to VertexIDs
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|UVs", meta=(ScriptMethod, HidePin = "Debug", AdvancedDisplay = "bUseInterpolatedNormal, TangentYDirection"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ComputeMeshLocalUVParam( 
		UDynamicMesh* TargetMesh, 
		FVector CenterPoint,
		int32 CenterPointTriangleID,
		TArray<int>& VertexIDs,
		TArray<FVector2D>& VertexUVs,
		double Radius = 1,
		bool bUseInterpolatedNormal = false,
		FVector TangentYDirection = FVector(0,0,0),
		double UVRotationDeg = 0.0,
		UGeometryScriptDebug* Debug = nullptr );

	/**
	 * Test two Box2D bounds for intersection, with optional support for working in a wrapped space
	 * 
	 * @param A First box
	 * @param B Second box
	 * @param bWrappedCoordinates Whether to test the boxes for intersection in a space wrapped to unit range of [0, 1]
	 * @return Whether the boxes intersect
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|UVs", meta = (DisplayName = "Intersects (UV Box2D)"))
	static UE_API bool IntersectsUVBox2D(FBox2D A, FBox2D B, bool bWrappedToUnitRange = false);


};

#undef UE_API
