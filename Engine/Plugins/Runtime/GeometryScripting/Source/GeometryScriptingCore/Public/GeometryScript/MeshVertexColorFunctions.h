// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshVertexColorFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UENUM(BlueprintType)
enum class EGeometryScriptBlurColorMode : uint8
{
	/** Blur the attributes where each neighbor is weighted equally. */
	Uniform = 0,
	/** Blur the attributes where each neighbor is weighted proportionally to the shared edge length. */
	EdgeLength = 1,
	/** Blur the attributes where each neighbor is weighted proportionally to the cotangent weight of the shared edge. */
	CotanWeights = 2
};

USTRUCT(BlueprintType)
struct FGeometryScriptBlurMeshVertexColorsOptions
{
	GENERATED_BODY()

	/** Blur red channel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool Red = true;

	/** Blur green channel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool Green = true;

	/** Blur blue channel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool Blue = true;

	/** Blur alpha channel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool Alpha = true;
};

UENUM(BlueprintType)
enum class ETransferVertexColorMethod : uint8
{
	/** For every vertex on the TargetMesh, find the closest point on the surface of the SourceMesh and transfer 
	 * colors from it. This is usually a point on the SourceMesh triangle where the colors are computed via 
	 * interpolation of the colors at the vertices of the triangle via barycentric coordinates.
	 */
	ClosestPointOnSurface = 0,

	/** For every vertex on the target mesh, find the closest point on the surface of the source mesh. If that point 
	 * is within the search radius (controlled via SearchPercentage), and their normals differ by less than the 
	 * NormalThreshold, then we directly copy the colors from the source point to the target mesh vertex 
	 * (same as the ClosestPointOnSurface method). For all the vertices we didn't copy the colors directly, 
	 * automatically compute the smooth colors. 
	 */
	Inpaint = 1
};

USTRUCT(BlueprintType)
struct FGeometryScriptTransferMeshVertexColorsOptions
{
	GENERATED_BODY()
	
	/** The type of algorithm to use for transferring the colors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	ETransferVertexColorMethod TransferMethod = ETransferVertexColorMethod::ClosestPointOnSurface;
	
	/** Defines the search radius as the RadiusPercentage * (input mesh bounding box diagonal). All points not within the search
	  * radius will be ignored. If negative, all points are considered. Only used in the Inpaint algorithm.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (UIMin = -1, UIMax = 2, ClampMin = -1, ClampMax = 2, EditCondition = "TransferMethod == ETransferVertexColorMethod::Inpaint"))
	double RadiusPercentage = -1;

	/** Maximum angle (in degrees) difference between the target and the source point normals to be considered a match. 
	 * If negative, normals are ignored. Only used in the Inpaint algorithm.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (UIMin = -1, UIMax = 180, ClampMin = -1, ClampMax = 180, EditCondition="TransferMethod == ETransferVertexColorMethod::Inpaint"))
	double NormalThreshold = -1;

	/** If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, Meta = (EditCondition = "TransferMethod == ETransferVertexColorMethod::Inpaint"))
	bool LayeredMeshSupport = true;

	/** The number of optional post-processing smoothing iterations applied to the vertices without the match. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100, EditCondition = "TransferMethod == ETransferVertexColorMethod::Inpaint"))
	int32 NumSmoothingIterations = 0; 

	/** The strength of each post-processing smoothing iteration. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1, EditCondition = "TransferMethod == ETransferVertexColorMethod::Inpaint"))
	float SmoothingStrength = 0.0f;

	/** Creates vertex instances per triangle to be able to have per-face vertex colors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bHardEdges = false;
	
	/** Ratio used to blend a vertex between its position and the center of the face (0 = vertex position, 1 = face centroid) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1))
	float BiasRatio = UE_KINDA_SMALL_NUMBER;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_VertexColors"))
class UGeometryScriptLibrary_MeshVertexColorFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Set all vertex colors (optionally specific channels) in the TargetMesh VertexColor Overlay to a constant value
	 * @param Color the constant color to set
	 * @param Flags specify which RGBA channels to set (default all channels)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshConstantVertexColor(
		UDynamicMesh* TargetMesh,
		FLinearColor Color,
		FGeometryScriptColorFlags Flags,
		bool bClearExisting = false,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Set the colors in the TargetMesh VertexColor Overlay identified by the Selection to a constant value.
	 * For a Vertex Selection, each existing VertexColor Overlay Element for the vertex is updated.
	 * For a Triangle or PolyGroup Selection, all Overlay Elements in the identified Triangles are updated.
	 * @param Color the constant color to set
	 * @param Flags specify which RGBA channels to set (default all channels)
	 * @param bCreateColorSeam if true, a "hard edge" in the vertex colors is created, by creating new Elements for all the triangles in the selection. If enabled, Vertex selections are converted to Triangle selections, and Flags is ignored. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshSelectionVertexColor(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		FLinearColor Color,
		FGeometryScriptColorFlags Flags,
		bool bCreateColorSeam = false,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Set all vertex colors in the TargetMesh VertexColor Overlay to the specified per-vertex colors
	 * @param VertexColorList per-vertex colors. Size must be less than or equal to the MaxVertexID of TargetMesh  (ie gaps are supported)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshPerVertexColors(
		UDynamicMesh* TargetMesh,
		FGeometryScriptColorList VertexColorList,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get a list of single vertex colors for each mesh vertex in the TargetMesh, derived from the VertexColor Overlay.
	 * The VertexColor Overlay may store multiple colors for a single vertex (ie different colors for that vertex on different triangles)
	 * In such cases the colors can either be averaged, or the last color seen will be used, depending on the bBlendSplitVertexValues parameter.
	 * @param ColorList output color list will be stored here. Size will be equal to the MaxVertexID of TargetMesh  (not the VertexCount!)
	 * @param bIsValidColorSet will be set to true if the VertexColor Overlay was valid
	 * @param bHasVertexIDGaps will be set to true if some vertex indices in TargetMesh were invalid, ie MaxVertexID > VertexCount 
	 * @param bBlendSplitVertexValues control how multiple colors at the same vertex should be interpreted
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	GetMeshPerVertexColors( 
		UDynamicMesh* TargetMesh, 
		FGeometryScriptColorList& ColorList, 
		bool& bIsValidColorSet,
		bool& bHasVertexIDGaps,
		bool bBlendSplitVertexValues = true);

	/**
	 * Apply a SRGB to Linear color transformation on all vertex colors
	 * on the mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ConvertMeshVertexColorsSRGBToLinear(
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Apply a Linear to SRGB color transformation on all vertex colors
	 * on the mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ConvertMeshVertexColorsLinearToSRGB(
		UDynamicMesh* TargetMesh,
		UGeometryScriptDebug* Debug = nullptr);


	/**
     * Blur the color attribute of the mesh. If the mesh has no color attribute, the function returns the mesh unchanged.
	 * 
     * @param TargetMesh The mesh containing the color attribute. 
     * @param Selection Only vertices in the selection will have their color attribute blurred.
     * @param NumIterations The number of blur iterations.
     * @param Strength Each iteration, we will blur between the vertex of the color at the previous iteration and its neighbors' average by Strength amount (expected to be in the zero to one range).
     * @param BlurMode Determines how neighbors are weighted when computing their average.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	BlurMeshVertexColors(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMeshSelection Selection,
		int NumIterations = 1,
		double Strength = 0.5,
		EGeometryScriptBlurColorMode BlurMode = EGeometryScriptBlurColorMode::Uniform,
		FGeometryScriptBlurMeshVertexColorsOptions Options = FGeometryScriptBlurMeshVertexColorsOptions(),
		UGeometryScriptDebug* Debug = nullptr);

	/** 
	 * Transfer the vertex colors from the SourceMesh to the TargetMesh. Assumes that the meshes are aligned. Otherwise, 
	 * use the TransformMesh geometry script function to align them.
	 * 
	 * @param SourceMesh The mesh we are transferring the colors from.
	 * @param TargetMesh The mesh we are transferring the colors to.
	 * @param Options The options to set for the transfer colors algorithm.
	 * @param Selection Optional subset of target mesh vertices to transfer weights to.
	 * If left empty, colors will be transferred to all target mesh vertices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	TransferVertexColorsFromMesh(
		UDynamicMesh* SourceMesh,
		UDynamicMesh* TargetMesh,
		FGeometryScriptTransferMeshVertexColorsOptions Options = FGeometryScriptTransferMeshVertexColorsOptions(),
		FGeometryScriptMeshSelection Selection = FGeometryScriptMeshSelection(),
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
