// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "MeshVoxelFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

UENUM(BlueprintType)
enum class EGeometryScriptGridSizingMethod : uint8
{
	GridCellSize = 0,
	GridResolution = 1
};


/***
 * Parameters for 3D grids, eg grids used for sampling, SDFs, voxelization, etc
 */
USTRUCT(BlueprintType)
struct FGeometryScript3DGridParameters
{
	GENERATED_BODY()
public:
	/** SizeMethod determines how the parameters below will be interpreted to define the size of a 3D sampling/voxel grid */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptGridSizingMethod SizeMethod = EGeometryScriptGridSizingMethod::GridResolution;

	/** Use a specific grid cell size, and construct a grid with dimensions large enough to contain the target object */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "SizeMethod == EGeometryScriptGridSizingMethod::GridCellSize"));
	float GridCellSize = 0.5;

	/** Use a specific grid resolution, with the grid cell size derived form the target object bounds such that this is the number of cells along the longest box dimension */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "SizeMethod == EGeometryScriptGridSizingMethod::GridResolution"))
	int GridResolution = 64;
};


USTRUCT(BlueprintType)
struct FGeometryScriptSolidifyOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScript3DGridParameters GridParameters;

	// If valid, will be used to define the region of space to operate on.
	// Otherwise, standard bounds based on the input mesh will be computed.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FBox CustomBounds = FBox(EForceInit::ForceInit);

	// Space with generalized winding number higher than this threshold is considered to be inside the input surface.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float WindingThreshold = 0.5;

	// If the solid surface extends beyond the bounds provided, whether to close off the surface at that boundary or leave it open
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSolidAtBoundaries = true;

	// Amount to extend bounds, applied to both min and max extents. Only applied to default input-mesh-based bounds, *not* Custom Bounds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (DisplayName = "Extend Mesh Bounds"))
	float ExtendBounds = 1.0;

	// Number of search steps to take when finding the marching cubes surface vertex positions.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int SurfaceSearchSteps = 3;

	/** When enabled, regions of the input mesh that have open boundaries (ie "shells") are thickened by extruding them into closed solids. This may be expensive on large meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bThickenShells = false;

	/** Open Shells are Thickened by offsetting vertices along their averaged vertex normals by this amount. Dimension is but clamped to twice the grid cell size. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double ShellThickness = 1.0;

};


UENUM(BlueprintType)
enum class EGeometryScriptMorphologicalOpType : uint8
{
	/** Expand the shapes outward */
	Dilate = 0,
	/** Shrink the shapes inward */
	Contract = 1,
	/** Dilate and then contract, to delete small negative features (sharp inner corners, small holes) */
	Close = 2,
	/** Contract and then dilate, to delete small positive features (sharp outer corners, small isolated pieces) */
	Open = 3

	// note: keep above compatible with TImplicitMorphology
};




USTRUCT(BlueprintType)
struct FGeometryScriptMorphologyOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScript3DGridParameters SDFGridParameters;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUseSeparateMeshGrid = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScript3DGridParameters MeshGridParameters;

	// If valid, will be used to define the region of space to operate on.
	// Otherwise, standard bounds based on the input mesh will be computed.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FBox CustomBounds = FBox(EForceInit::ForceInit);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptMorphologicalOpType Operation = EGeometryScriptMorphologicalOpType::Dilate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Distance = 1.0;

};




UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_MeshVoxelProcessing"))
class UGeometryScriptLibrary_MeshVoxelFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Replaces the mesh with a voxelized-and-meshed approximation (VoxWrap operation).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Voxel", meta = (ScriptMethod, HidePin = "Debug", Keywords = "Vox Wrap", DisplayName = "Apply Mesh Solidify (Voxel Wrap)"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyMeshSolidify(
		UDynamicMesh* TargetMesh,
		FGeometryScriptSolidifyOptions Options,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	* Replaces the mesh with an SDF-based offset mesh approximation (VoxOffset operation).
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Voxel", meta=(ScriptMethod, HidePin = "Debug", Keywords="Vox Offset Dilate Contract", DisplayName = "Apply Mesh Morphology (Voxel Offset)"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	ApplyMeshMorphology(
		UDynamicMesh* TargetMesh,
		FGeometryScriptMorphologyOptions Options,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
