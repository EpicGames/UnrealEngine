// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "MeshWeightMapFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_WeightMaps"))
class UGeometryScriptLibrary_MeshWeightMapFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Find or add a weight layer with the requested name.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* FindOrAddMeshWeightMap(
		UDynamicMesh* TargetMesh, FName Name, FGeometryScriptWeightMapHandle& WeightMapHandle,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Remove the requested weight layer, if present.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* RemoveMeshWeightMap(UDynamicMesh* TargetMesh, FGeometryScriptWeightMapHandle WeightMapHandle,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Get the values of a given mesh weight map as a scalar list
	 * 
	 * @param WeightValues The weights of each vertex in the Target Mesh
	 * @param bSkipGaps Whether to skip invalid vertex IDs in a non-compact mesh.
	 * @param bHasVertexIDGaps Whether any values in the list correspond to 'skipped' vertices. Will only be true if SkipGaps is false, and the Target Mesh is non-compact.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	GetMeshWeightMapValues(UDynamicMesh* TargetMesh, FGeometryScriptScalarList& WeightValues, FGeometryScriptWeightMapHandle WeightMapHandle, bool bSkipGaps, bool& bHasVertexIDGaps,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Set the values of a given mesh weight map from a scalar list.
	 * 
	 * @param WeightValues The weights to set on each vertex in the Target Mesh. Length must be the same as Target Mesh's Max Vertex ID, or the Vertex Count if Skip Gaps is true.
	 * @param bSkipGaps Whether to skip invalid vertex IDs in a non-compact mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|MeshQueries", meta = (ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshWeightMapValues(UDynamicMesh* TargetMesh, FGeometryScriptScalarList WeightValues, FGeometryScriptWeightMapHandle WeightMapHandle, bool bSkipGaps,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Set all weights in the given layer to a constant value
	 * @param Weight The constant value to set
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshConstantWeightMapValue(
		UDynamicMesh* TargetMesh,
		FGeometryScriptWeightMapHandle WeightMapHandle,
		float Weight,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Set the weights on selected vertices to a constant value.
	 * @param Weight The constant weight to set
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VertexColor", meta=(ScriptMethod, HidePin = "Debug"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh*
	SetMeshSelectionWeightMapValue(
		UDynamicMesh* TargetMesh,
		FGeometryScriptWeightMapHandle WeightMapHandle,
		FGeometryScriptMeshSelection Selection,
		float Weight,
		UGeometryScriptDebug* Debug = nullptr);


};

#undef UE_API
