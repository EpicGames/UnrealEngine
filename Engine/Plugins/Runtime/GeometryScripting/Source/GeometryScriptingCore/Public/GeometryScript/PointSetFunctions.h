// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "PointSetFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


UENUM(BlueprintType)
enum class EGeometryScriptInitKMeansMethod : uint8
{
	Random,
	UniformSpacing
};

USTRUCT(BlueprintType)
struct FGeometryScriptPointClusteringOptions
{
	GENERATED_BODY()
public:

	/** If not empty, will be used instead of Target Num Clusters. Specifies the initial cluster centers to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FVector> InitialClusterCenters;

	/** Number of clusters requested, if Initial Cluster Centers is empty. Actual clusters generated may be smaller, e.g. if there are fewer points */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int TargetNumClusters = 3;

	/** Method to initialize the cluster centers, if Initial Cluster Centers is empty */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptInitKMeansMethod InitializeMethod = EGeometryScriptInitKMeansMethod::UniformSpacing;

	/** Random Seed used to initialize clustering, if the Random cluster initialization is chosen */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition="InitializeMethod == EGeometryScriptInitKMeansMethod::Random"))
	int32 RandomSeed = 0;

	/** Maximum iterations to run the clustering process. Will stop earlier if/when clustering converges. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "1"))
	int32 MaxIterations = 500;
};

USTRUCT(BlueprintType)
struct FGeometryScriptPointPriorityOptions
{
	GENERATED_BODY()
public:

	/** If not empty, will be used to order the points so that higher-priority points are kept. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<float> OptionalPriorityWeights;

	/** Whether to ensure the kept points are approximately uniformly spaced */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bUniformSpacing = true;

};


USTRUCT(BlueprintType)
struct FGeometryScriptPointFlatteningOptions
{
	GENERATED_BODY()
public:

	/** Relative transform to use as a frame of reference. When flattening, the inverse transform will be applied to bring the points into the local space of the frame. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FTransform Frame = FTransform::Identity;

	/** Which axis to drop when flattening */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptAxis DropAxis = EGeometryScriptAxis::Z;
};



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_PointSetSampling"))
class UGeometryScriptLibrary_PointSetSamplingFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/**
	 * Use K-Means clustering to cluster the given points into a target number of clusters,
	 * and return an array with a cluster index per point.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet", meta = (AutoCreateRefTerm = "Options"))
	static UE_API void KMeansClusterToIDs(
		const TArray<FVector>& Points,
		const FGeometryScriptPointClusteringOptions& Options,
		TArray<int32>& PointClusterIndices);

	/**
	 * Use K-Means clustering to cluster the given points into a target number of clusters,
	 * and return the clusters as an array of lists of point indices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet", meta = (AutoCreateRefTerm = "Options"))
	static UE_API void KMeansClusterToArrays(
		const TArray<FVector>& Points,
		const FGeometryScriptPointClusteringOptions& Options,
		TArray<FGeometryScriptIndexList>& ClusterIDToLists);

	/**
	 * Find a subset of the given Points of a specified size.
	 * Can optionally specify a priorty weighting and/or request uniform spacing for the downsampled points.
	 * Note: Ordering of the result will balance:
	 * (1) if weights are provided, higher weight points come earlier and
	 * (2) if uniform spacing is requested, points will be ordered to have an octree-style coverage --
	 *     so the first 8 points will cover the 8 octants (where samples are available) and the subsequent points will progressively fill in the space
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet", meta = (HidePin = "Debug", AutoCreateRefTerm = "Options"))
	static UE_API void DownsamplePoints(
		const TArray<FVector>& Points,
		const FGeometryScriptPointPriorityOptions& Options,
		FGeometryScriptIndexList& DownsampledIndices,
		int32 KeepNumPoints = 100,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create an array of the positions of the input Transforms
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet")
	static UE_API void TransformsToPoints(
		const TArray<FTransform>& Transforms,
		TArray<FVector>& Points
	);

	/**
	 * Offset the location of all Transforms by Offset in the given Direction, either locally in the space of the transform or in world space.
	 * For example, this can offset mesh surface samples along the surface normal direction.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet")
	static UE_API void OffsetTransforms(
		UPARAM(ref) TArray<FTransform>& Transforms,
		double Offset,
		FVector Direction = FVector::UpVector,
		EGeometryScriptCoordinateSpace Space = EGeometryScriptCoordinateSpace::Local
	);

	/**
	 * Convert an array of points from 3D to 2D, by transforming into the given ReferenceFrame and taking the X,Y coordinates
	 * Note that to transform into the ReferenceFrame, we apply the inverse of the ReferenceFrame's transform.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet", meta = (AutoCreateRefTerm = "Options"))
	static UE_API void FlattenPoints(
		const TArray<FVector>& PointsIn3D,
		TArray<FVector2D>& PointsIn2D,
		const FGeometryScriptPointFlatteningOptions& Options
	);

	/**
	 * Convert an array of points from 2D to 3D, by transforming out of the given ReferenceFrame, with the given Height for the non-flat axis (default Z).
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet", meta = (AutoCreateRefTerm = "Options"))
	static UE_API void UnflattenPoints(
		const TArray<FVector2D>& PointsIn2D,
		TArray<FVector>& PointsIn3D,
		const FGeometryScriptPointFlatteningOptions& Options,
		double Height = 0.0
	);

	/**
	 * Make a Axis Aligned Bounding Box that bounds the given Points, optionally expanded by some additional amount on each side
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet")
	static UE_API UPARAM(DisplayName = "Bounding Box") FBox MakeBoundingBoxFromPoints(const TArray<FVector>& Points, double ExpandBy = 0.0);

	/**
	 * Create an array of the subset of AllPoints indicated by the Indices list
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PointSet")
	static UE_API void GetPointsFromIndexList(
		const TArray<FVector>& AllPoints,
		const FGeometryScriptIndexList& Indices,
		TArray<FVector>& SelectedPoints
	);

};

#undef UE_API
