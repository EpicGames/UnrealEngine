// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "Components/SplineComponent.h"
#include "PolyPathFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


UENUM(BlueprintType)
enum class EGeometryScriptSampleSpacing : uint8
{
	UniformDistance,
	UniformTime,
	ErrorTolerance
};

UENUM(BlueprintType)
enum class EGeometryScriptEvaluateSplineRange : uint8
{
	// Evaluate the full spline, ignoring any specified range
	FullSpline,
	// Evaluate a range specified by distances along the spline
	DistanceRange,
	// Evaluate a range specified by times, based on travelling at constant speed along the spline
	TimeRange_ConstantSpeed,
	// Evaluate a range specified by times, based on travelling at a constant rate of spline segments/second
	TimeRange_VariableSpeed
};

USTRUCT(BlueprintType)
struct FGeometryScriptSplineSamplingOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "SampleSpacing != EGeometryScriptSampleSpacing::ErrorTolerance"))
	int32 NumSamples = 10;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "SampleSpacing == EGeometryScriptSampleSpacing::ErrorTolerance"))
	float ErrorTolerance = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptSampleSpacing SampleSpacing = EGeometryScriptSampleSpacing::UniformDistance;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TEnumAsByte<ESplineCoordinateSpace::Type> CoordinateSpace = ESplineCoordinateSpace::Type::Local;

	// How the RangeStart and RangeEnd parameters will be interpreted
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptEvaluateSplineRange RangeMethod = EGeometryScriptEvaluateSplineRange::FullSpline;

	// If not evaluating the full spline, where to start sampling. Expressed in units based on the EvaluateRange value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "EvaluateRange != EGeometryScriptEvaluateSplineRange::FullSpline"))
	float RangeStart = 0;

	// If not evaluating the full spline, where to stop sampling. Expressed in units based on the EvaluateRange value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (EditCondition = "EvaluateRange != EGeometryScriptEvaluateSplineRange::FullSpline"))
	float RangeEnd = 1;

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_PolyPath"))
class UGeometryScriptLibrary_PolyPathFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/**
	* Returns the number of vertices in the the PolyPath.
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|PolyPath", meta=(ScriptMethod))
	static UE_API int GetPolyPathNumVertices(FGeometryScriptPolyPath PolyPath);

	/**
	* Returns the index of the last vertex in the PolyPath.  
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|PolyPath", meta=(ScriptMethod))
	static UE_API int GetPolyPathLastIndex(FGeometryScriptPolyPath PolyPath);

	/**
	* Returns the 3D position of the requested vertex in the PolyPath.
	* If the Index does not correspond to a vertex in the PolyPath, a Zero Vector (0,0,0) will be returned. 
	* @param Index specifies a vertex in the PolyPath.
	* @param bIsValidIndex will be false on return if the Index is not included in the PolyPath. 
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|PolyPath", meta=(ScriptMethod))
	static UE_API FVector GetPolyPathVertex(FGeometryScriptPolyPath PolyPath, int Index, bool& bIsValidIndex);

	/**
	* Returns the local tangent vector of the PolyPath at the specified vertex index.
	* If the Index does not correspond to a vertex in the PolyPath, a Zero Vector (0,0,0) will be returned. 
	* @param Index specifies a vertex in the PolyPath
	* @param bIsValidIndex will be false on return if the Index is not included in the PolyPath
	*/
	UFUNCTION(BlueprintPure, Category = "GeometryScript|PolyPath", meta = (ScriptMethod))
	static UE_API FVector GetPolyPathTangent(FGeometryScriptPolyPath PolyPath, int Index, bool& bIsValidIndex);

	/**
	* Returns the length of the PolyPath.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath", meta = (ScriptMethod))
	static UE_API double GetPolyPathArcLength(FGeometryScriptPolyPath PolyPath);

	/** Find the index of the vertex closest to a given point.  Returns -1 if the PolyPath has no vertices. */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath", meta = (ScriptMethod))
	static UE_API int32 GetNearestVertexIndex(FGeometryScriptPolyPath PolyPath, FVector Point);

	/** Create a 2D, flattened copy of the path by dropping the given axis, and using the other two coordinates as the new X, Y coordinates. */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath", meta = (ScriptMethod, DisplayName = "Flatten To 2D On Axis"))
	static UE_API UPARAM(DisplayName = "Poly Path") FGeometryScriptPolyPath FlattenTo2DOnAxis(FGeometryScriptPolyPath PolyPath, EGeometryScriptAxis DropAxis = EGeometryScriptAxis::Z);

	/**
	 * Create a closed circle around the origin on the XY plane, then transformed by Transform.
	 * By our convention for closed paths, the end vertex is *not* a duplicate of the start vertex.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API UPARAM(DisplayName = "Poly Path") FGeometryScriptPolyPath CreateCirclePath3D(FTransform Transform, float Radius = 10, int NumPoints = 10);

	/**
	 * Create an open arc around the origin on the XY plane, then transformed by Transform.
	 * As it is an open path, the end vertex exactly hits the target EndAngle (so will be positioned on the start vertex if the end aligns to the start)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API UPARAM(DisplayName = "Poly Path") FGeometryScriptPolyPath CreateArcPath3D(FTransform Transform, float Radius = 10, int NumPoints = 10, float StartAngle = 0, float EndAngle = 90);
	
	/**
	 * Create a closed circle on the XY plane around the given Center.
	 * By our convention for closed paths, the end vertex is *not* a duplicate of the start vertex.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API UPARAM(DisplayName = "Poly Path") FGeometryScriptPolyPath CreateCirclePath2D(FVector2D Center = FVector2D(0, 0), float Radius = 10, int NumPoints = 10);

	/**
	 * Create an open arc on the XY plane around the given Center.
	 * As it is an open path, the end vertex exactly hits the target EndAngle (so will be positioned on the start vertex if the end aligns to the start)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API UPARAM(DisplayName = "Poly Path") FGeometryScriptPolyPath CreateArcPath2D(FVector2D Center = FVector2D(0, 0), float Radius = 10, int NumPoints = 10, float StartAngle = 0, float EndAngle = 90);

	/**
	 * Sample positions from a USplineComponent into a PolyPath, based on the given SamplingOptions
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API void ConvertSplineToPolyPath(const USplineComponent* Spline, FGeometryScriptPolyPath& PolyPath, FGeometryScriptSplineSamplingOptions SamplingOptions);

	/**
	 * Sample a USplineComponent into a list of FTransforms, based on the given SamplingOptions.
	 * @param Frames Transforms are returned here, with X axis oriented along spline Tangent and Z as the 'Up' vector.
	 * @param FrameTimes the spline Time value used for each Frame. Note the Times Use Constant Velocity output indicates whether these times are w.r.t. to a constant-speed parameterization of the spline.
	 * @param RelativeTransform a constant Transform applied to each sample Transform in its local frame of reference. So, eg, an X Rotation will rotate each frame around the local spline Tangent vector
	 * @param bIncludeScale if true, the Scale of each FTransform is taken from the Spline, otherwise the Transforms have unit scale
	 * @return whether the FrameTimes are w.r.t. a 'constant speed' traversal of the spline
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API UPARAM(DisplayName = "Times Use Constant Velocity") bool SampleSplineToTransforms(
		const USplineComponent* Spline, 
		TArray<FTransform>& Frames, 
		TArray<double>& FrameTimes,
		FGeometryScriptSplineSamplingOptions SamplingOptions,
		FTransform RelativeTransform,
		bool bIncludeScale = true);


	/**
	* Populates an array of 3D vectors with the PolyPath vertex locations.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath", meta=(ScriptMethod))
	static UE_API void ConvertPolyPathToArray(FGeometryScriptPolyPath PolyPath, TArray<FVector>& VertexArray);

	/**
	* Creates a PolyPath from an array of 3D position vectors.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API void ConvertArrayToPolyPath(const TArray<FVector>& VertexArray, FGeometryScriptPolyPath& PolyPath);

	/**
	* Creates an array of 2D Vectors with the PolyPath vertex locations projected onto the XY plane.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath", meta = (ScriptMethod))
	static UE_API void ConvertPolyPathToArrayOfVector2D(FGeometryScriptPolyPath PolyPath, TArray<FVector2D>& VertexArray);

	/**
	* Creates a PolyPath from an array of 2D position vectors. The Z-coordinate of the corresponding PolyPath vertices will be zero.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolyPath")
	static UE_API void ConvertArrayOfVector2DToPolyPath(const TArray<FVector2D>& VertexArray, FGeometryScriptPolyPath& PolyPath);

	/**
	* Returns an array of 3D vectors with the PolyPath vertex locations.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "PolyPath To Array Of Vector", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|PolyPath")
	static UE_API TArray<FVector> Conv_GeometryScriptPolyPathToArray(FGeometryScriptPolyPath PolyPath);

	/**
	* Returns an array of 2D Vectors with the PolyPath vertex locations projected onto the XY plane.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "PolyPath To Array Of Vector2D", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|PolyPath")
	static UE_API TArray<FVector2D> Conv_GeometryScriptPolyPathToArrayOfVector2D(FGeometryScriptPolyPath PolyPath);

	/** 
	* Returns a PolyPath created from an array of 3D position vectors.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Array Of Vector To PolyPath", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|PolyPath")
	static UE_API FGeometryScriptPolyPath Conv_ArrayToGeometryScriptPolyPath(const TArray<FVector>& PathVertices);

	/**
	* Returns a PolyPath created from an array of 2D position vectors. The Z-coordinate of the corresponding PolyPath vertices will be zero.
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Array Of Vector2D To PolyPath", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|PolyPath")
	static UE_API FGeometryScriptPolyPath Conv_ArrayOfVector2DToGeometryScriptPolyPath(const TArray<FVector2D>& PathVertices);
};

#undef UE_API
