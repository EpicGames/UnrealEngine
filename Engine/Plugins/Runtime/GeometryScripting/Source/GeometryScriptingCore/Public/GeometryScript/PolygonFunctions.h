// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "Components/SplineComponent.h"
#include "PolyPathFunctions.h" // For spline sampling options
#include "PolygonFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

// Join types to define the shape of corners between polygon and polypath edges
UENUM(BlueprintType)
enum class EGeometryScriptPolyOffsetJoinType : uint8
{
	/* Uniform squaring on all convex edge joins. */
	Square,
	/* Arcs on all convex edge joins. */
	Round,
	/* Squaring of convex edge joins with acute angles ("spikes"). Use in combination with MiterLimit. */
	Miter,
};

// End types to apply when offsetting open paths
UENUM(BlueprintType)
enum class EGeometryScriptPathOffsetEndType : uint8
{
	/* Offsets both sides of a path, with square blunt ends */
	Butt,
	/* Offsets both sides of a path, with square extended ends */
	Square,
	/* Offsets both sides of a path, with round extended ends */
	Round,
};

USTRUCT(BlueprintType)
struct FGeometryScriptPolygonOffsetOptions
{
	GENERATED_BODY()
public:

	// How to join / extend corners between two edges
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolyOffsetJoinType JoinType = EGeometryScriptPolyOffsetJoinType::Square;

	// if JoinType is Miter, limits how far the miter can extend
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double MiterLimit = 2.0;

	// Whether to apply the offset to both sides of the polygon, i.e. adding an inner hole to any polygon. If false, the offset is only applied to one side.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bOffsetBothSides = false;

	// Scales the default number of vertices (per radian) used for round joins.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "0"))
	double StepsPerRadianScale = 1.0;

	// Maximum vertices per radian for round joins. Only applied if > 0.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "-1"))
	double MaximumStepsPerRadian = 10.0;
};


USTRUCT(BlueprintType)
struct FGeometryScriptOpenPathOffsetOptions
{
	GENERATED_BODY()
public:

	// How to join / extend corners between two edges
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPolyOffsetJoinType JoinType = EGeometryScriptPolyOffsetJoinType::Square;

	// if JoinType is Miter, limits how far the miter can extend
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double MiterLimit = 2.0;

	// How the ends of a path should be closed off
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPathOffsetEndType EndType = EGeometryScriptPathOffsetEndType::Square;

	// Scales the default number of vertices (per radian) used for round joins and ends.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "0"))
	double StepsPerRadianScale = 1.0;

	// Maximum vertices per radian for round joins and ends. Only applied if > 0.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options, meta = (ClampMin = "-1"))
	double MaximumStepsPerRadian = 10.0;

};

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_SimplePolygon"))
class UGeometryScriptLibrary_SimplePolygonFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Returns the number of vertices in a Simple Polygon
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API int32 GetPolygonVertexCount(FGeometryScriptSimplePolygon Polygon);

	/**
	 * Returns the specified vertex of a Simple Polygon. VertexIndex loops around, so e.g., -1 gives the last vertex in the polygon.
	 * If Polygon has no vertices, returns the zero vector.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API FVector2D GetPolygonVertex(FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, bool& bPolygonIsEmpty);

	/**
	 * Set the specified vertex of a Simple Polygon. VertexIndex loops around, so e.g., -1 gives the last vertex in the polygon.
	 * Does nothing if Polygon has no vertices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API void SetPolygonVertex(UPARAM(ref) FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, FVector2D Position, bool& bPolygonIsEmpty);

	/**
	 * Set the specified vertex of a Simple Polygon. Returns the index of the added vertex.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Vertex Index") int32 AddPolygonVertex(UPARAM(ref) FGeometryScriptSimplePolygon& Polygon, FVector2D Position);

	/**
	 * Returns a vertex's tangent of a Simple Polygon. VertexIndex loops around, so e.g., -1 gives the tangent of the last vertex in the polygon.
	 * If Polygon has no vertices, returns the zero vector.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API FVector2D GetPolygonTangent(FGeometryScriptSimplePolygon Polygon, int32 VertexIndex, bool& bPolygonIsEmpty);

	/**
	 * Returns the arc length of a Simple Polygon
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API double GetPolygonArcLength(FGeometryScriptSimplePolygon Polygon);

	/**
	 * Returns the area enclosed by a Simple Polygon
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod, DisplayName = "Get Simple Polygon Area"))
	static UE_API double GetPolygonArea(FGeometryScriptSimplePolygon Polygon);

	/**
	 * Returns the bounding box of a Simple Polygon
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon", meta = (ScriptMethod))
	static UE_API FBox2D GetPolygonBounds(FGeometryScriptSimplePolygon Polygon);

	/**
	 * Sample positions from a USplineComponent into a Simple Polyon, based on the given SamplingOptions
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|SimplePolygon")
	static UE_API void ConvertSplineToPolygon(const USplineComponent* Spline, FGeometryScriptSimplePolygon& Polygon, FGeometryScriptSplineSamplingOptions SamplingOptions, EGeometryScriptAxis DropAxis = EGeometryScriptAxis::Z);

	/**
	 * Returns an array of 3D vectors with the Polygon vertex locations, with Z coordinate set to zero.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Simple Polygon To Array Of Vector", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|SimplePolygon")
	static UE_API TArray<FVector> Conv_GeometryScriptSimplePolygonToArray(FGeometryScriptSimplePolygon Polygon);

	/**
	 * Returns an array of 2D Vectors with the Polygon vertex locations.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Simple Polygon To Array Of Vector2D", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|SimplePolygon")
	static UE_API TArray<FVector2D> Conv_GeometryScriptSimplePolygonToArrayOfVector2D(FGeometryScriptSimplePolygon Polygon);

	/** 
	 * Returns a Polygon created from an array of 3D position vectors, ignoring the Z coordinate.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Array Of Vector To Simple Polygon", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|SimplePolygon")
	static UE_API FGeometryScriptSimplePolygon Conv_ArrayToGeometryScriptSimplePolygon(const TArray<FVector>& PathVertices);

	/**
	 * Returns a Polygon created from an array of 2D position vectors.
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Array Of Vector2D To Simple Polygon", CompactNodeTitle = "->", BlueprintAutocast), Category = "GeometryScript|SimplePolygon")
	static UE_API FGeometryScriptSimplePolygon Conv_ArrayOfVector2DToGeometryScriptSimplePolygon(const TArray<FVector2D>& PathVertices);
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_PolygonList"))
class UGeometryScriptLibrary_PolygonListFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Returns the number of polygons in the Polygon List
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API int32 GetPolygonCount(FGeometryScriptGeneralPolygonList PolygonList);

	/**
	 * Returns the number of holes in a Polygon. Returns zero for an invalid PolygonIndex.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API int32 GetPolygonHoleCount(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int32 PolygonIndex);

	/**
	 * Returns the number of vertices in a Polygon's outer shape, if HoleIndex is -1, or in the specified inner hole.
	 * Returns 0 for invalid Polygon or Hole indices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta=(ScriptMethod))
	static UE_API int32 GetPolygonVertexCount(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndices, int32 PolygonIndex, int32 HoleIndex = -1);

	/**
	 * Returns the vertices of a Polygon -- either of the outer polygon, if HoleIndex is -1, or specified inner hole.
	 * OutVertices will be empty for invalid Polygon or Hole indices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API void GetPolygonVertices(FGeometryScriptGeneralPolygonList PolygonList, TArray<FVector2D>& OutVertices, bool& bValidIndices, int32 PolygonIndex, int32 HoleIndex = -1);

	/**
	 * Returns a specified Simple Polygon from a Polygon List -- either the outer polygon, if HoleIndex is -1, or specified inner hole.
	 * Polygon will be empty for invalid Polygon or Hole indices.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptSimplePolygon GetSimplePolygon(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndices, int32 PolygonIndex, int32 HoleIndex = -1);

	/**
	 * Returns the specified vertex of a Polygon -- either of the outer polygon, if HoleIndex is -1, or specified inner hole.
	 * Vertex will be the zero vector for invalid Polygon or Hole indices, or if the polygon is empty. VertexIndex will loop.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FVector2D GetPolygonVertex(FGeometryScriptGeneralPolygonList PolygonList, bool& bIsValidVertex, int32 VertexIndex, int32 PolygonIndex, int32 HoleIndex = -1);

	/**
	 * Returns the area enclosed by a specified Polygon in a Polygon List. Returns zero for an invalid PolygonIndex. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod, DisplayName = "Get Area of Polygon in List"))
	static UE_API double GetPolygonArea(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int32 PolygonIndex);

	/**
	 * Returns the bounding box of a Polygon. Returns an empty, invalid box for an invalid PolygonIndex.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FBox2D GetPolygonBounds(FGeometryScriptGeneralPolygonList PolygonList, bool& bValidIndex, int32 PolygonIndex);

	/**
	 * Returns the area enclosed by a Polygon List
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API double GetPolygonListArea(FGeometryScriptGeneralPolygonList PolygonList);

	/**
	 * Returns the bounding box of a Polygon List
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FBox2D GetPolygonListBounds(FGeometryScriptGeneralPolygonList PolygonList);

	/**
	 * Create a Polygon List of a single Polygon, with optional holes
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod, AutoCreateRefTerm = "HolePolygons"))
	static UE_API UPARAM(DisplayName = "Polygon List") FGeometryScriptGeneralPolygonList CreatePolygonListFromSinglePolygon(FGeometryScriptSimplePolygon OuterPolygon, const TArray<FGeometryScriptSimplePolygon>& HolePolygons, bool bFixHoleOrientations = true);

	/**
	 * Add Polygon to a Polygon List, with optional holes. Returns index of the added polygon.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod, AutoCreateRefTerm = "HolePolygons"))
	static UE_API UPARAM(DisplayName = "Polygon Index") int32 AddPolygonToList(UPARAM(ref) FGeometryScriptGeneralPolygonList& PolygonList, FGeometryScriptSimplePolygon OuterPolygon, const TArray<FGeometryScriptSimplePolygon>& HolePolygons, bool bFixHoleOrientations = true);

	/**
	 * Create a Polygon List from an array of Simple Polygons
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList")
	static UE_API UPARAM(DisplayName = "Polygon List") FGeometryScriptGeneralPolygonList CreatePolygonListFromSimplePolygons(const TArray<FGeometryScriptSimplePolygon>& OuterPolygons);

	/**
	 * Append the polygons in 'Polygons to Append' to Polygon List
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API void AppendPolygonList(UPARAM(ref) FGeometryScriptGeneralPolygonList& PolygonList, FGeometryScriptGeneralPolygonList PolygonsToAppend);
	
	/** Compute union of all polygons in Polygon List. Also resolves self-intersections within each polygon. */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsUnion(FGeometryScriptGeneralPolygonList PolygonList, bool bCopyInputOnFailure = true);

	/** Compute difference of Polygon List and Polygons to Subtract */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsDifference(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToSubtract);
	
	/** Compute intersection of Polygon List and Polygons to Intersect */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsIntersection(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToIntersect);
	
	/** Compute exclusive or of Polygon List and Polygons to Exclusive Or */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsExclusiveOr(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptGeneralPolygonList PolygonsToExclusiveOr);

	/** Apply a single offset to a list of closed polygons */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsOffset(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
		double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);

	/** Apply two offsets in sequence to a list of closed polygons */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsOffsets(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
		double FirstOffset, double SecondOffset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);

	/** Apply a morphological "open" operator to a list of closed polygons -- first offsetting by -Offset, then by +Offset. If Offset is negative, this will instead function as a 'Close' operation */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsMorphologyOpen(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
		double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);

	/** Apply a morphological "close" operator to a list of closed polygons -- first offsetting by +Offset, then by -Offset. If Offset is negative, this will instead function as an 'Open' operation */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList", meta = (ScriptMethod))
	static UE_API FGeometryScriptGeneralPolygonList PolygonsMorphologyClose(FGeometryScriptGeneralPolygonList PolygonList, FGeometryScriptPolygonOffsetOptions OffsetOptions,
		double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);

	/** Apply an offset to a single open 2D path, generating closed polygons as a result */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList")
	static UE_API FGeometryScriptGeneralPolygonList CreatePolygonsFromPathOffset(TArray<FVector2D> Path, FGeometryScriptOpenPathOffsetOptions OffsetOptions,
		double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);
	
	/** Apply an offset to a set of open 2D PolyPaths, generating closed polygons as a result */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|PolygonList")
	static UE_API FGeometryScriptGeneralPolygonList CreatePolygonsFromOpenPolyPathsOffset(TArray<FGeometryScriptPolyPath> PolyPaths, FGeometryScriptOpenPathOffsetOptions OffsetOptions,
		double Offset, bool& bOperationSuccess, bool bCopyInputOnFailure = true);
	
};

#undef UE_API
