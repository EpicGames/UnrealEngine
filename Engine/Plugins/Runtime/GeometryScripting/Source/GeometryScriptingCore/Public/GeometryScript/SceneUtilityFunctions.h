// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "SceneUtilityFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UStaticMesh;
class UMaterialInterface;
class UDynamicMesh;
class UDynamicMeshPool;

USTRUCT(BlueprintType)
struct FGeometryScriptCopyMeshFromComponentOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWantNormals = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWantTangents = true;

	// Whether to request per-instance vertex colors (where applicable; applies to RenderData LODs of Static Mesh components)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWantInstanceColors = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FGeometryScriptMeshReadLOD RequestedLOD = FGeometryScriptMeshReadLOD();
};

USTRUCT(BlueprintType)
struct FGeometryScriptDetermineMeshOcclusionOptions
{
	GENERATED_BODY()
public:

	// Approximate spacing between samples on triangle faces used for determining visibility
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	double SamplingDensity = 1.0;

	// Whether to treat faces as double-sided when determining visibility
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDoubleSided = false;

	// Number of directions to test for visibility
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	int32 NumSearchDirections = 128;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_SceneUtils"))
class UGeometryScriptLibrary_SceneUtilityFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create a new UDynamicMeshPool object. 
	 * Caller needs to create a UProperty reference to the returned object, or it will be garbage-collected.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Utility")
	static UE_API UPARAM(DisplayName = "Dynamic Mesh Pool") UDynamicMeshPool*
	CreateDynamicMeshPool();


	/**
	 * Copy the mesh from a given Component to a Dynamic Mesh.
	 * Supported Component types are:
	 * StaticMeshComponent, SkinnedMeshComponent, SplineMeshComponent, DynamicMeshComponent, 
	 * GeometryCacheComponent, GeometryCollectionComponent, and BrushComponent
	 * This function offers minimal control over the copying. If more control is needed for Static Meshes, use CopyMeshFromStaticMesh.
	 * @param bTransformToWorld if true, output mesh is in World space
	 * @param LocalToWorld local-to-world transform is returned here (whether mesh is in local or world space)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Scene", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh*
	CopyMeshFromComponent(
		USceneComponent* Component,
		UDynamicMesh* ToDynamicMesh,
		FGeometryScriptCopyMeshFromComponentOptions Options,
		bool bTransformToWorld,
		FTransform& LocalToWorld,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Configure the Material set on a PrimitiveComponent, by repeatedly calling SetMaterial.
	 * This is a simple utility function and it's behavior will depend on the specifics of the Component.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Scene", meta = (HidePin = "Debug"))
	static UE_API void
	SetComponentMaterialList(
		UPrimitiveComponent* Component, 
		const TArray<UMaterialInterface*>& MaterialList,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Extract the Collision Geometry from FromObject and copy/approximate it with meshes stored in ToDynamicMesh.
	 * For Simple Collision, FromObject can be a StaticMesh Asset or any PrimitiveComponent
	 * For Complex Collision, FromObject can be a Static/Skeletal Mesh Asset/Component, Dynamic Mesh Component, or other Component w/ complex collision support
	 * @param bTransformToWorld if true, output mesh is in World space
	 * @param LocalToWorld local-to-world transform is returned here (whether mesh is in local or world space)
	 * @param bUseComplexCollision if true, complex collision is extracted, otherwise Simple collision shapes are meshed
	 * @param SphereResolution determines tessellation density of sphere and capsule shapes
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Scene", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh*
	CopyCollisionMeshesFromObject(
		UObject* FromObject,
		UDynamicMesh* ToDynamicMesh,
		bool bTransformToWorld,
		FTransform& LocalToWorld,
		EGeometryScriptOutcomePins& Outcome,
		bool bUseComplexCollision = false,
		int SphereResolution = 16,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Determine which meshes are entirely hidden by other meshes in the set, when viewed from outside.
	 * 
	 * @param SourceMeshes			Meshes to test for occlusion. Note: The same mesh may appear multiple times in this array, if it is instanced with different transforms.
	 * @param SourceMeshTransforms	A transform for each source mesh. Array must have the same length as SourceMeshes.
	 * @param OutMeshIsHidden		Array will be filled with a bool per source mesh, indicating whether that mesh is hidden (true) or visible (false)
	 * @param TransparentMeshes		Transparent source meshes, to test for occlusion but which do not occlude.
	 * @param TransparentMeshTransforms		Array of transforms for each transparent mesh
	 * @param OutTransparentMeshIsHidden	Array will be filled with a bool per transparent mesh, indicating whether that mesh is hidden (true) or visible (false)
	 * @param OccludeMeshes			Array of optional meshes which can occlude SourceMeshes, but for which we will not test occlusion.
	 * @param OccludeMeshTransforms	Array of transforms for each occlude mesh. Array must have the same length as OccludeMeshes.
	 * @param Options				Settings to control how occlusion is tested
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Scene", meta = (HidePin = "Debug", AutoCreateRefTerm = "SourceMeshes, SourceMeshTransforms, TransparentMeshes, TransparentMeshTransforms, OccludeMeshes, OccludeMeshTransforms, OcclusionOptions"))
	static UE_API void DetermineMeshOcclusion(
		const TArray<UDynamicMesh*>& SourceMeshes,
		const TArray<FTransform>& SourceMeshTransforms,
		TArray<bool>& OutMeshIsHidden,
		const TArray<UDynamicMesh*>& TransparentMeshes,
		const TArray<FTransform>& TransparentMeshTransforms,
		TArray<bool>& OutTransparentMeshIsHidden,
		const TArray<UDynamicMesh*>& OccludeMeshes,
		const TArray<FTransform>& OccludeMeshTransforms,
		const FGeometryScriptDetermineMeshOcclusionOptions& OcclusionOptions,
		UGeometryScriptDebug* Debug = nullptr);
};

#undef UE_API
