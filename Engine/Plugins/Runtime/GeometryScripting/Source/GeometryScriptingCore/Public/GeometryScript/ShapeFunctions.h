// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScriptTypes.h"
#include "Math/Box.h"
#include "ShapeFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API



UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Transform"))
class UGeometryScriptLibrary_TransformFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create a Transform at the given Location, with the ZAxis vector as the Z axis of the Transform.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Transform")
	static UE_API UPARAM(DisplayName="Transform") FTransform
	MakeTransformFromZAxis(FVector Location, FVector ZAxis);

	/**
	 * Create a Transform at the given Location, with the ZAxis vector as the Z axis
	 * of the Transform, and the X or Y axis oriented to the Tangent vector, based on
	 * the bTangentIsX parameter.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Transform")
	static UE_API UPARAM(DisplayName="Transform") FTransform
	MakeTransformFromAxes(FVector Location, FVector ZAxis, FVector TangentAxis, bool bTangentIsX = true);

	/**
	 * Get the Vector for the direction of the X/Y/Z axis of the Transform, ie the 
	 * Vector resulting from transforming the unit direction vectors (1,0,0) / etc  
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Transform")
	static UE_API UPARAM(DisplayName="Direction") FVector
	GetTransformAxisVector(FTransform Transform, EGeometryScriptAxis Axis = EGeometryScriptAxis::X);

	/**
	 * Get the Ray at the Transform Location aligned with the direction of the X/Y/Z axis of the Transform, 
	 * ie the Direction Vector resulting from transforming the unit direction vectors (1,0,0) / etc  
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Transform")
	static UE_API UPARAM(DisplayName="Ray") FRay
	GetTransformAxisRay(FTransform Transform, EGeometryScriptAxis Axis = EGeometryScriptAxis::X);

	/**
	 * Get the Plane at the Transform Location with the Plane Normal aligned with the direction of the X/Y/Z axis of the Transform, 
	 * ie the Direction Vector resulting from transforming the unit direction vectors (1,0,0) / etc  
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Transform")
	static UE_API UPARAM(DisplayName="Ray") FPlane
	GetTransformAxisPlane(FTransform Transform, EGeometryScriptAxis Axis = EGeometryScriptAxis::X);

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Ray"))
class UGeometryScriptLibrary_RayFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create a Ray from two points, placing the Origin at A and the Direction as Normalize(B-A)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray")
	static UE_API UPARAM(DisplayName="Ray") FRay
	MakeRayFromPoints(FVector A, FVector B);

	/**
	 * Create a Ray from an Origin and Direction, with optionally non-normalized Direction
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray")
	static UE_API UPARAM(DisplayName="Ray") FRay 
	MakeRayFromPointDirection(FVector Origin, FVector Direction, bool bDirectionIsNormalized = true);

	/**
	 * Apply the given Transform to the given Ray, or optionally the Transform Inverse, and return the new transformed Ray
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Transformed Ray") FRay 
	GetTransformedRay(FRay Ray, FTransform Transform, bool bInvert = false);

	/**
	 * Get a Point at the given Distance along the Ray (Origin + Distance*Direction)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Point") FVector 
	GetRayPoint(FRay Ray, double Distance);

	/**
	 * Get two points along the ray. 
	 * @param StartPoint returned as Origin + StartDistance*Direction
	 * @param EndPoint returned as Origin + EndDistance*Direction, Unless EndDistance = 0, then MaxFloat is used as the Distance
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod, StartDistance="0", EndDistance = "0"))
	static UE_API void 
	GetRayStartEnd(FRay Ray, double StartDistance, double EndDistance, FVector& StartPoint, FVector& EndPoint);

	/**
	 * Project the given Point onto the closest point along the Ray, and return the Ray Parameter/Distance at that Point
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Ray Paramater") double 
	GetRayParameter(FRay Ray, FVector Point);

	/**
	 * Get the distance from Point to the closest point on the Ray
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Distance") double 
	GetRayPointDistance(FRay Ray, FVector Point);

	/**
	 * Get the closest point on the Ray to the given Point
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Closest Point") FVector 
	GetRayClosestPoint(FRay Ray, FVector Point);

	/**
	 * Check if the Ray intersects a Sphere defined by the SphereCenter and SphereRadius.
	 * This function returns two intersection distances (ray parameters). If the ray grazes the sphere, both
	 * distances will be the same, and if it misses, they will be MAX_FLOAT. 
	 * Use the function GetRayPoint to convert the distances to points on the ray/sphere.
	 * 
	 * @param Distance1 Distance along ray (Ray Parameter) to first/closer intersection point with sphere
	 * @param Distance2 Distance along ray (Ray Parameter) to second/further intersection point with sphere
	 * @return true if ray intersects sphere
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersects") bool 
	GetRaySphereIntersection(FRay Ray, FVector SphereCenter, double SphereRadius, double& Distance1, double& Distance2);

	/**
	 * Find the intersection of a Ray and a Box
	 * @param HitDistance Distance along the ray (Ray Parameter) to first intersection point with the Box
	 * @return true if the ray hits the box, and false otherwise
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersects") bool 
	GetRayBoxIntersection(FRay Ray, FBox Box, double& HitDistance);

	/**
	 * Find the intersection of a Ray and a Plane
	 * @param HitDistance the returned Distance along the ray (Ray Parameter) to intersection point with the Plane
	 * @return true if the ray hits the plane (only false if ray is parallel with plane)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersects") bool 
	GetRayPlaneIntersection(FRay Ray, FPlane Plane, double& HitDistance);

	/**
	 * Compute the pair of closest points on a 3D Ray and Line. 
	 * The Line is defined by an Origin and Direction (ie same as a Ray) but extends infinitely in both directions.
	 * @param RayParameter the Ray Parameter of the closest point on the Ray (range 0, inf)
	 * @param RayPoint the point on the Ray corresponding to RayParameter
	 * @param LineParameter the Line parameter of the closest point on the Line (range -inf, inf)
	 * @param LinePoint the point on the Line corresponding to LineParameter
	 * @return the minimum distance between the Ray and Line, ie between RayPoint and LinePoint
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Distance") double
	GetRayLineClosestPoint(FRay Ray, FVector LineOrigin, FVector LineDirection, double& RayParameter, FVector& RayPoint, double& LineParameter, FVector& LinePoint);

	/**
	 * Compute the pair of closest points on a 3D Ray and Line Segment
	 * The Line Segment is defined by its two Endpoints.
	 * @param RayParameter the Ray Parameter of the closest point on the Ray (range 0, inf)
	 * @param RayPoint the point on the Ray corresponding to RayParameter
	 * @param SegPoint the point on the Segment
	 * @return the minimum distance between the Ray and Segment, ie between RayPoint and SegPoint
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Ray", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Distance") double
	GetRaySegmentClosestPoint(FRay Ray, FVector SegStartPoint, FVector SegEndPoint, double& RayParameter, FVector& RayPoint, FVector& SegPoint);

};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_Box"))
class UGeometryScriptLibrary_BoxFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create a Box from a Center point and X/Y/Z Dimensions (*not* Extents, which are half-dimensions)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box")
	static UE_API UPARAM(DisplayName="Box") FBox 
	MakeBoxFromCenterSize(FVector Center, FVector Dimensions);

	/**
	 * Create a Box from a Center point and X/Y/Z Extents (Extents are half-dimensions)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box")
	static UE_API UPARAM(DisplayName="Box") FBox 
	MakeBoxFromCenterExtents(FVector Center, FVector Extents);

	/**
	 * Get the Center point and X/Y/Z Dimensions of a Box (full dimensions, not Extents)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API void GetBoxCenterSize(FBox Box, FVector& Center, FVector& Dimensions);


	/**
	 * Get the position of a corner of the Box. Corners are indexed from 0 to 7, using
	 * an ordering where 0 is the Min corner, 1/2/3 are +Z/+Y/+X from the Min corner, 
	 * 7 is the Max corner, and 4/5/6 are -Z/-Y/-X from the Max corner.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Corner Point") FVector 
	GetBoxCorner(FBox Box, int CornerIndex);

	/**
	 * Get the position of the center of a face of the Box. Faces are indexed from 0 to 5,
	 * using an ordering where 0/1 are the MinZ/MaxZ faces, 2/3 are MinY/MaxY, and 4/5 are MinX/MaxX
	 * @param FaceNormal returned Normal vector of the identified face
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Center Point") FVector 
	GetBoxFaceCenter(FBox Box, int FaceIndex, FVector& FaceNormal);

	/**
	 * Get the Volume and Surface Area of a Box
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API void GetBoxVolumeArea(FBox Box, double& Volume, double& SurfaceArea);

	/**
	 * Get the input Box expanded by adding the ExpandBy parameter to both the Min and Max.
	 * Dimensions will be clamped to the center point if any of ExpandBy are larger than half the box size
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Expanded Box") FBox 
	GetExpandedBox(FBox Box, FVector ExpandBy);

	/**
	 * Apply the input Transform to the corners of the input Box, and return the new Box containing those points
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Transformed Box") FBox 
	GetTransformedBox(FBox Box, FTransform Transform);

	/**
	 * Test if Box1 and Box2 intersect
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersects") bool 
	TestBoxBoxIntersection(FBox Box1, FBox Box2);

	/**
	 * Find the Box formed by the intersection of Box1 and Box2
	 * @param bIsIntersecting if the boxes do not intersect, this will be returned as false, otherwise true
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersection Box") FBox 
	FindBoxBoxIntersection(FBox Box1, FBox Box2, bool& bIsIntersecting);

	/**
	 * Calculate the minimum distance between Box1 and Box2
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Distance") double 
	GetBoxBoxDistance(FBox Box1, FBox Box2);

	/**
	 * Test if a Point is inside the Box, returning true if so, otherwise false
	 * @param bConsiderOnBoxAsInside if true, a point lying on the box face is considered "inside", otherwise it is considered "outside"
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Is Inside") bool 
	TestPointInsideBox(FBox Box, FVector Point, bool bConsiderOnBoxAsInside = true);

	/**
	 * Find the point on the faces of the Box that is closest to the input Point.
	 * If the Point is inside the Box, it is returned, ie points Inside do not project to the Box Faces
	 * @param bIsInside if the Point is inside the Box, this will return as true, otherwise false
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Closest Point") FVector 
	FindClosestPointOnBox(FBox Box, FVector Point, bool& bIsInside);

	/**
	 * Calculate the minimum distance between the Box and the Point
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Distance") double 
	GetBoxPointDistance(FBox Box, FVector Point);

	/**
	 * Check if the Box intersects a Sphere defined by the SphereCenter and SphereRadius
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|Box", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Intersects") bool 
	TestBoxSphereIntersection(FBox Box, FVector SphereCenter, double SphereRadius);

};

// Oriented Box methods
UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_OrientedBox"))
class UGeometryScriptLibrary_OrientedBoxFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Create an Oriented Box from a Box and a Transform
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta = (KeyWords = "Transform"))
	static UE_API UPARAM(DisplayName="OrientedBox") FOrientedBox 
	MakeOrientedBoxFromBox(FBox Box, FTransform Transform);

	/**
	 * Get the orientation of an Oriented Box
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod, KeyWords = "Rotation"))
	static UE_API UPARAM(DisplayName = "Orientation") FRotator GetOrientedBoxOrientation(FOrientedBox Box);

	/**
	 * Get the position of a corner of the Oriented Box. Corners are indexed from 0 to 7:
	 * Corner 0 is in the negative direction along each axis, 1/2/3 are along +Z/+Y/+X from the corner 0,
	 * 7 is the all-positive direction corner, and 4/5/6 are along -Z/-Y/-X from the corner 7.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Corner Point") FVector 
	GetOrientedBoxCorner(FOrientedBox Box, int CornerIndex);

	/**
	 * Get the position of the center of a face of the Oriented Box. Faces are indexed from 0 to 5,
	 * using an ordering where 0/1 are the negative then positive Z-axis faces, 2/3 are Y-axis faces, and 4/5 are X-axis faces
	 * @param FaceNormal returned Normal vector of the identified face
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Center Point") FVector 
	GetOrientedBoxFaceCenter(FOrientedBox Box, int FaceIndex, FVector& FaceNormal);

	/**
	 * Get the Volume and Surface Area of an Oriented Box
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod))
	static UE_API void GetOrientedBoxVolumeArea(FOrientedBox Box, double& Volume, double& SurfaceArea);

	/**
	 * Get the Oriented Box with extents expanded by the ExpandBy vector
	 * Can also shrink the box if negative ExpandBy values are provided, but will clamp to Max(0, MinExtent)
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod, KeyWords = "Shrink"))
	static UE_API UPARAM(DisplayName="Expanded Oriented Box") FOrientedBox 
	GetExpandedOrientedBox(FOrientedBox OrientedBox, FVector ExpandBy, double MinExtent = 0);

	/**
	 * Apply the Transform to the Oriented Box
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod))
	static UE_API UPARAM(DisplayName="Transformed Box") FOrientedBox 
	GetTransformedOrientedBox(FOrientedBox OrientedBox, FTransform Transform);

	/**
	 * Find the bounding Box enclosing the input Oriented Box.
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta=(ScriptMethod, KeyWords = "Bounding"))
	static UE_API UPARAM(DisplayName="Bounding Box") FBox 
	ConvertOrientedBoxToAxisAlignedBox(FOrientedBox Box);

	/**
	 * Test if a Point is inside or on the Box, returning true if so, otherwise false
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta = (ScriptMethod))
	static UE_API UPARAM(DisplayName = "Is Inside") bool
	TestPointInsideOrOnOrientedBox(FOrientedBox Box, FVector Point);

	/**
	 * Find the point on the faces of the Box that is closest to the input Point.
	 * If the Point is inside the Box, it is returned, ie points Inside do not project to the Box Faces
	 * @param bIsInsideOrOn if the Point is inside or on the Box, this will return as true, otherwise false
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta = (ScriptMethod, KeyWords = "Project"))
	static UE_API UPARAM(DisplayName = "Closest Point") FVector
	FindClosestPointOnOrientedBox(FOrientedBox Box, FVector Point, bool& bIsInsideOrOn);

	/**
	 * Test if two oriented boxes intersect
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta = (ScriptMethod, KeyWords = "Overlaps"))
	static UE_API UPARAM(DisplayName = "Intersects") bool
	TestOrientedBoxOrientedBoxIntersection(FOrientedBox Box1, FOrientedBox Box2);

	/**
	 * Test if an axis-aligned box intersects an oriented box
	 */
	UFUNCTION(BlueprintPure, Category = "GeometryScript|Shapes|OrientedBox", meta = (ScriptMethod, KeyWords = "Overlaps"))
	static UE_API UPARAM(DisplayName = "Intersects") bool
	TestBoxOrientedBoxIntersection(FBox Box, FOrientedBox OrientedBox);
};

#undef UE_API
