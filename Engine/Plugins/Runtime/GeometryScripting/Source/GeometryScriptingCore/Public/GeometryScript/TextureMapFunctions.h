// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "TextureMapFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API


UENUM(BlueprintType)
enum class EGeometryScriptPixelSamplingMethod : uint8
{
	Bilinear = 0,
	Nearest = 1
};




USTRUCT(BlueprintType)
struct FGeometryScriptSampleTextureOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptPixelSamplingMethod SamplingMethod = EGeometryScriptPixelSamplingMethod::Bilinear;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bWrap = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UVScale = FVector2D(1,1);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FVector2D UVOffset = FVector2D(0,0);
};




UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_TextureUtils"))
class UGeometryScriptLibrary_TextureMapFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Samples the given TextureMap at the list of UV positions and returns the color at each position in ColorList output.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|TextureUtils", meta = (HidePin = "Debug"))
	static UE_API void 
	SampleTexture2DAtUVPositions(  
		FGeometryScriptUVList UVList, 
		UTexture2D* Texture,
		FGeometryScriptSampleTextureOptions SampleOptions,
		FGeometryScriptColorList& ColorList,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Sample the the given TextureMap at the list of UV positions and return the color at each position in ColorList output.
	 * This function fetches GPU data before sampling so, depending on your application, it can be inefficient and slow!
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|TextureUtils", meta = (HidePin = "Debug"))
	static UE_API void 
	SampleTextureRenderTarget2DAtUVPositions(  
		FGeometryScriptUVList UVList, 
		UTextureRenderTarget2D* Texture,
		FGeometryScriptSampleTextureOptions SampleOptions,
		FGeometryScriptColorList& ColorList,
		UGeometryScriptDebug* Debug = nullptr);


};

#undef UE_API
