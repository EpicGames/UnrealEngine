// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "VectorMathFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_VectorMath"))
class UGeometryScriptLibrary_VectorMathFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Compute the length/magnitude of each vector in VectorListA and return in new ScalarList.
	 * Note that SquaredLength can be computed using VectorDot(A,A).
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	VectorLength(FGeometryScriptVectorList VectorList);

	/**
	 * Compute the dot-product between each pair of vectors in VectorListA and VectorListB and return in new ScalarList
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	VectorDot(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB);

	/**
	* Compute the cross-product between each pair of vectors in VectorListA and VectorListB and return in new VectorList
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptVectorList 
	VectorCross(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB);

	/**
	 * Normalize each vector in VectorList, and store in VectorList. 
	 * If a vector is degenerate, set the normal to the SetOnFailure vector.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	VectorNormalizeInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FVector SetOnFailure = FVector::ZeroVector);

	/**
	 * Transform each vector in VectorList, and store in VectorList.
	 * @param bAsPosition Whether to treat input as positions or vectors (if vectors, will ignore the Transform's translation part)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	VectorTransformInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FTransform Transform, bool bAsPosition = true);

	/**
	 * Inverse transform each vector in VectorList, and store in VectorList.
	 * @param bAsPosition Whether to treat input as positions or vectors (if vectors, will ignore the Transform's translation part)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	VectorInverseTransformInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FTransform Transform, bool bAsPosition = true);

	/**
	 * Project each vector in VectorList to the given Plane, and store in VectorList.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	VectorPlaneProjectInPlace(UPARAM(ref) FGeometryScriptVectorList& VectorList, FPlane Plane);

	/**
	 * Compute (ConstantA * A) + (ConstantB * B) for each pair of vectors in VectorListA and VectorListB and return in new VectorList.
	 * By default (constants = 1) this just adds the two vectors. Set ConstantB = -1 to subtract B from A. 
	 * Can also be used to Linear Interpolate, by setting ConstantB = (1-ConstantA)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptVectorList 
	VectorBlend(FGeometryScriptVectorList VectorListA, FGeometryScriptVectorList VectorListB, double ConstantA = 1.0, double ConstantB = 1.0);

	/**
	* Compute (ConstantA * A) + (ConstantB * B) for each pair of vectors in VectorListA and VectorListB, and store in VectorListB
	* By default (constants = 1) this just adds the two vectors. Set ConstantB = -1 to subtract B from A. 
	* Can also be used to Linear Interpolate, by setting ConstantB = (1-ConstantA)
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	VectorBlendInPlace(FGeometryScriptVectorList VectorListA, UPARAM(ref) FGeometryScriptVectorList& VectorListB, double ConstantA = 1.0, double ConstantB = 1.0);

	/**
	 * Compute (ScalarMultiplier * Scalar * Vector) for each scalar/vector pair in the two input lists, and return in a new VectorList.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptVectorList 
	ScalarVectorMultiply(FGeometryScriptScalarList ScalarList, FGeometryScriptVectorList VectorList, double ScalarMultiplier = 1.0);

	/**
	 * Compute (ScalarMultiplier * Scalar * Vector) for each scalar/vector pair in the two input lists, and store in the input VectorList
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	ScalarVectorMultiplyInPlace(FGeometryScriptScalarList ScalarList, UPARAM(ref) FGeometryScriptVectorList& VectorList, double ScalarMultiplier = 1.0);

	/**
	 * Compute (Constant * Vector) for each element in VectorList, and return in a new list
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath")
	static UE_API FGeometryScriptVectorList 
	ConstantVectorMultiply(double Constant, FGeometryScriptVectorList VectorList);

	/**
	 * Compute (Constant * Vector) for each element in VectorList, and store in VectorList
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath")
	static UE_API void 
	ConstantVectorMultiplyInPlace(double Constant, UPARAM(ref) FGeometryScriptVectorList& VectorList);


	/**
	 * Convert each Vector in VectorList to a Scalar by computing (ConstantX*Vector.X + ConstantY*Vector.Y + ConstantZ*Vector.Z), and return in a new ScalarList.
	 * This can be used to extract the X/Y/Z values from a Vector, or other component-wise math
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	VectorToScalar(FGeometryScriptVectorList VectorList, double ConstantX = 1.0, double ConstantY = 0.0, double ConstantZ = 0.0);



	/**
	 * Compute (Numerator / Scalar) for each element of ScalarList and return in a new ScalarList. 
	 * If Abs(Scalar) < Epsilon, set to SetOnFailure value.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	ScalarInvert(FGeometryScriptScalarList ScalarList, double Numerator = 1.0, double SetOnFailure = 0.0, double Epsilon = 0.0);

	/**
	* Compute (Numerator / Scalar) for each element of ScalarList and store in input ScalarList
	* If Abs(Scalar) < Epsilon, set to SetOnFailure value.
	*/
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	ScalarInvertInPlace(UPARAM(ref) FGeometryScriptScalarList& ScalarList, double Numerator = 1.0, double SetOnFailure = 0.0, double Epsilon = 0.0);

	/**
	 * Compute (ConstantA * A) + (ConstantB * B) for each pair of values in ScalarListA and ScalarListB and return in new ScalarList.
	 * By default (constants = 1) this just adds the two values. Set ConstantB = -1 to subtract B from A. 
	 * Can also be used to Linear Interpolate, by setting ConstantB = (1-ConstantA)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	ScalarBlend(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList ScalarListB, double ConstantA = 1.0, double ConstantB = 1.0);

	/**
	 * Compute (ConstantA * A) + (ConstantB * B) for each pair of values in ScalarListA and ScalarListB and return in ScalarListB.
	 * By default (constants = 1) this just adds the two values. Set ConstantB = -1 to subtract B from A. 
	 * Can also be used to Linear Interpolate, by setting ConstantB = (1-ConstantA)
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	ScalarBlendInPlace(FGeometryScriptScalarList ScalarListA, UPARAM(ref) FGeometryScriptScalarList& ScalarListB, double ConstantA = 1.0, double ConstantB = 1.0);

	/**
	 * Returns a Scalar List constructed with each element is the product (ConstantMultiplier * A * B) 
	 * where A and B are the corresponding elements from ScalarListA and ScalarListB accordingly. 
	 * If ScalarListA and ScalarListB have different lengths, no operation will be performed and an empty Scalar List will be returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API FGeometryScriptScalarList 
	ScalarMultiply(FGeometryScriptScalarList ScalarListA, FGeometryScriptScalarList ScalarListB, double ConstantMultiplier = 1.0);

	/**
	 * Compute (ConstantMultiplier * A * B)  where A and B are the corresponding elements from ScalarListA and ScalarListB accordingly, and store the 
	 * result in ScalarListA. 
	 * If ScalarListA and ScalarListB have different lengths, the computation will be skipped.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath", meta=(ScriptMethod))
	static UE_API void 
	ScalarMultiplyInPlace(FGeometryScriptScalarList ScalarListA, UPARAM(ref) FGeometryScriptScalarList& ScalarListB, double ConstantMultiplier = 1.0);


	/**
	 * Returns a Scalar List of the same length as the input Scalar List, with the elements computed as (Constant * A) where A is the corresponding element in the input List.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath")
	static UE_API FGeometryScriptScalarList 
	ConstantScalarMultiply(double Constant, FGeometryScriptScalarList ScalarList);

	/**
	 * Compute (Constant * A) for each element A is the Scalar List, and the result is stored in the original Scalar List. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|Math|VectorMath")
	static UE_API void 
	ConstantScalarMultiplyInPlace(double Constant, UPARAM(ref) FGeometryScriptScalarList& ScalarList);

};

#undef UE_API
