// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "VolumeTextureBakeFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGCORE_API

class UDynamicMesh;
class UVolumeTexture;

UENUM(BlueprintType)
enum class EDistanceFieldComputeMode : uint8
{
	// Compute distances in a narrow band around the input. Outside of this band, field will have large, correctly-signed values.
	NarrowBand,
	// Compute distances in the full grid
	FullGrid
};

UENUM(BlueprintType)
enum class EDistanceFieldUnits : uint8
{
	// Express distance as a number of voxels
	NumberOfVoxels,
	// Directly specify distances
	Distance
};

// Settings for computing distance fields
USTRUCT(BlueprintType)
struct FComputeDistanceFieldSettings
{
	GENERATED_BODY()
public:

	// Whether to compute distances only in a band around the surface (faster) or compute the full grid
	// Note: If full grid is computed, the distances will still be more accurately computed in the narrow band
	// In narrow band mode, values outside the band will have a large magnitude with the correct sign
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EDistanceFieldComputeMode ComputeMode = EDistanceFieldComputeMode::NarrowBand;

	// Width of the narrow band where distances are computed accurately
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float NarrowBandWidth = 2;

	// Whether Narrow Band Width is expressed as a number of voxels (rounded up to nearest int) or a distance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EDistanceFieldUnits NarrowBandUnits = EDistanceFieldUnits::NumberOfVoxels;

	// Number of voxels to use along each axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FIntVector VoxelsPerDimensions = FIntVector(32,32,32);

	// Whether to round voxel count on each dimension up to the nearest power of two
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRequirePower2 = true;
};

USTRUCT(BlueprintType)
struct FDistanceFieldToTextureSettings
{
	GENERATED_BODY()
public:

	// Scale values by this amount before writing them to the texture
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Scale = 1;

	// Offset values by this amount before writing them to the texture (after applying Scale)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float Offset = 0;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_VolumeBake"))
class UGeometryScriptLibrary_VolumeTextureBakeFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	// Write a distance field to the given existing volume texture
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|VolumeBake")
	static UE_API UPARAM(DisplayName = "Success") bool BakeSignedDistanceToVolumeTexture(
		const UDynamicMesh* TargetMesh,
		UVolumeTexture* VolumeTexture,
		FComputeDistanceFieldSettings DistanceSettings, 
		FDistanceFieldToTextureSettings TextureSettings
	);

};

#undef UE_API
