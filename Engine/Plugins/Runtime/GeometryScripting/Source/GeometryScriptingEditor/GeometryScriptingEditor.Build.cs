// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryScriptingEditor : ModuleRules
{
	public GeometryScriptingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);


		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);


		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"PhysicsCore",
				"RenderCore",
				"GeometryCore",
				"GeometryFramework",
				"DynamicMesh",
				"GeometryScriptingCore",
				"EditorSubsystem"
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"MeshDescription",
				"StaticMeshDescription",
				"MeshConversion",
				"GeometryAlgorithms",
				"ModelingOperators",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"EditorFramework",
				"UnrealEd",
				"BSPUtils",
				"ImageCore"
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

	}
}
