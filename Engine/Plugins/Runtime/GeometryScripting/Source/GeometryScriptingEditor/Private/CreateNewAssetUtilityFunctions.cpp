// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/CreateNewAssetUtilityFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/MeshBones.h"
#include "UDynamicMesh.h"

#include "Engine/Engine.h" // for GEngine
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Animation/Skeleton.h"
#include "ReferenceSkeleton.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "RenderingThread.h"

#include "Engine/BlockingVolume.h"
#include "Components/BrushComponent.h"
#include "Engine/Polys.h"
#include "Model.h"
#include "BSPOps.h"		// in UnrealEd
#include "Editor/EditorEngine.h"		// for FActorLabelUtilities
#include "Editor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/Paths.h"
#include "PackageTools.h"

#include "ConversionUtils/DynamicMeshToVolume.h"
#include "AssetUtils/CreateStaticMeshUtil.h"
#include "AssetUtils/CreateSkeletalMeshUtil.h"
#include "AssetUtils/CreateTexture2DUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "Engine/SkinnedAssetCommon.h"

#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateNewAssetUtilityFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_CreateNewAssetUtilityFunctions"

static bool CreateReferenceSkeletonFromMeshLods(const TArray<FDynamicMesh3>& MeshLods, FReferenceSkeleton& RefSkeleton, bool& bOrderChanged)
{
	TArray<FName> BoneNames;
	TArray<int32> BoneParentIdx;
	TArray<FTransform> BonePose;
		
	if (!FMeshBones::CombineLodBonesToReferenceSkeleton(MeshLods, BoneNames, BoneParentIdx, BonePose, bOrderChanged))
	{
		return false;
	}

	FReferenceSkeletonModifier Modifier(RefSkeleton, nullptr);

	for (int32 BoneIdx = 0; BoneIdx < BoneNames.Num(); ++BoneIdx)
	{
		Modifier.Add(FMeshBoneInfo(BoneNames[BoneIdx], BoneNames[BoneIdx].ToString(), BoneParentIdx[BoneIdx]), BonePose[BoneIdx]);
	}

	return true;
}

namespace UE::CreateNewAssetUtilityFunctionsLocals
{
	static bool IsAssetPathValid(const FString& Path, bool bLogFailureReason = true)
	{
		FText BadPackageNameReason;
		if (!FPackageName::IsValidTextForLongPackageName(Path, &BadPackageNameReason))
		{
			if (bLogFailureReason)
			{
				MakeScriptError(EGeometryScriptErrorType::InvalidInputs,
					FText::Format(LOCTEXT("GeometryScript_BadPackageName", "Invalid Package Name '{0}' (Reason: {1})"),
						FText::FromString(Path), BadPackageNameReason)
				);
			}
			return false;
		}
		return true;
	}
}


void UGeometryScriptLibrary_CreateNewAssetFunctions::CreateUniqueNewAssetPathName(
	FString AssetFolderPath,
	FString BaseAssetName,
	FString& UniqueAssetPathAndName,
	FString& UniqueAssetName,
	FGeometryScriptUniqueAssetNameOptions Options,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	Outcome = EGeometryScriptOutcomePins::Failure;
	int Attempts = 0;
	while (Attempts++ < 10)
	{
		FString UUID = UE::Modeling::GenerateRandomShortHexString(Options.UniqueIDDigits);
		UniqueAssetName = FString::Printf(TEXT("%s_%s"), *BaseAssetName, *UUID);
		UniqueAssetPathAndName = UPackageTools::SanitizePackageName(FPaths::Combine(AssetFolderPath, UniqueAssetName));
		if (!UE::CreateNewAssetUtilityFunctionsLocals::IsAssetPathValid(UniqueAssetPathAndName))
		{
			// path has invalid format; early out here b/c changing the random ID won't fix this
			return;
		}

		// if asset does not exist at this path, we can use it
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(UniqueAssetPathAndName));
		if (AssetData.IsValid() == false)
		{
			Outcome = EGeometryScriptOutcomePins::Success;
			return;
		}
	}

	UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("CreateUniqueNewAssetPathName_Failed", "Failed to find available unique Asset Path/Name"));
}


AVolume* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewVolumeFromMesh(
	UDynamicMesh* FromDynamicMesh,
	UWorld* CreateInWorld,
	FTransform ActorTransform,
	FString BaseActorName,
	FGeometryScriptCreateNewVolumeFromMeshOptions Options,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;

	if (FromDynamicMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewVolumeFromMesh_InvalidInput1", "CreateNewVolumeFromMesh: FromDynamicMesh is Null"));
		return nullptr;
	}
	if (CreateInWorld == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewVolumeFromMesh_InvalidInput2", "CreateNewVolumeFromMesh: CreateInWorld is Null"));
		return nullptr;
	}	
	if (FromDynamicMesh->GetTriangleCount() < 4)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewVolumeFromMesh_InvalidInput3", "CreateNewVolumeFromMesh: FromDynamicMesh does not define a valid Volume"));
		return nullptr;
	}
	// todo: other safety checks

	// spawn new actor

	UClass* VolumeClass = ABlockingVolume::StaticClass();
	if (Options.VolumeType != nullptr 
		&& Cast<AVolume>(Options.VolumeType->GetDefaultObject(false)) != nullptr )
	{
		VolumeClass = Options.VolumeType;
	}

	GEditor->BeginTransaction(LOCTEXT("CreateNewVolumeFromMesh_Transaction", "Create Volume"));

	FTransform NewActorTransform = FTransform::Identity;
	AVolume* const NewVolumeActor = [VolumeClass, CreateInWorld, &NewActorTransform]() -> AVolume*
	{
		if (UActorFactory* const VolumeFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject(VolumeClass))
		{
			AActor* const Actor = VolumeFactory->CreateActor(VolumeClass, CreateInWorld->GetCurrentLevel(), NewActorTransform);
			return Cast<AVolume>(Actor);
		}
		return nullptr;
	}();

	if (!NewVolumeActor)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewVolumeFromMesh_FailedSpawn", "CreateNewVolumeFromMesh: Failed to spawn new volume actor."));
		return nullptr;
	}

	NewVolumeActor->BrushType = EBrushType::Brush_Add;
	UModel* Model = NewObject<UModel>(NewVolumeActor);
	NewVolumeActor->Brush = Model;
	NewVolumeActor->GetBrushComponent()->Brush = NewVolumeActor->Brush;

	UE::Conversion::FMeshToVolumeOptions ConvertOptions;
	ConvertOptions.bAutoSimplify = true;
	ConvertOptions.MaxTriangles = FMath::Max(1, Options.MaxTriangles);

	FromDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		UE::Conversion::DynamicMeshToVolume(ReadMesh, NewVolumeActor, ConvertOptions);
	});

	NewVolumeActor->SetActorTransform(ActorTransform);
	FActorLabelUtilities::SetActorLabelUnique(NewVolumeActor, BaseActorName);
	NewVolumeActor->PostEditChange();

	GEditor->EndTransaction();

	Outcome = EGeometryScriptOutcomePins::Success;
	return NewVolumeActor;
}


AVolume* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewVolumeFromMesh_WorldContext(
	UObject* WorldContextObject,
	UDynamicMesh* FromDynamicMesh,
	FTransform ActorTransform,
	FString BaseActorName,
	FGeometryScriptCreateNewVolumeFromMeshOptions Options,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	if (!GEngine)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewVolumeFromMesh_NoEngine", "CreateNewVolumeFromMesh: Engine is not available. Cannot get world context object"));
		return nullptr;
	}
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return CreateNewVolumeFromMesh(FromDynamicMesh, World, ActorTransform, BaseActorName, Options, Outcome, Debug);
	}
	return nullptr;
}



UStaticMesh* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewStaticMeshAssetFromMeshLODs(
	TArray<UDynamicMesh*> FromDynamicMeshLODs, 
	FString AssetPathAndName,
	FGeometryScriptCreateNewStaticMeshAssetOptions Options,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;
	if (FromDynamicMeshLODs.IsEmpty())
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewStaticMeshAssetFromMeshLODs_EmptyLODArray", "CreateNewStaticMeshAssetFromMeshLODs: LOD array is empty"));
		return nullptr;
	}

	for (int LodIdx = 0; LodIdx < FromDynamicMeshLODs.Num(); ++LodIdx)
	{	
		const UDynamicMesh* FromDynamicMesh = FromDynamicMeshLODs[LodIdx];
		if (FromDynamicMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewStaticMeshAssetFromMeshLODs_LODIsNull", "CreateNewStaticMeshAssetFromMeshLODs: LOD {0} is Null"), FText::AsNumber(LodIdx)));
			return nullptr;
		}
		if (FromDynamicMesh->GetTriangleCount() == 0)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewStaticMeshAssetFromMeshLODs_ZeroTriangles", "CreateNewStaticMeshAssetFromMeshLODs: LOD {0} has no triangles"), FText::AsNumber(LodIdx)));
			return nullptr;
		}
	}
	AssetPathAndName = UPackageTools::SanitizePackageName(AssetPathAndName);
	if (!UE::CreateNewAssetUtilityFunctionsLocals::IsAssetPathValid(AssetPathAndName))
	{
		return nullptr;
	}
	// todo: other safety checks

	GEditor->BeginTransaction(LOCTEXT("CreateNewStaticMeshAssetFromMeshLODs_Transaction", "Create StaticMesh"));

	UE::AssetUtils::FStaticMeshAssetOptions AssetOptions;
	AssetOptions.NewAssetPath = AssetPathAndName;

	AssetOptions.NumSourceModels = FromDynamicMeshLODs.Num();

	// CreateStaticMeshAsset below will handle this, but we could allow passing in materials as an option...
	//AssetOptions.NumMaterialSlots = CreateMeshParams.Materials.Num();
	//AssetOptions.AssetMaterials = (CreateMeshParams.AssetMaterials.Num() == AssetOptions.NumMaterialSlots) ?
	//	FilterMaterials(CreateMeshParams.AssetMaterials) : FilterMaterials(CreateMeshParams.Materials);

	AssetOptions.bEnableRecomputeNormals = Options.bEnableRecomputeNormals;
	AssetOptions.bEnableRecomputeTangents = Options.bEnableRecomputeTangents;
	AssetOptions.bGenerateNaniteEnabledMesh = Options.bEnableNanite;
	AssetOptions.NaniteSettings = Options.NaniteSettings;

	AssetOptions.bCreatePhysicsBody = Options.bEnableCollision;
	AssetOptions.CollisionType = Options.CollisionMode;

	AssetOptions.bConvertBackToNonManifold = Options.bUseOriginalVertexOrder;

	/**
	 * We are making a copy of each LOD mesh since UDynamicMesh can potentially be editable asynchronously in the future, 
	 * so we should not hold onto the pointer outside the function.
	 */
	TArray<FDynamicMesh3> CopyFromDynamicMeshLODs;
	CopyFromDynamicMeshLODs.SetNum(FromDynamicMeshLODs.Num());

	for (int LodIdx = 0; LodIdx < FromDynamicMeshLODs.Num(); ++LodIdx) 
	{
		const UDynamicMesh* LODMesh = FromDynamicMeshLODs[LodIdx];
		FDynamicMesh3* CopyLODMesh = &CopyFromDynamicMeshLODs[LodIdx];

		LODMesh->ProcessMesh([CopyLODMesh](const FDynamicMesh3& ReadMesh)
		{
			*CopyLODMesh = ReadMesh;
		});

		AssetOptions.SourceMeshes.DynamicMeshes.Add(CopyLODMesh);
	}

	UE::AssetUtils::FStaticMeshResults ResultData;
	UE::AssetUtils::ECreateStaticMeshResult AssetResult = UE::AssetUtils::CreateStaticMeshAsset(AssetOptions, ResultData);

	if (AssetResult != UE::AssetUtils::ECreateStaticMeshResult::Ok)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("CreateNewStaticMeshAssetFromMeshLODs_Failed", "CreateNewStaticMeshAssetFromMeshLODs: Failed to create new Asset"));
		return nullptr;
	}

	UStaticMesh* NewStaticMesh = ResultData.StaticMesh;
	NewStaticMesh->PostEditChange();

	GEditor->EndTransaction();

	// publish new asset so that asset editor updates
	FAssetRegistryModule::AssetCreated(NewStaticMesh);

	Outcome = EGeometryScriptOutcomePins::Success;
	return NewStaticMesh;
}

UStaticMesh* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewStaticMeshAssetFromMesh(
	UDynamicMesh* FromDynamicMesh, 
	FString AssetPathAndName,
	FGeometryScriptCreateNewStaticMeshAssetOptions Options,
	EGeometryScriptOutcomePins& Outcome,
	UGeometryScriptDebug* Debug)
{
	return CreateNewStaticMeshAssetFromMeshLODs({FromDynamicMesh}, AssetPathAndName, Options, Outcome, Debug);
}


USkeletalMesh* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewSkeletalMeshAssetFromMeshLODs(
	TArray<UDynamicMesh*> FromDynamicMeshLODs,
	USkeleton* InSkeleton,
	FString AssetPathAndName, 
	FGeometryScriptCreateNewSkeletalMeshAssetOptions Options,
	EGeometryScriptOutcomePins& Outcome, 
	UGeometryScriptDebug* Debug)
{
	using namespace UE::AssetUtils;

	Outcome = EGeometryScriptOutcomePins::Failure;
	if (FromDynamicMeshLODs.IsEmpty())
	{
		AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_EmptyLODArray", "CreateNewSkeletalMeshAssetFromMeshLODs: FromDynamicMesh array is empty"));
		return nullptr;
	}

	for (int32 LodIdx = 0; LodIdx < FromDynamicMeshLODs.Num(); ++LodIdx)
	{	
		const UDynamicMesh* FromDynamicMesh = FromDynamicMeshLODs[LodIdx];
		if (FromDynamicMesh == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_LODIsNull", "CreateNewSkeletalMeshAssetFromMeshLODs: LOD {0} is Null"), FText::AsNumber(LodIdx)));
			return nullptr;
		}
		if (FromDynamicMesh->GetTriangleCount() == 0)
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_ZeroTriangles", "CreateNewSkeletalMeshAssetFromMeshLODs: LOD {0} has no triangles"), FText::AsNumber(LodIdx)));
			return nullptr;
		}
		if (FromDynamicMesh->GetMeshRef().HasAttributes() == false || FromDynamicMesh->GetMeshRef().Attributes()->GetSkinWeightsAttributes().Num() == 0)
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_NoSkinWeights", "CreateNewSkeletalMeshAssetFromMeshLODs: LOD {0} has no skin weight attributes"), FText::AsNumber(LodIdx)));
			return nullptr;
		}
		if (InSkeleton == nullptr)
		{
			AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_NullSkeleton", "CreateNewSkeletalMeshAssetFromMeshLODs: Skeleton is Null"));
			return nullptr;
		}
	}
	AssetPathAndName = UPackageTools::SanitizePackageName(AssetPathAndName);
	if (!UE::CreateNewAssetUtilityFunctionsLocals::IsAssetPathValid(AssetPathAndName))
	{
		return nullptr;
	}
	// todo: other safety checks

	GEditor->BeginTransaction(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_Transaction", "Create SkeletalMesh"));

	FSkeletalMeshAssetOptions AssetOptions;
	AssetOptions.NewAssetPath = AssetPathAndName;
	AssetOptions.Skeleton = InSkeleton;

	AssetOptions.NumSourceModels = FromDynamicMeshLODs.Num();

	if (!Options.Materials.IsEmpty())
	{
		TArray<FSkeletalMaterial> Materials;

		for (const TPair<FName, TObjectPtr<UMaterialInterface>>& Item : Options.Materials)
		{
			Materials.Add(FSkeletalMaterial{Item.Value, Item.Key});
		}
		AssetOptions.SkeletalMaterials = MoveTemp(Materials);
		AssetOptions.NumMaterialSlots = AssetOptions.SkeletalMaterials.Num(); 
	}
	else
	{
		AssetOptions.NumMaterialSlots = 1;
	}
	
	AssetOptions.bEnableRecomputeNormals = Options.bEnableRecomputeNormals;
	AssetOptions.bEnableRecomputeTangents = Options.bEnableRecomputeTangents;
	AssetOptions.bApplyNaniteSettings = Options.bApplyNaniteSettings;
	AssetOptions.NaniteSettings = Options.NaniteSettings;

	/**
	 * We are making a copy of each LOD mesh since UDynamicMesh can potentially be editable asynchronously in the future, 
	 * so we should not hold onto the pointer outside the function.
	 */
	TArray<FDynamicMesh3> CopyFromDynamicMeshLODs;
	CopyFromDynamicMeshLODs.SetNum(FromDynamicMeshLODs.Num());

	for (int32 LodIdx = 0; LodIdx < FromDynamicMeshLODs.Num(); ++LodIdx) 
	{
		const UDynamicMesh* LODMesh = FromDynamicMeshLODs[LodIdx];
		FDynamicMesh3* CopyLODMesh = &CopyFromDynamicMeshLODs[LodIdx];

		LODMesh->ProcessMesh([CopyLODMesh](const FDynamicMesh3& ReadMesh)
		{
			*CopyLODMesh = ReadMesh;
		});
	}

	// Check if all LODs have bone attributes.
	int32 LODWithoutBoneAttrib = -1;
	for (int32 Idx = 0; Idx < CopyFromDynamicMeshLODs.Num(); ++Idx)
	{
		if (!CopyFromDynamicMeshLODs[Idx].Attributes()->HasBones())
		{
			LODWithoutBoneAttrib = Idx;
			break;
		}
	}

	TUniquePtr<FReferenceSkeleton> RefSkeleton = nullptr;
	if (LODWithoutBoneAttrib >= 0) // If at least one LOD doesn't contain the bone attributes then add LOD meshes as is
	{
		for (const FDynamicMesh3& FromDynamicMesh : CopyFromDynamicMeshLODs)
		{
			AssetOptions.SourceMeshes.DynamicMeshes.Add(&FromDynamicMesh);
		}
		
		if (Options.bUseMeshBoneProportions)
		{
			AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, FText::Format(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_MissingBoneAttributes", "CreateNewSkeletalMeshAssetFromMeshLODs: Mesh bone proportions were requested, but the LOD {0} has no bone attributes. Proportions will be ignored."), FText::AsNumber(LODWithoutBoneAttrib)));
		}
	}
	else // If bone attributes are available then attempt to reindex the weights and if requested create a ReferenceSkeleton
	{	
		TArray<FName> ToSkeleton; // array of bone names in the final reference skeleton
		bool bNeedToReindex = true; // do we need to re-index the bone weights with respect to the reference skeleton
		if (Options.bUseMeshBoneProportions)
		{
			// If mesh LODs have bone attributes and the user requested to use mesh bone proportions then we create a 
			// new reference skeleton by finding the mesh with the largest number of bones and creating reference  
			// skeleton out of its bone attributes
			RefSkeleton = MakeUnique<FReferenceSkeleton>();
			if (CreateReferenceSkeletonFromMeshLods(CopyFromDynamicMeshLODs, *RefSkeleton, bNeedToReindex))
			{
				ToSkeleton = RefSkeleton->GetRawRefBoneNames();

				// Asset will now use the custom reference skeleton instead of the InSkeleton reference skeleton
				AssetOptions.RefSkeleton = RefSkeleton.Get(); 
			}
			else 
			{
				// if we failed to get reference skeleton from Lods, fall back to the skeleton asset
				ToSkeleton = InSkeleton->GetReferenceSkeleton().GetRawRefBoneNames();
				AppendWarning(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_FailedToCombineBoneAttributesForLods", "CreateNewSkeletalMeshAssetFromMeshLODs: Failed to combine the bone attributes for the Lods. Skeleton asset will be used instead."));
			}
		}
		else
		{
			ToSkeleton = InSkeleton->GetReferenceSkeleton().GetRawRefBoneNames();
		}
	
		for (int32 LodIdx = 0; LodIdx < CopyFromDynamicMeshLODs.Num(); ++LodIdx)
		{
			FDynamicMesh3& FromDynamicMesh = CopyFromDynamicMeshLODs[LodIdx];

			if (bNeedToReindex) // potentially need to re-index the weights
			{
				FDynamicMeshAttributeSet* AttribSet = FromDynamicMesh.Attributes();

				// Check if the skeleton we are trying to bind the mesh to is the same as the current mesh skeleton.
				const TArray<FName>& FromSkeleton = AttribSet->GetBoneNames()->GetAttribValues();

				if (FromSkeleton != ToSkeleton)
				{
					for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& Entry : AttribSet->GetSkinWeightsAttributes())
					{
						FDynamicMeshVertexSkinWeightsAttribute* SkinWeightAttrib = Entry.Value.Get();
						
						// Reindex the bone indices
						if (SkinWeightAttrib->ReindexBoneIndicesToSkeleton(FromSkeleton, ToSkeleton) == false)
						{
							AppendError(Debug, EGeometryScriptErrorType::OperationFailed, FText::Format(LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_FailedReindexing", "CreateNewSkeletalMeshAssetFromMeshLODs: LOD {0} has invalid skinning data or the bone data is not compatible with the specified skeleton."), FText::AsNumber(LodIdx)));
							return nullptr;
						}
					}

					// Update the bones on the mesh to match the reference skeleton being used. We try to retain any existing bone color assignments
					// as much as we can.
					TMap<FName, FVector4f> BoneColors;
					if (AttribSet->GetBoneNames() && AttribSet->GetBoneColors())
					{
						for (int32 BoneIndex = 0; BoneIndex < AttribSet->GetNumBones(); BoneIndex++)
						{
							BoneColors.Add(AttribSet->GetBoneNames()->GetValue(BoneIndex), AttribSet->GetBoneColors()->GetValue(BoneIndex));
						}
					}
					const FReferenceSkeleton* ToRefSkeleton = AssetOptions.RefSkeleton ? AssetOptions.RefSkeleton : &InSkeleton->GetReferenceSkeleton();

					AttribSet->EnableBones(ToRefSkeleton->GetRawBoneNum());
					const TArray<FMeshBoneInfo>& BoneInfos = ToRefSkeleton->GetRawRefBoneInfo();
					const TArray<FTransform>& BonePoses = ToRefSkeleton->GetRawRefBonePose();

					for (int32 BoneIndex = 0; BoneIndex < ToRefSkeleton->GetRawBoneNum(); BoneIndex++)
					{
						AttribSet->GetBoneNames()->SetValue(BoneIndex, BoneInfos[BoneIndex].Name);
						AttribSet->GetBoneParentIndices()->SetValue(BoneIndex, BoneInfos[BoneIndex].ParentIndex);
						AttribSet->GetBonePoses()->SetValue(BoneIndex, BonePoses[BoneIndex]);

						if(const FVector4f* BoneColor = BoneColors.Find(BoneInfos[BoneIndex].Name))
						{
							AttribSet->GetBoneColors()->SetValue(BoneIndex, *BoneColor);
						}
					}
				}
			}

			AssetOptions.SourceMeshes.DynamicMeshes.Add(&FromDynamicMesh);
		}
	}
	

	FSkeletalMeshResults ResultData;
	const ECreateSkeletalMeshResult AssetResult = CreateSkeletalMeshAsset(AssetOptions, ResultData);

	if (AssetResult != ECreateSkeletalMeshResult::Ok)
	{
		AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("CreateNewSkeletalMeshAssetFromMeshLODs_Failed", "CreateNewSkeletalMeshAssetFromMeshLODs: Failed to create new Asset"));
		return nullptr;
	}

	GEditor->EndTransaction();

	// publish new asset so that asset editor updates
	FAssetRegistryModule::AssetCreated(ResultData.SkeletalMesh);

	Outcome = EGeometryScriptOutcomePins::Success;
	return ResultData.SkeletalMesh;
}


USkeletalMesh* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewSkeletalMeshAssetFromMesh(
	UDynamicMesh* FromDynamicMeshLODs,
	USkeleton* InSkeleton,
	FString AssetPathAndName, 
	FGeometryScriptCreateNewSkeletalMeshAssetOptions Options,
	EGeometryScriptOutcomePins& Outcome, 
	UGeometryScriptDebug* Debug)
{
	return CreateNewSkeletalMeshAssetFromMeshLODs({FromDynamicMeshLODs}, InSkeleton, AssetPathAndName, Options, Outcome, Debug);
}

UTexture2D* UGeometryScriptLibrary_CreateNewAssetFunctions::CreateNewTexture2DAsset(
		UTexture2D* FromTexture, 
		FString AssetPathAndName,
		FGeometryScriptCreateNewTexture2DAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug)
{
	Outcome = EGeometryScriptOutcomePins::Failure;
	if (FromTexture == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("CreateNewTexture2DAsset_InvalidInput1", "CreateNewTexture2DAsset: FromTexture is Null"));
		return nullptr;
	}
	AssetPathAndName = UPackageTools::SanitizePackageName(AssetPathAndName);
	if (!UE::CreateNewAssetUtilityFunctionsLocals::IsAssetPathValid(AssetPathAndName))
	{
		return nullptr;
	}

	UE::AssetUtils::FTexture2DAssetOptions AssetOptions;
	AssetOptions.NewAssetPath = AssetPathAndName;
	AssetOptions.bOverwriteIfExists = Options.bOverwriteIfExists;

	UE::AssetUtils::FTexture2DAssetResults ResultData;
	UE::AssetUtils::ECreateTexture2DResult AssetResult = UE::AssetUtils::SaveGeneratedTexture2DAsset(
		FromTexture, AssetOptions, ResultData);

	switch (AssetResult)
	{
	case UE::AssetUtils::ECreateTexture2DResult::NameError:
	{
		const FText Error = FText::Format(LOCTEXT("CreateNewTexture2DAsset_InvalidInputName", "CreateNewTexture2DAsset: AssetPathAndName '{0}' already exists."), FText::FromString(AssetPathAndName));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		break;
	}
	case UE::AssetUtils::ECreateTexture2DResult::OverwriteTypeError:
	{
		const FText Error = FText::Format(LOCTEXT("CreateNewTexture2DAsset_InvalidOverwriteType", "CreateNewTexture2DAsset: AssetPathAndName '{0}' already exists and is not a UTexture2D."), FText::FromString(AssetPathAndName));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		break;
	}
	case UE::AssetUtils::ECreateTexture2DResult::InvalidInputTexture:
	{
		const FText Error = FText::Format(LOCTEXT("CreateNewTexture2DAsset_InvalidInputPackage", "CreateNewTexture2DAsset: Failed to read input texture '{0}'."), FText::FromString(FromTexture->GetPathName()));
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Error);
		break;
	}
	case UE::AssetUtils::ECreateTexture2DResult::InvalidPackage:
	case UE::AssetUtils::ECreateTexture2DResult::UnknownError:
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("CreateNewTexture2DAsset_Failed", "CreateNewTexture2DAsset: Failed to create new Asset."));
		break;
	}
	case UE::AssetUtils::ECreateTexture2DResult::Ok:
	{
		Outcome = EGeometryScriptOutcomePins::Success;
		break;
	}
	}
	
	return ResultData.Texture;
}




#undef LOCTEXT_NAMESPACE
