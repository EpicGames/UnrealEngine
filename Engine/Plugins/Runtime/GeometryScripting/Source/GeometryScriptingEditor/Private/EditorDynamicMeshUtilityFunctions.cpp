// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/EditorDynamicMeshUtilityFunctions.h"

#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorDynamicMeshUtilityFunctions)


namespace UE
{
namespace Geometry
{

// hidden representation of dynamic mesh change used by FDynamicMeshChangeContainer
struct FDynamicMeshChangeContainerInternals
{
	TSharedPtr<FDynamicMesh3> InitialMesh;
};

}
}



UDynamicMesh* UGeometryScriptLibrary_EditorDynamicMeshFunctions::BeginTrackedMeshChange(UDynamicMesh* TargetMesh, FDynamicMeshChangeContainer& ChangeContainer)
{
	if ( TargetMesh == nullptr )
	{
		UE_LOGF(LogGeometry, Warning, "BeginTrackedMeshChange: TargetMesh was null");
		return TargetMesh;
	}
	if (ChangeContainer.bIsActive)
	{
		UE_LOGF(LogGeometry, Warning, "BeginTrackedMeshChange: ChangeContainer was marked as active, discarding previous change");
	}

	// todo: A FDynamicMesh3ChangeTracker would often be more efficient. However currently this seems to result
	// in some problems...
	ChangeContainer.ChangeInternals = MakeShared<UE::Geometry::FDynamicMeshChangeContainerInternals>();
	ChangeContainer.ChangeInternals->InitialMesh = MakeShared<UE::Geometry::FDynamicMesh3>(TargetMesh->GetMeshRef());

	ChangeContainer.bIsActive = true;
	ChangeContainer.TargetMesh = TargetMesh;

	return TargetMesh;
}

UDynamicMesh* UGeometryScriptLibrary_EditorDynamicMeshFunctions::EmitTrackedMeshChange(UDynamicMesh* TargetMesh, FDynamicMeshChangeContainer& ChangeContainer)
{
	if ( TargetMesh == nullptr )
	{
		UE_LOGF(LogGeometry, Warning, "BeginTrackedMeshChange: TargetMesh was null");
		return TargetMesh;
	}

	if (GUndo == nullptr)
	{
		UE_LOGF(LogGeometry, Warning, "EmitTrackedMeshChange: No Transaction is actively open, ignoring.");
		return TargetMesh;
	}

	if (ChangeContainer.bIsActive == false || ChangeContainer.ChangeInternals.IsValid() == false)
	{
		UE_LOGF(LogGeometry, Warning, "EmitTrackedMeshChange: ChangeContainer was not active, ignoring");
		return TargetMesh;
	}
	if (ChangeContainer.TargetMesh != TargetMesh)
	{
		UE_LOGF(LogGeometry, Warning, "EmitTrackedMeshChange: ChangeContainer was created for a different UDynamicMesh, ignoring");
		return TargetMesh;
	}

	TUniquePtr<FMeshReplacementChange> Change = MakeUnique<FMeshReplacementChange>(
		ChangeContainer.ChangeInternals->InitialMesh, MakeShared<FDynamicMesh3>(TargetMesh->GetMeshRef()) );

	// container has been used and now needs to be reset
	ChangeContainer.bIsActive = false;
	ChangeContainer.TargetMesh = nullptr;
	ChangeContainer.ChangeInternals.Reset();

	GUndo->StoreUndo(TargetMesh, MoveTemp(Change));

	return TargetMesh;
}



UDynamicMesh* UGeometryScriptLibrary_EditorDynamicMeshFunctions::StashDebugMesh(UDynamicMesh* TargetMesh, FString DebugMeshName)
{
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		UE::Geometry::Debug::StashDebugMesh(ReadMesh, DebugMeshName);
	});
	return TargetMesh;
}


UDynamicMesh* UGeometryScriptLibrary_EditorDynamicMeshFunctions::FetchDebugMesh(FString DebugMeshName, UDynamicMesh* ToTargetMesh, bool bClearDebugMesh, bool& bDebugMeshExists)
{
	bDebugMeshExists = false;
	FDynamicMesh3 ResultMesh;
	if (UE::Geometry::Debug::FetchDebugMesh(DebugMeshName, ResultMesh, bClearDebugMesh))
	{
		ToTargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh = MoveTemp(ResultMesh);
		});
		bDebugMeshExists = true;
	}

	return ToTargetMesh;
}
