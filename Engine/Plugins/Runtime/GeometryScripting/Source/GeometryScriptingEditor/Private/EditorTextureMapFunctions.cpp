// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/EditorTextureMapFunctions.h"

#include "Engine/Texture2D.h"
#include "Image/ImageDimensions.h"
#include "TextureResource.h"
#include "TextureCompiler.h"
#include "PixelFormat.h"
#include "RenderUtils.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorTextureMapFunctions)

#if WITH_EDITOR

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_EditorTextureMapFunctions"

FGeometryScriptChannelPackResult UGeometryScriptLibrary_EditorTextureMapFunctions::ChannelPack(
	FGeometryScriptChannelPackSource RChannelSource,
	FGeometryScriptChannelPackSource GChannelSource,
	FGeometryScriptChannelPackSource BChannelSource,
	FGeometryScriptChannelPackSource AChannelSource,
	bool bOutputSRGB,
	UGeometryScriptDebug* Debug)
{
	FGeometryScriptChannelPackResult Result;

	struct FInputInfo
	{
		FImage Image;
		FGeometryScriptChannelPackSource Source;
		FText NoValidChannelError;
		FText InvalidChannelError;
		FText CouldNotReadTextureError;
	};

	FInputInfo Infos[4] =
		{
			{
				{}, // Filled in later
				RChannelSource,
				LOCTEXT("ChannelPack_NoValidChannelError_R",
					"ChannelPack: RChannelSource given a Texture which has no readable source Channel"),
				LOCTEXT("ChannelPack_InvalidChannelError_R",
					"ChannelPack: RChannelSource given a Channel incompatible with the Texture's pixel format"),
				LOCTEXT("ChannelPack_TexReadFailed_R",
					"ChannelPack: RChannelSource Texture could not be read."),
			},
			{
				{}, // Filled in later
				GChannelSource,
				LOCTEXT("ChannelPack_NoValidChannelError_G",
					"ChannelPack: GChannelSource given a Texture which has no readable source Channel"),
				LOCTEXT("ChannelPack_InvalidChannelError_G",
					"ChannelPack: GChannelSource given a Channel incompatible with the Texture's pixel format"),
				LOCTEXT("ChannelPack_TexReadFailed_G",
					"ChannelPack: GChannelSource Texture could not be read."),
			},
			{
				{}, // Filled in later
				BChannelSource,
				LOCTEXT("ChannelPack_NoValidChannelError_B",
					"ChannelPack: BChannelSource given a Texture which has no readable source Channel"),
				LOCTEXT("ChannelPack_InvalidChannelError_B",
					"ChannelPack: BChannelSource given a Channel incompatible with the Texture's pixel format"),
				LOCTEXT("ChannelPack_TexReadFailed_B",
					"ChannelPack: BChannelSource Texture could not be read."),
			},
			{
				{}, // Filled in later
				AChannelSource,
				LOCTEXT("ChannelPack_NoValidChannelError_A",
					"ChannelPack: AChannelSource given a Texture which has no readable source Channel"),
				LOCTEXT("ChannelPack_InvalidChannelError_A",
					"ChannelPack: AChannelSource given a Channel incompatible with the Texture's pixel format"),
				LOCTEXT("ChannelPack_TexReadFailed_A",
					"ChannelPack: AChannelSource Texture could not be read."),
			},
		};

	const FText AllInputTexturesNullError = LOCTEXT("ChannelPack_AllInputTexturesNull",
		"ChannelPack: All input textures were null, this node requires at least one non-null input texture");
	const FText FailedToFindValidTextureSource = LOCTEXT("ChannelPack_FailedToFindValidTextureSource",
		"ChannelPack: Failed to find a valid source format for input textures. Sources may be cooked assets / lacking source data");
	const FText MismatchingResolutionError = LOCTEXT("ChannelPack_MismatchingResolution",
		"ChannelPack: All non-null source textures must have matching sizes/resolutions");
	const FText NoValidChannelsForOutputError = LOCTEXT("ChannelPack_NoValidChannelsForOutput",
		"ChannelPack: Deduced an output texture which has no valid channels");
	const FText CouldNotCreateOutputError = LOCTEXT("ChannelPack_CouldNotCreateOutput",
		"ChannelPack: Could not create output texture");
	const FText CouldNotCopyDataToOutputError = LOCTEXT("ChannelPack_CouldNotCopyDataToOutput",
		"ChannelPack: Could not copy platform data to source data in the output texture");

	//
	// Check for input errors
	//

	ETextureSourceFormat OutputSourceFormat = TSF_Invalid;
	FImageDimensions Dimensions(0, 0);
	bool bAllInputTexturesNull = true;

	for (int InfoIndex = 0; InfoIndex < 4; ++InfoIndex)
	{
		const FInputInfo& Info = Infos[InfoIndex];
		const TObjectPtr<UTexture2D> Texture = Info.Source.Texture;
		if (!Texture)
		{
			continue;
		}

		bAllInputTexturesNull = false;

		// Make sure the texture is compiled so we query the default texture
		FTextureCompilingManager::Get().FinishCompilation({ Texture });

		// Deduce output texture dimensions

		FImageDimensions TextureDimensions(Texture->GetSizeX(), Texture->GetSizeY());
		if (Dimensions == FImageDimensions(0, 0))
		{
			Dimensions = TextureDimensions;
		}
		else if (Dimensions != TextureDimensions)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, MismatchingResolutionError);
			return Result;
		}

		// Deduce output texture source format

		if (OutputSourceFormat == TSF_Invalid)
		{
			OutputSourceFormat = Texture->Source.GetFormat();
		}
		else
		{
			OutputSourceFormat = FImageCoreUtils::GetCommonSourceFormat(OutputSourceFormat, Texture->Source.GetFormat());
		}

		// Verify input textures have the requested channel

		EPixelFormatChannelFlags ValidChannels = GetPixelFormatValidChannels(Texture->GetPixelFormat());

		if (ValidChannels == EPixelFormatChannelFlags::None)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Info.NoValidChannelError);
			return Result;
		}

		const auto InvalidChannel = [&Info, ValidChannels] (EGeometryScriptRGBAChannel Test, EPixelFormatChannelFlags Required)
		{
			if (Info.Source.Channel == Test)
			{
				return !EnumHasAnyFlags(ValidChannels, Required);
			}
			return false;
		};
		
		if (InvalidChannel(EGeometryScriptRGBAChannel::R, EPixelFormatChannelFlags::R) ||
			InvalidChannel(EGeometryScriptRGBAChannel::G, EPixelFormatChannelFlags::G) ||
			InvalidChannel(EGeometryScriptRGBAChannel::B, EPixelFormatChannelFlags::B) ||
			InvalidChannel(EGeometryScriptRGBAChannel::A, EPixelFormatChannelFlags::A))
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Info.InvalidChannelError);
			return Result;
		}
	}

	if (bAllInputTexturesNull)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, AllInputTexturesNullError);
		return Result;
	}

	if (OutputSourceFormat == TSF_Invalid)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, FailedToFindValidTextureSource);
		return Result;
	}

	//
	// Compute the output texture format
	//

	ERawImageFormat::Type OutEquivalentFormat;
	const ERawImageFormat::Type OutputRawImageFormat =
		FImageCoreUtils::ConvertToRawImageFormat(OutputSourceFormat);
	const EPixelFormat OutputPixelFormat =
		FImageCoreUtils::GetPixelFormatForRawImageFormat(OutputRawImageFormat, &OutEquivalentFormat);

	EPixelFormatChannelFlags OutputValidChannels = GetPixelFormatValidChannels(OutputPixelFormat);

	// TODO Should we also report a warning if OutputValidChannels do not refer to at least one non-null input texture?
	// Example: If only one input texture is given, with pixel format PF_R16F, then the output texture will have the
	// same pixel format so if the input isn't connected to the RChannelPackSource pin then the output texture will be
	// filled with the default value given to the RChannelPackSource pin, which is probably not what was intended
	if (OutputValidChannels == EPixelFormatChannelFlags::None)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, NoValidChannelsForOutputError);
		return Result;
	}

	//
	// Convert all given input textures to the output format
	//

	for (int InfoIndex = 0; InfoIndex < 4; InfoIndex++)
	{
		FInputInfo& Info = Infos[InfoIndex];
		const TObjectPtr<UTexture2D> Texture = Info.Source.Texture;
		if (Texture)
		{
			// Create an image with the desired output format, currently this will fail if WITH_EDITORONLY_DATA == 0
			bool bSuccess = FImageUtils::GetTexture2DSourceImage(Texture, Info.Image);
			if (!bSuccess)
			{
				UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, Info.CouldNotReadTextureError);
				return Result;
			}

			switch (Info.Source.ReadGammaSpace)
			{
			case EGeometryScriptReadGammaSpace::FromTextureSettings:
				Info.Image.ChangeFormat(OutputRawImageFormat, Texture->SRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
				break;
			case EGeometryScriptReadGammaSpace::Linear:
				Info.Image.ChangeFormat(OutputRawImageFormat, EGammaSpace::Linear);
				break;
			case EGeometryScriptReadGammaSpace::SRGB:
				Info.Image.ChangeFormat(OutputRawImageFormat, EGammaSpace::sRGB);
				break;
			default:
				ensure(false);
			}
		}
	}

	FImage OutputImage;
	OutputImage.Init(
		Dimensions.GetWidth(),
		Dimensions.GetHeight(),
		OutputRawImageFormat,
		bOutputSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);

	switch (OutputImage.Format)
	{
	case ERawImageFormat::G8:
		{
			// See :ValidChannelsForGrayPixelFormat
			ensure(EnumHasAnyFlags(OutputValidChannels, EPixelFormatChannelFlags::R));
			const FImage& RChannelImage = Infos[0].Image;

			TArrayView64<uint8> Pixels = OutputImage.AsG8();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				uint8& Pixel = Pixels[PixelIndex];
				Pixel = RChannelSource.Texture
					? RChannelImage.AsG8()[PixelIndex]
					: static_cast<uint8>(RChannelSource.DefaultValue);
			}
		} break;

	case ERawImageFormat::G16:     // note G8/G16 = gray = replicate to 3 channels, R16F = just in red channel
		{
			// This ensure does not test equality because it looks like EPixelFormatChannelFlags::RGB would also be a
			// valid choice given the comment about G8/G16 being gray. It still makes sense to just read from the red
			// channel though. :ValidChannelsForGrayPixelFormat
			ensure(EnumHasAnyFlags(OutputValidChannels, EPixelFormatChannelFlags::R));
			const FImage& RChannelImage = Infos[0].Image;

			TArrayView64<uint16> Pixels = OutputImage.AsG16();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				uint16& Pixel = Pixels[PixelIndex];
				Pixel = RChannelSource.Texture
					? RChannelImage.AsG16()[PixelIndex]
					: static_cast<uint16>(RChannelSource.DefaultValue);
			}
		} break;

	case ERawImageFormat::BGRA8: // FColor
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::RGBA);

			auto GetValue = [&Infos](EGeometryScriptRGBAChannel OutChannel, int PixelIndex) -> uint8
			{
				const FImage& Image = Infos[static_cast<int>(OutChannel)].Image;
				const FGeometryScriptChannelPackSource& Source = Infos[static_cast<int>(OutChannel)].Source;
				if (Source.Texture)
				{
					switch (Source.Channel)
					{
					case EGeometryScriptRGBAChannel::R:
						return Image.AsBGRA8()[PixelIndex].R;
					case EGeometryScriptRGBAChannel::G:
						return Image.AsBGRA8()[PixelIndex].G;
					case EGeometryScriptRGBAChannel::B:
						return Image.AsBGRA8()[PixelIndex].B;
					case EGeometryScriptRGBAChannel::A:
						return Image.AsBGRA8()[PixelIndex].A;
					}
				}
				return static_cast<uint8>(Source.DefaultValue);
			};
	
			TArrayView64<FColor> Pixels = OutputImage.AsBGRA8();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				FColor& Pixel = Pixels[PixelIndex];
				Pixel.R = GetValue(EGeometryScriptRGBAChannel::R, PixelIndex);
				Pixel.G = GetValue(EGeometryScriptRGBAChannel::G, PixelIndex);
				Pixel.B = GetValue(EGeometryScriptRGBAChannel::B, PixelIndex);
				Pixel.A = GetValue(EGeometryScriptRGBAChannel::A, PixelIndex);
			}
		} break;

	case ERawImageFormat::BGRE8:
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::RGBA);

			auto GetValue = [&Infos](EGeometryScriptRGBAChannel OutChannel, int PixelIndex) -> uint8
			{
				const FImage& Image = Infos[static_cast<int>(OutChannel)].Image;
				const FGeometryScriptChannelPackSource& Source = Infos[static_cast<int>(OutChannel)].Source;
				if (Source.Texture)
				{
					switch (Source.Channel)
					{
					case EGeometryScriptRGBAChannel::R:
						return Image.AsBGRE8()[PixelIndex].R;
					case EGeometryScriptRGBAChannel::G:
						return Image.AsBGRE8()[PixelIndex].G;
					case EGeometryScriptRGBAChannel::B:
						return Image.AsBGRE8()[PixelIndex].B;
					case EGeometryScriptRGBAChannel::A:
						return Image.AsBGRE8()[PixelIndex].A;
					}
				}
				return static_cast<uint8>(Source.DefaultValue);
			};
	
			TArrayView64<FColor> Pixels = OutputImage.AsBGRE8();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				FColor& Pixel = Pixels[PixelIndex];
				Pixel.R = GetValue(EGeometryScriptRGBAChannel::R, PixelIndex);
				Pixel.G = GetValue(EGeometryScriptRGBAChannel::G, PixelIndex);
				Pixel.B = GetValue(EGeometryScriptRGBAChannel::B, PixelIndex);
				Pixel.A = GetValue(EGeometryScriptRGBAChannel::A, PixelIndex);
			}
		} break;

	case ERawImageFormat::RGBA16:
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::RGBA);

			auto GetValue = [&Infos](EGeometryScriptRGBAChannel OutChannel, int PixelIndex) -> uint16
			{
				const FImage& Image = Infos[static_cast<int>(OutChannel)].Image;
				const FGeometryScriptChannelPackSource& Source = Infos[static_cast<int>(OutChannel)].Source;
				if (Source.Texture)
				{
					switch (Source.Channel)
					{
					case EGeometryScriptRGBAChannel::R:
						return Image.AsRGBA16()[PixelIndex * 4 + 0];
					case EGeometryScriptRGBAChannel::G:
						return Image.AsRGBA16()[PixelIndex * 4 + 1];
					case EGeometryScriptRGBAChannel::B:
						return Image.AsRGBA16()[PixelIndex * 4 + 2];
					case EGeometryScriptRGBAChannel::A:
						return Image.AsRGBA16()[PixelIndex * 4 + 3];
					}
				}
				return static_cast<uint16>(Source.DefaultValue);
			};
	
			TArrayView64<uint16> Texels = OutputImage.AsRGBA16();
			for (int PixelIndex = 0; PixelIndex < Texels.Num() / 4; ++PixelIndex)
			{
				Texels[PixelIndex * 4 + 0] = GetValue(EGeometryScriptRGBAChannel::R, PixelIndex);
				Texels[PixelIndex * 4 + 1] = GetValue(EGeometryScriptRGBAChannel::G, PixelIndex);
				Texels[PixelIndex * 4 + 2] = GetValue(EGeometryScriptRGBAChannel::B, PixelIndex);
				Texels[PixelIndex * 4 + 3] = GetValue(EGeometryScriptRGBAChannel::A, PixelIndex);
			}
		} break;

	case ERawImageFormat::RGBA16F: // FFloat16Color
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::RGBA);

			auto GetValue = [&Infos](EGeometryScriptRGBAChannel OutChannel, int PixelIndex) -> FFloat16
			{
				const FImage& Image = Infos[static_cast<int>(OutChannel)].Image;
				const FGeometryScriptChannelPackSource& Source = Infos[static_cast<int>(OutChannel)].Source;
				if (Source.Texture)
				{
					switch (Source.Channel)
					{
					case EGeometryScriptRGBAChannel::R:
						return Image.AsRGBA16F()[PixelIndex].R;
					case EGeometryScriptRGBAChannel::G:
						return Image.AsRGBA16F()[PixelIndex].G;
					case EGeometryScriptRGBAChannel::B:
						return Image.AsRGBA16F()[PixelIndex].B;
					case EGeometryScriptRGBAChannel::A:
						return Image.AsRGBA16F()[PixelIndex].A;
					}
				}
				return FFloat16(Source.DefaultValue);
			};
	
			TArrayView64<FFloat16Color> Pixels = OutputImage.AsRGBA16F();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				FFloat16Color& Pixel = Pixels[PixelIndex];
				Pixel.R = GetValue(EGeometryScriptRGBAChannel::R, PixelIndex);
				Pixel.G = GetValue(EGeometryScriptRGBAChannel::G, PixelIndex);
				Pixel.B = GetValue(EGeometryScriptRGBAChannel::B, PixelIndex);
				Pixel.A = GetValue(EGeometryScriptRGBAChannel::A, PixelIndex);
			}
		} break;

	case ERawImageFormat::RGBA32F: // FLinearColor
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::RGBA);

			auto GetValue = [&Infos](EGeometryScriptRGBAChannel OutChannel, int PixelIndex) -> float
			{
				const FImage& Image = Infos[static_cast<int>(OutChannel)].Image;
				const FGeometryScriptChannelPackSource& Source = Infos[static_cast<int>(OutChannel)].Source;
				if (Source.Texture)
				{
					switch (Source.Channel)
					{
					case EGeometryScriptRGBAChannel::R:
						return Image.AsRGBA32F()[PixelIndex].R;
					case EGeometryScriptRGBAChannel::G:
						return Image.AsRGBA32F()[PixelIndex].G;
					case EGeometryScriptRGBAChannel::B:
						return Image.AsRGBA32F()[PixelIndex].B;
					case EGeometryScriptRGBAChannel::A:
						return Image.AsRGBA32F()[PixelIndex].A;
					}
				}
				return Source.DefaultValue;
			};
	
			TArrayView64<FLinearColor> Pixels = OutputImage.AsRGBA32F();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				FLinearColor& Pixel = Pixels[PixelIndex];
				Pixel.R = GetValue(EGeometryScriptRGBAChannel::R, PixelIndex);
				Pixel.G = GetValue(EGeometryScriptRGBAChannel::G, PixelIndex);
				Pixel.B = GetValue(EGeometryScriptRGBAChannel::B, PixelIndex);
				Pixel.A = GetValue(EGeometryScriptRGBAChannel::A, PixelIndex);
			}
		} break;

	case ERawImageFormat::R16F:
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::R);
			const FImage& RChannelImage = Infos[0].Image;

			TArrayView64<FFloat16> Pixels = OutputImage.AsR16F();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				FFloat16& Pixel = Pixels[PixelIndex];
				Pixel = RChannelSource.Texture
					? RChannelImage.AsR16F()[PixelIndex]
					: FFloat16(RChannelSource.DefaultValue);
			}
		} break;

	case ERawImageFormat::R32F:
		{
			ensure(OutputValidChannels == EPixelFormatChannelFlags::R);
			const FImage& RChannelImage = Infos[0].Image;

			TArrayView64<float> Pixels = OutputImage.AsR32F();
			for (int PixelIndex = 0; PixelIndex < Pixels.Num(); ++PixelIndex)
			{
				float& Pixel = Pixels[PixelIndex];
				Pixel = RChannelSource.Texture
					? RChannelImage.AsR32F()[PixelIndex]
					: RChannelSource.DefaultValue;
			}
		} break;

	default:
		{
			ensure(false);
		} break;
	}

	if (OutEquivalentFormat != OutputRawImageFormat)
	{
		// We need to do a conversion as explained by the comment for FImageCoreUtils::GetPixelFormatForRawImageFormat,
		// which we reproduce here:
		//
		//       EPixelFormat is the graphics texture pixel format
		//       it is a much larger superset of ERawImageFormat
		//       GetPixelFormatForRawImageFormat does not map to the very closest EPixelFormat
		//      	instead map to a close one that is actually usable as Texture
		//       if *pOutEquivalentFormat != InFormat , then conversion is needed
		//        and conversion can be done using CopyImage to OutEquivalentFormat
		//
		FImage OutputImageConverted;
		OutputImage.CopyTo(OutputImageConverted, OutEquivalentFormat, bOutputSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
		Result.Output = FImageUtils::CreateTexture2DFromImage(OutputImageConverted);
	}
	else
	{
		Result.Output = FImageUtils::CreateTexture2DFromImage(OutputImage);
	}

	if (Result.Output == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, CouldNotCreateOutputError);
		return Result;
	}

	//
	// Set output texture options and copy platform data to source data
	//

	// Must wait until the texture is done with previous operations before changing settings and getting it to rebuild.
	Result.Output->WaitForPendingInitOrStreaming();
	Result.Output->PreEditChange(nullptr);

	{
		Result.Output->SRGB = bOutputSRGB;
		Result.Output->CompressionNone = true;
		Result.Output->MipGenSettings = TMGS_NoMipmaps;
		Result.Output->CompressionSettings = TC_Default;

		const int32 SizeX = Dimensions.GetWidth();
		const int32 SizeY = Dimensions.GetHeight();

		void* RawMipData = Result.Output->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_ONLY);
		if (RawMipData == nullptr)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, CouldNotCopyDataToOutputError);
			return Result;
		}

		switch (OutEquivalentFormat)
		{
		case ERawImageFormat::G8:
			{
				const uint8* SourceMipData = reinterpret_cast<const uint8*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_G8);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, SizeX * SizeY * sizeof(uint8)); // TODO If it's gray, should this be multiplied by 3?
			} break;

		case ERawImageFormat::G16:
			{
				const uint16* SourceMipData = reinterpret_cast<const uint16*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_G16);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, SizeX * SizeY * sizeof(uint16)); // TODO If it's gray, should this be multiplied by 3?
			} break;

		case ERawImageFormat::BGRA8:
			{
				const FColor* SourceMipData = reinterpret_cast<const FColor*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_BGRA8);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::BGRE8:
			{
				const FColor* SourceMipData = reinterpret_cast<const FColor*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_BGRE8);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::RGBA16:
			{
				const uint16* SourceMipData = reinterpret_cast<const uint16*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_RGBA16);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::RGBA16F:
			{
				const FFloat16* SourceMipData = reinterpret_cast<const FFloat16*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_RGBA16F);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::RGBA32F:
			{
				const FLinearColor* SourceMipData = reinterpret_cast<const FLinearColor*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_RGBA32F);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::R16F:
			{
				const FFloat16* SourceMipData = reinterpret_cast<const FFloat16*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_R16F);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		case ERawImageFormat::R32F:
			{
				const float* SourceMipData = reinterpret_cast<const float*>(RawMipData);
				Result.Output->Source.Init2DWithMipChain(SizeX, SizeY, TSF_R32F);
				uint8* DestData = Result.Output->Source.LockMip(0);
				FMemory::Memcpy(DestData, SourceMipData, Result.Output->Source.CalcMipSize(0));
			} break;

		default:
			{
				ensure(false);
			} break;
		}

		Result.Output->Source.UnlockMip(0);
		Result.Output->GetPlatformData()->Mips[0].BulkData.Unlock();
		Result.Output->UpdateResource();
	}

	Result.Output->PostEditChange();

	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
