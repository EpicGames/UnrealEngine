// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryActors/EditorGeometryGenerationSubsystem.h"
#include "GeometryActors/GeneratedDynamicMeshActor.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorGeometryGenerationSubsystem)


void UEditorGeometryGenerationManager::Tick(float DeltaTime)
{
	// rebuild all valid generated actors, if necessary
	// note: we have a separate to-rebuild array in case the process of the rebuild modifies the ActiveGeneratedActors array
	TArray<AGeneratedDynamicMeshActor*> ToRebuild;
	for (AGeneratedDynamicMeshActor* Actor : ActiveGeneratedActors)
	{
		if (Actor->IsValidLowLevel() && IsValid(Actor) && !Actor->IsUnreachable() && Actor->GetLevel() != nullptr && Actor->WantsRebuild())
		{
			ToRebuild.Add(Actor);
		}
	}
	for (AGeneratedDynamicMeshActor* Actor : ToRebuild)
	{
		Actor->ExecuteRebuildGeneratedMeshIfPending();
	}
}

bool UEditorGeometryGenerationManager::IsTickable() const
{
	return (ActiveGeneratedActors.Num() > 0);
}

TStatId UEditorGeometryGenerationManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UEditorGeometryGenerationManager, STATGROUP_Tickables);
}


void UEditorGeometryGenerationManager::Shutdown()
{
	ActiveGeneratedActors.Reset();
}

void UEditorGeometryGenerationManager::RegisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor)
{
	ActiveGeneratedActors.Add(Actor);
}

void UEditorGeometryGenerationManager::UnregisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor)
{
	ActiveGeneratedActors.Remove(Actor);
}




bool UEditorGeometryGenerationSubsystem::bIsShuttingDown = false;


void UEditorGeometryGenerationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	GenerationManager = NewObject<UEditorGeometryGenerationManager>(this);

	GEditor->OnEditorClose().AddUObject(this, &UEditorGeometryGenerationSubsystem::OnShutdown);
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UEditorGeometryGenerationSubsystem::OnShutdown);
}

void UEditorGeometryGenerationSubsystem::Deinitialize()
{
	GEditor->OnEditorClose().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	bIsShuttingDown = true;

	if (GenerationManager)
	{
		GenerationManager->Shutdown();
		GenerationManager = nullptr;
	}
}


void UEditorGeometryGenerationSubsystem::OnShutdown()
{
	bIsShuttingDown = true;
}



bool UEditorGeometryGenerationSubsystem::RegisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor)
{
	if (bIsShuttingDown || GEditor == nullptr)		// subsystem no longer exists
	{
		return false;
	}

	UEditorGeometryGenerationSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorGeometryGenerationSubsystem>();
	if (ensure(Subsystem))
	{
		Subsystem->GenerationManager->RegisterGeneratedMeshActor(Actor);
		return true;
	}
	return false;
}

void UEditorGeometryGenerationSubsystem::UnregisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor)
{
	if (bIsShuttingDown || GEditor == nullptr)		// subsystem no longer exists
	{
		return;
	}

	UEditorGeometryGenerationSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorGeometryGenerationSubsystem>();
	if (ensure(Subsystem))
	{
		Subsystem->GenerationManager->UnregisterGeneratedMeshActor(Actor);
	}
}



