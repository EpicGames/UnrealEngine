// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryActors/GeneratedDynamicMeshActor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Level.h"
#include "GeometryActors/EditorGeometryGenerationSubsystem.h"

#include "Editor/EditorEngine.h" // for CopyPropertiesForUnrelatedObjects
#include "Engine/StaticMeshActor.h"

#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeneratedDynamicMeshActor)

#define LOCTEXT_NAMESPACE "AGeneratedDynamicMeshActor"

AGeneratedDynamicMeshActor::AGeneratedDynamicMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

AGeneratedDynamicMeshActor::~AGeneratedDynamicMeshActor()
{
	// make sure we are unregistered on destruction
	UnregisterWithGenerationManager();
}


void AGeneratedDynamicMeshActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	// Currently we rely on a mix of various Actor functions to tell us when to register w/ generation manager.
	// If that turns out to not work reliably, we can do it here at every Construction script invocation
	//RegisterWithGenerationManager();

	bGeneratedMeshRebuildPending = true;
}

void AGeneratedDynamicMeshActor::PostLoad()
{
	Super::PostLoad();
	RegisterWithGenerationManager();
}

void AGeneratedDynamicMeshActor::PostActorCreated()
{
	Super::PostActorCreated();
	RegisterWithGenerationManager();
	
	// Default newly-created generated dynamic meshes to not be element-selectable
	if (UDynamicMeshComponent* Component = GetDynamicMeshComponent())
	{
		Component->SetAllowsGeometrySelection(false);
	}
}

void AGeneratedDynamicMeshActor::Destroyed()
{
	UnregisterWithGenerationManager();
	Super::Destroyed();
}



void AGeneratedDynamicMeshActor::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	// Handle UWorld::AddToWorld(), i.e. turning on level visibility
	if (const ULevel* Level = GetLevel())
	{
		// This function gets called in editor all the time, we're only interested the case where our level is being added to world.
		if (Level->bIsAssociatingLevel)
		{
			RegisterWithGenerationManager();
		}
	}
}

void AGeneratedDynamicMeshActor::PostUnregisterAllComponents()
{
	// Handle UWorld::RemoveFromWorld(), i.e. turning off level visibility
	if (const ULevel* Level = GetLevel())
	{
		// This function gets called in editor all the time, we're only interested the case where our level is being removed from world.
		if (Level->bIsDisassociatingLevel)
		{
			UnregisterWithGenerationManager();
		}
	}

	Super::PostUnregisterAllComponents();
}


#if WITH_EDITOR

void AGeneratedDynamicMeshActor::PostEditUndo()
{
	Super::PostEditUndo();

	// There is no direct signal that an Actor is being created or destroyed due to undo/redo.
	// Currently (5.1) the checks below will tell us if the undo/redo has destroyed the
	// Actor, and we assume otherwise it was created

	if (IsActorBeingDestroyed() || !IsValidChecked(this))	// equivalent to AActor::IsPendingKillPending()
	{
		UnregisterWithGenerationManager();
	}
	else
	{
		RegisterWithGenerationManager();
	}
}

void AGeneratedDynamicMeshActor::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	Super::PostEditUndo(TransactionAnnotation);

	// There is no direct signal that an Actor is being created or destroyed due to undo/redo.
	// Currently (5.1) the checks below will tell us if the undo/redo has destroyed the
	// Actor, and we assume otherwise it was created

	if (IsActorBeingDestroyed() || !IsValidChecked(this))	// equivalent to AActor::IsPendingKillPending()
	{
		UnregisterWithGenerationManager();
	}
	else
	{
		RegisterWithGenerationManager();
	}
}
#endif



void AGeneratedDynamicMeshActor::RegisterWithGenerationManager()
{
	// do not run mesh generation for CDO
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	// Do not run mesh generation for actors spawned for PIE.
	// (If the Actor existed in Editor the existing UDynamicMesh will still be duplicated to PIE)
	if (GetPackage()->GetPIEInstanceID() != INDEX_NONE)
	{
		return;
	}

	if (bIsRegisteredWithGenerationManager == false)
	{
		// this could fail if the subsystem is not initialized yet, or if it is shutting down
		bIsRegisteredWithGenerationManager = UEditorGeometryGenerationSubsystem::RegisterGeneratedMeshActor(this);
	}
}


void AGeneratedDynamicMeshActor::UnregisterWithGenerationManager()
{
	if (bIsRegisteredWithGenerationManager)
	{
		UEditorGeometryGenerationSubsystem::UnregisterGeneratedMeshActor(this);
		bIsRegisteredWithGenerationManager = false;
		bGeneratedMeshRebuildPending = false;
	}
}

bool AGeneratedDynamicMeshActor::WantsRebuild()
{
	if (bFrozen)
	{
		return false;
	}
	if (bGeneratedMeshRebuildPending == false)
	{
		return false;
	}

	return true;
}


void AGeneratedDynamicMeshActor::ExecuteRebuildGeneratedMeshIfPending()
{
	if (!WantsRebuild())
	{
		return;
	}

	UDynamicMeshComponent* Component = GetDynamicMeshComponent();
	if (Component == nullptr)
	{
		return;
	}

	if (!ensure(!bIsCurrentlyRebuilding))
	{
		return;
	}
	bIsCurrentlyRebuilding = true;
	ON_SCOPE_EXIT
	{
		bIsCurrentlyRebuilding = false;
	};

	// Automatically defer collision updates during generated mesh rebuild. If we do not do this, then
	// each mesh change will result in collision being rebuilt, which is very expensive !!
	bool bEnabledDeferredCollision = false;
	if (Component->bDeferCollisionUpdates == false)
	{
		Component->SetDeferredCollisionUpdatesEnabled(true, false);
		bEnabledDeferredCollision = true;
	}

	if (bResetOnRebuild && Component->GetDynamicMesh())
	{
		Component->GetDynamicMesh()->Reset();
	}

	FEditorScriptExecutionGuard Guard;

	if (bEnableRebuildProgress)
	{
		FScopedSlowTask Progress(this->NumProgressSteps, FText::FromString(this->ProgressMessage));
		Progress.MakeDialogDelayed(this->DialogDelay, true);
		ActiveSlowTask = &Progress;
		CurProgressAccumSteps = 0;
		RebuildGeneratedMesh(Component->GetDynamicMesh());
		ActiveSlowTask = nullptr;
	}
	else
	{
		RebuildGeneratedMesh(Component->GetDynamicMesh());
	}

	bGeneratedMeshRebuildPending = false;

	if (bEnabledDeferredCollision)
	{
		Component->SetDeferredCollisionUpdatesEnabled(false, true);
	}
}

void AGeneratedDynamicMeshActor::RebuildGeneratedMesh(UDynamicMesh* TargetMesh)
{
	OnRebuildGeneratedMesh(TargetMesh);
}

void AGeneratedDynamicMeshActor::MarkForMeshRebuild(bool bImmediate, bool bImmediateEvenIfFrozen)
{
	// note we ignore the immediate rebuild flag if we're already rebuilding
	if (bImmediate && !bIsCurrentlyRebuilding)
	{
		bool bRestoreFrozenState = bFrozen;
		if (bImmediateEvenIfFrozen && bFrozen)
		{
			bFrozen = false;
		}

		bGeneratedMeshRebuildPending = true;
		ExecuteRebuildGeneratedMeshIfPending();

		bFrozen = bRestoreFrozenState;
	}
	else
	{
		bGeneratedMeshRebuildPending = true;
	}
}

void AGeneratedDynamicMeshActor::IncrementProgress(int NumSteps, FString Message)
{
	if (ActiveSlowTask)
	{
		int NextProgressAccumSteps = FMath::Min(CurProgressAccumSteps + NumSteps, NumProgressSteps);
		ActiveSlowTask->EnterProgressFrame((float)(NextProgressAccumSteps - CurProgressAccumSteps), FText::FromString(Message));
		CurProgressAccumSteps = NextProgressAccumSteps;
	}
}


void AGeneratedDynamicMeshActor::CopyPropertiesToStaticMesh(AStaticMeshActor* StaticMeshActor, bool bCopyComponentMaterials)
{
	StaticMeshActor->Modify();
	StaticMeshActor->UnregisterAllComponents();
	UEditorEngine::CopyPropertiesForUnrelatedObjects(this, StaticMeshActor);

	if (bCopyComponentMaterials)
	{
		if (UStaticMeshComponent* SMComponent = StaticMeshActor->GetStaticMeshComponent())
		{
			if (UDynamicMeshComponent* DMComponent = this->GetDynamicMeshComponent())
			{
				TArray<UMaterialInterface*> Materials = DMComponent->GetMaterials();
				for (int32 k = 0; k < Materials.Num(); ++k)
				{
					SMComponent->SetMaterial(k, Materials[k]);
				}
			}
		}
	}

	StaticMeshActor->ReregisterAllComponents();
}


void AGeneratedDynamicMeshActor::CopyPropertiesFromStaticMesh(AStaticMeshActor* StaticMeshActor, bool bCopyComponentMaterials)
{
	this->Modify();
	this->UnregisterAllComponents();
	UEditorEngine::CopyPropertiesForUnrelatedObjects(StaticMeshActor, this);

	if (bCopyComponentMaterials)
	{
		if (UStaticMeshComponent* SMComponent = StaticMeshActor->GetStaticMeshComponent())
		{
			if (UDynamicMeshComponent* DMComponent = this->GetDynamicMeshComponent())
			{
				TArray<UMaterialInterface*> Materials = SMComponent->GetMaterials();
				DMComponent->ConfigureMaterialSet(Materials);
			}
		}
	}

	this->ReregisterAllComponents();
}


#undef LOCTEXT_NAMESPACE
