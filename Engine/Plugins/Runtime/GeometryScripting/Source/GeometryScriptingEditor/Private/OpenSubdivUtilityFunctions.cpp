// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryScript/OpenSubdivUtilityFunctions.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Operations/SubdividePoly.h"
#include "Polygroups/PolygroupSet.h"
#include "GroupTopology.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenSubdivUtilityFunctions)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGeometryScriptLibrary_OpenSubdivUtilityFunctions"


UDynamicMesh* UGeometryScriptLibrary_OpenSubdivFunctions::ApplyPolygroupCatmullClarkSubD(
	UDynamicMesh* TargetMesh,
	int32 Subdivisions,
	FGeometryScriptGroupLayer GroupLayer,
	UGeometryScriptDebug* Debug) 
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPolygroupCatmullClarkSubD_InvalidInput", "ApplyPolygroupCatmullClarkSubD: TargetMesh is Null"));
		return TargetMesh;
	}
	if (Subdivisions <= 0)
	{
		return TargetMesh;
	}
	if (Subdivisions > 6)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPolygroupCatmullClarkSubD_MaxSubdivisions", "ApplyPolygroupCatmullClarkSubD: Clamping Subdivision to 6 levels to avoid catastrophic memory usage. Use repeated Subdivisions if higher levels are necessary."));
		Subdivisions = 6;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FPolygroupLayer InputGroupLayer{ GroupLayer.bDefaultLayer, GroupLayer.ExtendedLayerIndex };
		if (InputGroupLayer.CheckExists(&EditMesh) == false)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPolygroupCatmullClarkSubD_MissingGroups", "ApplyPolygroupCatmullClarkSubD: Target Polygroup Layer does not exist"));
			return;
		}

		constexpr bool bAutoBuild = false;
		TUniquePtr<FGroupTopology> Topo = GroupLayer.bDefaultLayer
			? MakeUnique<FGroupTopology>(&EditMesh, bAutoBuild)
			: MakeUnique<FGroupTopology>(&EditMesh, EditMesh.Attributes()->GetPolygroupLayer(GroupLayer.ExtendedLayerIndex), bAutoBuild);

		Topo->ShouldAddExtraCornerAtVert =
			[&EditMesh](const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)
				{
					// Help avoid creating group boundary with only 2 group edges
					return EditMesh.IsBoundaryVertex(Vid);
				};
			
		Topo->RebuildTopology();

		FSubdividePoly SubD(*Topo, EditMesh, Subdivisions);

		FSubdividePoly::ETopologyCheckResult CheckResult = SubD.ValidateTopology();
		if (CheckResult != FSubdividePoly::ETopologyCheckResult::Ok)
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyPolygroupCatmullClarkSubD_NoPolygons", "ApplyPolygroupCatmullClarkSubD: Target Polygroup Layer does not define valid Polygons for Subdivision"));
			return;
		}

		SubD.SubdivisionScheme = ESubdivisionScheme::CatmullClark;
		SubD.NormalComputationMethod = ESubdivisionOutputNormals::Interpolated;
		SubD.UVComputationMethod = ESubdivisionOutputUVs::Interpolated;

		FDynamicMesh3 SubDMesh;
		bool bOK = SubD.ComputeTopologySubdivision();
		bOK = bOK && SubD.ComputeSubdividedMesh(SubDMesh);
		if (bOK)
		{
			EditMesh = MoveTemp(SubDMesh);
		}
		else
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyPolygroupCatmullClarkSubD_Failed", "ApplyPolygroupCatmullClarkSubD: Subdivision Failed"));
		}
	});

	return TargetMesh;
}




UDynamicMesh* UGeometryScriptLibrary_OpenSubdivFunctions::ApplyTriangleLoopSubD(
	UDynamicMesh* TargetMesh,
	int32 Subdivisions,
	UGeometryScriptDebug* Debug) 
{
	if (TargetMesh == nullptr)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyTriangleLoopSubD_InvalidInput", "ApplyTriangleLoopSubD: TargetMesh is Null"));
		return TargetMesh;
	}
	if (Subdivisions <= 0)
	{
		return TargetMesh;
	}
	if (Subdivisions > 6)
	{
		UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::InvalidInputs, LOCTEXT("ApplyTriangleLoopSubD_MaxSubdivisions", "ApplyTriangleLoopSubD: Clamping Subdivision to 6 levels to avoid catastrophic memory usage. Use repeated Subdivisions if higher levels are necessary."));
		Subdivisions = 6;
	}

	TargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FGroupTopology Topo;
		FSubdividePoly SubD(Topo, EditMesh, Subdivisions);
		SubD.SubdivisionScheme = ESubdivisionScheme::Loop;
		SubD.NormalComputationMethod = ESubdivisionOutputNormals::Interpolated;
		SubD.UVComputationMethod = ESubdivisionOutputUVs::Interpolated;

		FDynamicMesh3 SubDMesh;
		bool bOK = SubD.ComputeTopologySubdivision();
		bOK = bOK && SubD.ComputeSubdividedMesh(SubDMesh);
		if (bOK)
		{
			EditMesh = MoveTemp(SubDMesh);
		}
		else
		{
			UE::Geometry::AppendError(Debug, EGeometryScriptErrorType::OperationFailed, LOCTEXT("ApplyTriangleLoopSubD_Failed", "ApplyTriangleLoopSubD: Subdivision Failed"));
		}
	});

	return TargetMesh;
}






#undef LOCTEXT_NAMESPACE
