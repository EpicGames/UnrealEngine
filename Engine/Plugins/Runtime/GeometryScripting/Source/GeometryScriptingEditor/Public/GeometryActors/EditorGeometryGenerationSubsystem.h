// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "TickableEditorObject.h"

#include "EditorGeometryGenerationSubsystem.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API

class AGeneratedDynamicMeshActor;


/**
 * UEditorGeometryGenerationSubsystem manages recomputation of "generated" mesh actors, eg
 * to provide procedural mesh generation in-Editor. Generally such procedural mesh generation
 * is expensive, and if many objects need to be generated, the regeneration needs to be 
 * managed at a higher level to ensure that the Editor remains responsive/interactive.
 * 
 * AGeneratedDynamicMeshActors register themselves with this Subsystem, and
 * allow the Subsystem to tell them when they should regenerate themselves (if necessary).
 * The current behavior is to run all pending generations on a Tick, however in future
 * this regeneration will be more carefully managed via throttling / timeslicing / etc.
 * 
 */
UCLASS(MinimalAPI)
class UEditorGeometryGenerationSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

public:
	UPROPERTY()
	TObjectPtr<UEditorGeometryGenerationManager> GenerationManager;


	//
	// Static functions that simplify registration of an Actor w/ the Subsystem
	//
public:
	static UE_API bool RegisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor);
	static UE_API void UnregisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor);


protected:
	// callback connected to engine/editor shutdown events to set bIsShuttingDown, which disables the subsystem static functions above
	UE_API virtual void OnShutdown();

private:
	static UE_API bool bIsShuttingDown;
};




/**
 * UEditorGeometryGenerationManager is a class used by UEditorGeometryGenerationSubsystem to
 * store registrations and provide a Tick()
 */
UCLASS(MinimalAPI)
class UEditorGeometryGenerationManager : public UObject, public FTickableEditorObject
{
	GENERATED_BODY()

public:
	UE_API virtual void Shutdown();

	UE_API virtual void RegisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor);
	UE_API virtual void UnregisterGeneratedMeshActor(AGeneratedDynamicMeshActor* Actor);

public:

	//~ Begin FTickableEditorObject interface
	UE_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	UE_API virtual bool IsTickable() const override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

protected:
	TSet<AGeneratedDynamicMeshActor*> ActiveGeneratedActors;
};

#undef UE_API
