// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshActor.h"

#include "GeneratedDynamicMeshActor.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API

class AStaticMeshActor;
struct FScopedSlowTask;


/**
 * AGeneratedDynamicMeshActor is an Editor-only subclass of ADynamicMeshActor that provides 
 * special support for dynamic procedural generation of meshes in the Editor, eg via Blueprints. 
 * Expensive procedural generation implemented via BP can potentially cause major problems in 
 * the Editor, in particular with interactive performance. AGeneratedDynamicMeshActor provides
 * special infrastructure for this use case. Essentially, instead of doing procedural generation
 * in the Construction Script, a BP-implementable event OnRebuildGeneratedMesh is available,
 * and doing the procedural mesh regeneration when that function fires will generally provide
 * better in-Editor interactive performance.
 */
UCLASS(MinimalAPI)
class AGeneratedDynamicMeshActor : public ADynamicMeshActor
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual ~AGeneratedDynamicMeshActor();


public:
	/** If true, the DynamicMeshComponent will be "Frozen" in its current state, and automatic rebuilding will be disabled. However the DynamicMesh can still be modified by explicitly-called functions/etc. */
	UPROPERTY(Category = "DynamicMeshActor", EditAnywhere, BlueprintReadWrite)
	bool bFrozen = false;

	/** If true, the DynamicMeshComponent will be cleared before the RebuildGeneratedMesh event is executed. */
	UPROPERTY(Category = "DynamicMeshActor|Advanced", EditAnywhere, BlueprintReadWrite)
	bool bResetOnRebuild = true;

	/**
	 * This event will be fired to notify the BP that the generated Mesh should
	 * be rebuilt. GeneratedDynamicMeshActor BP subclasses should rebuild their 
	 * meshes on this event, instead of doing so directly from the Construction Script.
	 */
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, Category = "Events")
	UE_API void OnRebuildGeneratedMesh(UDynamicMesh* TargetMesh);

	/**
	 * Mark this Actor as modified so that OnRebuildGeneratedMesh runs 
	 */
	UFUNCTION(BlueprintCallable, Category = "DynamicMeshActor")
	UE_API void MarkForMeshRebuild(bool bImmediate = false, bool bImmediateEvenIfFrozen = false);


	/**
	 * This function will fire the RebuildGeneratedMesh function if the actor has been
	 * marked for a pending rebuild (eg via OnConstruction)
	 */
	UE_API virtual void ExecuteRebuildGeneratedMeshIfPending();

	UE_API virtual bool WantsRebuild();


protected:
	/**
	 * Overridable native event for when the generated Mesh should be rebuilt.
	 * This function will fire the OnRebuildGeneratedMesh event to the BP.
	 */
	UE_API virtual void RebuildGeneratedMesh(UDynamicMesh* TargetMesh);


public:

	/** 
	 * Attempt to copy Actor Properties to a StaticMeshActor. Optionally copy DynamicMeshComponent material list to the StaticMeshComponent.
	 * This function is useful when (eg) swapping from a DynamicMeshActor to a StaticMeshActor as it will allow
	 * many configured Actor settings to be preserved (like assigned DataLayers, etc)
	 */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	UE_API void CopyPropertiesToStaticMesh(AStaticMeshActor* StaticMeshActor, bool bCopyComponentMaterials = false);

	/**
	 * Attempt to copy Actor Properties from a StaticMeshActor. Optionally copy DynamicMeshComponent material list to the StaticMeshComponent.
	 * This function is useful when (eg) swapping from a StaticMeshActor to a DynamicMeshActor as it will allow
	 * many configured Actor settings to be preserved (like assigned DataLayers, etc) 
	 */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	UE_API void CopyPropertiesFromStaticMesh(AStaticMeshActor* StaticMeshActor, bool bCopyComponentMaterials = false);



public:

	//~ Begin UObject/AActor Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostActorCreated() override;
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
	UE_API virtual void Destroyed() override;

	// these are called when Actor exists in a sublevel that is hidden/shown
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;

#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;
#endif


protected:
	// this internal flag is set in OnConstruction, and will cause ExecuteRebuildGeneratedMesh to
	// fire the RebuildGeneratedMesh event, after which the flag will be cleared
	bool bGeneratedMeshRebuildPending = false;

	// indicates that this Actor is registered with the UEditorGeometryGenerationSubsystem, which 
	// is where the mesh rebuilds are executed
	bool bIsRegisteredWithGenerationManager = false;

private:
	// flag set during a rebuild -- used to help detect/avoid recursive rebuilds
	bool bIsCurrentlyRebuilding = false;
protected:

	UE_API virtual void RegisterWithGenerationManager();
	UE_API virtual void UnregisterWithGenerationManager();



	//
	// Support for progress dialog display during OnRebuildGeneratedMesh. This is intended to be
	// used for long-running procedural generators that otherwise block the Editor. 
	// 
	// This functionality may be deprecated/removed in the future.
	//

public:
	/** If enabled, a long-running OnRebuildGeneratedMesh event will show a progress dialog (The Script being executed must call IncrementProgress regularly) */
	UPROPERTY(Category = "DynamicMeshActor|Progress", EditAnywhere, BlueprintReadWrite)
	bool bEnableRebuildProgress = false;

	/** Delay in seconds before the progress dialog is shown, if enabled */
	UPROPERTY(Category = "DynamicMeshActor|Progress", EditAnywhere, BlueprintReadWrite)
	float DialogDelay = 1;

	/** Number of progress steps/ticks that the progress bar will be subdivided into */
	UPROPERTY(Category = "DynamicMeshActor|Progress", EditAnywhere, BlueprintReadWrite)
	int NumProgressSteps = 4;

	/** The default progress message */
	UPROPERTY(Category = "DynamicMeshActor|Progress", EditAnywhere, BlueprintReadWrite)
	FString ProgressMessage = TEXT("Rebuilding Mesh...");

	/** Call this function from within OnRebuildGeneratedMesh to update progress tracking. */
	UFUNCTION(BlueprintCallable, Category = "DynamicMeshActor|Progress")
	UE_API void IncrementProgress(int NumSteps, FString Message);

protected:
	FScopedSlowTask* ActiveSlowTask = nullptr;
	int32 CurProgressAccumSteps = 0;

};

#undef UE_API
