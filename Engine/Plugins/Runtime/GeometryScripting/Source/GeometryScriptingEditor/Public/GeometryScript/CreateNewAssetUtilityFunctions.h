// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "Engine/BlockingVolume.h"
#include "BodySetupEnums.h"
#include "CreateNewAssetUtilityFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API


class UMaterialInterface;
class UStaticMesh;
class UDynamicMesh;
class AVolume;


USTRUCT(BlueprintType)
struct FGeometryScriptUniqueAssetNameOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = Options)
	int32 UniqueIDDigits = 6;
};


USTRUCT(BlueprintType)
struct FGeometryScriptCreateNewVolumeFromMeshOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = Options)
	TSubclassOf<class AVolume> VolumeType = ABlockingVolume::StaticClass();

	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bAutoSimplify = true;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	int32 MaxTriangles = 250;
};


USTRUCT(BlueprintType)
struct FGeometryScriptCreateNewStaticMeshAssetOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeNormals = false;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeTangents = false;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableNanite = false;

	/** Nanite settings to set on new StaticMesh Asset */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	FMeshNaniteSettings NaniteSettings;

	/** Replaced NaniteProxyTrianglePercent with usage of Engine FMeshNaniteSettings, use NaniteSettings property instead */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Deprecated, AdvancedDisplay, meta = (DisplayName = "DEPRECATED NANITE SETTING"))
	float NaniteProxyTrianglePercent = 0.0f;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableCollision = true;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	TEnumAsByte<ECollisionTraceFlag> CollisionMode = ECollisionTraceFlag::CTF_UseDefault;
	
	/** Use the original vertex order found in the source data. This is useful if the inbound mesh was originally non-manifold, and needs to keep
	 *  the non-manifold structure when re-created. */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bUseOriginalVertexOrder = false;
};


USTRUCT(BlueprintType)
struct FGeometryScriptCreateNewSkeletalMeshAssetOptions
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeNormals = false;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bEnableRecomputeTangents = false;

	UPROPERTY(BlueprintReadWrite, Category = Options)
	TMap<FName, TObjectPtr<UMaterialInterface>> Materials;

	/** If true, will use the skeleton proportions (if available) stored in the dynamic mesh. */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bUseMeshBoneProportions = false;

	/** Whether to apply the provided Nanite Settings to the new Skeletal Mesh asset. If false, the default settings (nanite disabled) will be used. */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bApplyNaniteSettings = false;

	/** Nanite settings to apply to the new Skeletal Mesh Asset, if bApplyNaniteSettings is true */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	FMeshNaniteSettings NaniteSettings;
	
	/** Use the original vertex order found in the source data. This is useful if the inbound mesh was originally non-manifold, and needs to keep
	 *  the non-manifold structure when re-created. */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bUseOriginalVertexOrder = false;
};


USTRUCT(BlueprintType)
struct FGeometryScriptCreateNewTexture2DAssetOptions
{
	GENERATED_BODY()
public:
	/** If true, overwrite any existing texture assets using the same AssetPathAndName */
	UPROPERTY(BlueprintReadWrite, Category = Options)
	bool bOverwriteIfExists = false;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_NewAssetUtils"))
class UGeometryScriptLibrary_CreateNewAssetFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API void
	CreateUniqueNewAssetPathName(
		FString AssetFolderPath,
		FString BaseAssetName,
		FString& UniqueAssetPathAndName,
		FString& UniqueAssetName,
		FGeometryScriptUniqueAssetNameOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);


	/**
	 * Create a new Volume from a Dynamic Mesh, in the same world as the calling blueprint.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome", DisplayName = "Create New Volume From Mesh", WorldContext = "WorldContextObject"))
	static UE_API UPARAM(DisplayName = "Volume Actor") AVolume*
	CreateNewVolumeFromMesh_WorldContext(
		UObject* WorldContextObject,
		UDynamicMesh* FromDynamicMesh,
		FTransform ActorTransform,
		FString BaseActorName,
		FGeometryScriptCreateNewVolumeFromMeshOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create a new Volume from a Dynamic Mesh, in the specified world.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome", DisplayName = "Create New Volume From Mesh In Target World"))
	static UE_API UPARAM(DisplayName = "Volume Actor") AVolume* 
	CreateNewVolumeFromMesh(
		UDynamicMesh* FromDynamicMesh, 
		UPARAM(ref) UWorld* CreateInWorld,
		FTransform ActorTransform,
		FString BaseActorName,
		FGeometryScriptCreateNewVolumeFromMeshOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	/**
	 * Create a new StaticMesh asset from a DynamicMesh. Save the asset at the AssetPathAndName location.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Static Mesh Asset") UStaticMesh* 
	CreateNewStaticMeshAssetFromMesh(
		UDynamicMesh* FromDynamicMesh, 
		FString AssetPathAndName,
		FGeometryScriptCreateNewStaticMeshAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	 * Create a new StaticMesh asset from a collection of LODs represented by an array of DynamicMeshes.
	 * Save the asset at the AssetPathAndName location.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ScriptName="CreateNewStaticMeshAssetFromMeshLods;CreateNewStaticMeshAssetFromMeshLODs", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Static Mesh Asset") UStaticMesh* 
	CreateNewStaticMeshAssetFromMeshLODs(
		TArray<UDynamicMesh*> FromDynamicMesh, 
		FString AssetPathAndName,
		FGeometryScriptCreateNewStaticMeshAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	 * Create a new SkeletalMesh asset from a DynamicMesh and a Skeleton.
	 * Save the asset at the AssetPathAndName location.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Skeletal Mesh Asset") USkeletalMesh* 
	CreateNewSkeletalMeshAssetFromMesh(
		UDynamicMesh* FromDynamicMesh, 
		USkeleton* InSkeleton,
		FString AssetPathAndName,
		FGeometryScriptCreateNewSkeletalMeshAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);
	
	/**
	 * Create a new SkeletalMesh asset from a collection of LODs represented by an array of DynamicMeshes and a Skeleton.
	 * Save the asset at the AssetPathAndName location.
	 */
	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ScriptName="CreateNewSkeletalMeshAssetFromMeshLods;CreateNewSkeletalMeshAssetFromMeshLODs", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Skeletal Mesh Asset") USkeletalMesh* 
	CreateNewSkeletalMeshAssetFromMeshLODs(
		TArray<UDynamicMesh*> FromDynamicMeshLODs, 
		USkeleton* InSkeleton,
		FString AssetPathAndName,
		FGeometryScriptCreateNewSkeletalMeshAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|AssetManagement", meta = (HidePin = "Debug", ExpandEnumAsExecs = "Outcome"))
	static UE_API UPARAM(DisplayName = "Texture 2D Asset") UTexture2D* 
	CreateNewTexture2DAsset(
		UTexture2D* FromTexture, 
		FString AssetPathAndName,
		FGeometryScriptCreateNewTexture2DAssetOptions Options,
		EGeometryScriptOutcomePins& Outcome,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
