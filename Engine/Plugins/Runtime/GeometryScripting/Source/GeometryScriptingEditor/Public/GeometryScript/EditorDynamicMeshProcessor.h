// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMeshProcessor.h"

#include "EditorDynamicMeshProcessor.generated.h"

// Editor-only variant of UDynamicMeshProcessorBlueprint.
// Blueprints with this parent class can use editor-only blueprint nodes
// (e.g. from GeometryScriptingEditor) that are not available to runtime blueprints.
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable)
class UEditorDynamicMeshProcessorBlueprint : public UDynamicMeshProcessorBlueprint
{
	GENERATED_BODY()
};
