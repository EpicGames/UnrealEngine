// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "EditorDynamicMeshUtilityFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API

namespace UE::Geometry
{ 
	struct FDynamicMeshChangeContainerInternals;
}


/**
 * FDynamicMeshChangeContainer is a temporary struct usable in Blueprints to
 * emit "change" transactions for a UDynamicMesh. The internals are hidden
 * and this type is only intended to be used with the BeginTrackedMeshChange()
 * and EmitTrackedMeshChange() functions
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Dynamic Mesh ChangeTracker"))
struct FDynamicMeshChangeContainer
{
	GENERATED_BODY()
public:
	bool bIsActive = false;

	// This is only used as a token to ensure that the ChangeContainer is not applied to a different UDynamicMesh,
	// so it is not a uproperty/etc
	UDynamicMesh* TargetMesh;

	// internals are only available inside the cpp file and are likely to change
	TSharedPtr<UE::Geometry::FDynamicMeshChangeContainerInternals> ChangeInternals;
};





UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_EditorDynamicMeshUtil"))
class UGeometryScriptLibrary_EditorDynamicMeshFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Save current state of TargetMesh so that an undoable/redoable Change can be emitted 
	 * after TargetMesh is modified, using EmitTrackedMeshChange().
	 * @param ChangeTracker output structure containing initial TargetMesh state
	 */
	UFUNCTION(BlueprintCallable, Category = "DynamicMesh|Changes")
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	BeginTrackedMeshChange(UDynamicMesh* TargetMesh, FDynamicMeshChangeContainer& ChangeTracker);

	/**
	 * Emit an undo/redo Change for a modified TargetMesh, based on the ChangeTracker information that was 
	 * saved (via call to BeginTrackedMeshChange) before TargetMesh was modified. This function must be
	 * called in the context of a Transaction (ie BeginTransaction / EndTransaction pair)
	 */
	UFUNCTION(BlueprintCallable, Category = "DynamicMesh|Changes")
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	EmitTrackedMeshChange(UDynamicMesh* TargetMesh, UPARAM(ref) FDynamicMeshChangeContainer& ChangeTracker);


	/**
	 * Store a copy of TargetMesh with name DebugMeshName.
	 * The mesh can later be recovered via FetchDebugMesh.
	 * @warning This function stores the mesh in a global data structure, the caller must take care to avoid storing large numbers of debug meshes
	 */
	UFUNCTION(BlueprintCallable, Category = "DynamicMesh|Changes")
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	StashDebugMesh(UDynamicMesh* TargetMesh, FString DebugMeshName);


	/**
	 * Fetch a debug FDynamicMesh3 saved with DebugMeshName from the global debug mesh storage and copy to ToTargetMesh.
	 * If DebugMeshName does not exist, a cube will be returned.
	 * @param bClearDebugMesh if true, debug mesh will be removed from global storage
	 * @param bDebugMeshExists will return as true if DebugMeshName existed
	 */
	UFUNCTION(BlueprintCallable, Category = "DynamicMesh|Changes")
	static UE_API UPARAM(DisplayName = "Dynamic Mesh") UDynamicMesh* 
	FetchDebugMesh(FString DebugMeshName, UDynamicMesh* ToTargetMesh, bool bClearDebugMesh, bool& bDebugMeshExists);
};

#undef UE_API
