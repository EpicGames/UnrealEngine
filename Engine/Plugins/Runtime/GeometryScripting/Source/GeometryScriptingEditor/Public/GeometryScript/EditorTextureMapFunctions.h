// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "EditorTextureMapFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API

#if WITH_EDITOR

class UTexture2D;

UENUM(BlueprintType)
enum class EGeometryScriptRGBAChannel : uint8
{
	R = 0,
	G = 1,
	B = 2,
	A = 3,
};

UENUM(BlueprintType)
enum class EGeometryScriptReadGammaSpace : uint8
{
	/* Read color data from Texture directly, without any conversion */
	FromTextureSettings = 0,
	/* Read linear color data from Texture, converting if needed */
	Linear = 1,
	/* Read sRGB color data from Texture, converting if needed */
	SRGB = 2,
};

USTRUCT(BlueprintType)
struct FGeometryScriptChannelPackSource
{
	GENERATED_BODY()

	/** The Texture which should be read/sourced. If null then the DefaultValue is used instead */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> Texture = nullptr;

	/** If Texture is not null, this determines how the color data will be read/sourced */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptReadGammaSpace ReadGammaSpace = EGeometryScriptReadGammaSpace::FromTextureSettings;

	/** If Texture is not null, this determines which channel is read/sourced */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	EGeometryScriptRGBAChannel Channel = EGeometryScriptRGBAChannel::R;

	/** If Texture is null, this value is read/sourced and the Channel and ReadGammaSpace values are ignored */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Advanced)
	float DefaultValue = 255;
};

USTRUCT(BlueprintType)
struct FGeometryScriptChannelPackResult
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TObjectPtr<UTexture2D> Output = nullptr;
};


UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_EditorTextureUtils"))
class UGeometryScriptLibrary_EditorTextureMapFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "GeometryScript|TextureUtils", meta = (HidePin = "Debug"))
	static UE_API FGeometryScriptChannelPackResult ChannelPack(
		FGeometryScriptChannelPackSource RChannelSource,
		FGeometryScriptChannelPackSource GChannelSource,
		FGeometryScriptChannelPackSource BChannelSource,
		FGeometryScriptChannelPackSource AChannelSource,
		bool OutputSRGB,
		UGeometryScriptDebug* Debug = nullptr);
};

#endif // WITH_EDITOR

#undef UE_API
