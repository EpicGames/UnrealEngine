// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "OpenSubdivUtilityFunctions.generated.h"

#define UE_API GEOMETRYSCRIPTINGEDITOR_API

UCLASS(MinimalAPI, meta = (ScriptName = "GeometryScript_OpenSubdiv"))
class UGeometryScriptLibrary_OpenSubdivFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:


	UFUNCTION(BlueprintCallable, Category = "GeometryScript|OpenSubdiv", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply PolyGroup Catmull Clark SubD"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyPolygroupCatmullClarkSubD(
		UDynamicMesh* FromDynamicMesh, 
		int32 Subdivisions,
		FGeometryScriptGroupLayer GroupLayer,
		UGeometryScriptDebug* Debug = nullptr);


	UFUNCTION(BlueprintCallable, Category = "GeometryScript|OpenSubdiv", meta=(ScriptMethod, HidePin = "Debug", DisplayName = "Apply Triangle Loop SubD"))
	static UE_API UPARAM(DisplayName = "Target Mesh") UDynamicMesh* 
	ApplyTriangleLoopSubD(
		UDynamicMesh* FromDynamicMesh, 
		int32 Subdivisions,
		UGeometryScriptDebug* Debug = nullptr);

};

#undef UE_API
